// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GerstnerWaveFunctions.ush: Utility functions for Gerstner waves computation.
=============================================================================*/ 

#pragma once

#include "WaterDataFunctions.ush"

#define Gravity 980
#define SOLVE_NORMAL_Z 1
#define SteepnessThreshold 50

struct WaveOutput
{
	float3 Normal;
	float3 WPO;
};

struct WaveOutputs
{
	WaveOutput LowFrequency;
	WaveOutput MiddleFrequency;
	WaveOutput HighFrequency;
	WaveOutput AllWaves;
};

struct GerstnerWaveRenderer
{
	float2 WaveParamRTSize;
	int MaxWaves;
	float2 WorldPos;
	float Time;
};

FWaveParams GetWaveRTData(GerstnerWaveRenderer inWaveRenderer, Texture2D inWaveParamsRT, SamplerState inWaveParamsRTSampler, int inWaveIndex, int inWaterBodyIndex)
{
	FWaveParams OutWaveParams;
		
	float2 WaveUV;
	float WaveUV_b;
		
	inWaveIndex *= 2;
		
	WaveUV.x = inWaveIndex / inWaveRenderer.WaveParamRTSize.x;
	WaveUV.y = inWaterBodyIndex / inWaveRenderer.WaveParamRTSize.y;
	WaveUV += 0.5 / inWaveRenderer.WaveParamRTSize;
		
	WaveUV_b = WaveUV.x + (1 / inWaveRenderer.WaveParamRTSize.x);
		
	OutWaveParams.Direction = inWaveParamsRT.SampleLevel(inWaveParamsRTSampler, WaveUV, 0).rg;
		
	float3 TempParams = inWaveParamsRT.SampleLevel(inWaveParamsRTSampler, float2(WaveUV_b, WaveUV.y), 0).rgb;
		
	OutWaveParams.Wavelength = TempParams.x;
	OutWaveParams.Amplitude = TempParams.y;
	OutWaveParams.Steepness = TempParams.z;
		
	return OutWaveParams;
}
	
WaveOutput AddWaves(WaveOutput inWaveA, WaveOutput inWaveB)
{
	inWaveA.Normal += inWaveB.Normal;
	inWaveA.WPO += inWaveB.WPO;
		
	return inWaveA;
}
	
float3 FinalizeNormal(float3 inNormal)
{
	return normalize(float3(inNormal.xy, 1 - inNormal.z));
}
	
float3 PackNormalAndWPO(WaveOutput inWave)
{
	float3 packedoutput = 0;
		
	packedoutput = floor(inWave.WPO * 100);
	packedoutput += ((inWave.Normal + 1.01) / 2.02);
	
	return packedoutput;
}
	
float3 UnpackWaveNormal(float inPackedWave)
{
	float3 outnormal = frac(inPackedWave);
	outnormal *= 2.02;
	outnormal -= 1.01;
	return outnormal;
}
	
float3 UnpackWaveWPO(float inPackedWave)
{
	float3 outWPO;
	outWPO = floor(inPackedWave) / 100;
	return outWPO;
}

float2 GetWavePositionAndDispersion(GerstnerWaveRenderer inWaveRenderer, FWaveParams inWaveParams)
{
	float dispersion = 2 * PI / inWaveParams.Wavelength;
	float2 wavevector = inWaveParams.Direction * dispersion;
	float wavespeed = sqrt(dispersion * Gravity);
	float wavetime = wavespeed * inWaveRenderer.Time;
		
	float wavepos = dot(inWaveRenderer.WorldPos, wavevector) - wavetime;
		
	return float2(wavepos, dispersion);
}
	
WaveOutput ComputeSingleWaveVectorized(float inWaveSin, float inWaveCos, float inDispersion, FWaveParams inWaveParams)
{
	WaveOutput OutWave;
		
	float wKA = inWaveParams.Amplitude * inDispersion;
		
	float q = inWaveParams.Steepness / wKA;
		
	OutWave.Normal.xy = inWaveSin * wKA * inWaveParams.Direction;
		
#if SOLVE_NORMAL_Z
	OutWave.Normal.z = inWaveCos * wKA * q;
#else
		OutWave.Normal.z = 0;
#endif

	OutWave.WPO.xy = -q * inWaveSin * inWaveParams.Direction * inWaveParams.Amplitude;
	OutWave.WPO.z = inWaveCos * inWaveParams.Amplitude;

	return OutWave;
}
	
WaveOutput GetSingleGerstnerWave(GerstnerWaveRenderer inWaveRenderer, Texture2D inWaveParamsRT, SamplerState inWaveParamsRTSampler, int inWaveIndex, int inWaterBodyIndex)
{
	WaveOutput OutWave;
		
	FWaveParams CurrentWave;
	CurrentWave = GetWaveRTData(inWaveRenderer, inWaveParamsRT, inWaveParamsRTSampler, inWaveIndex, inWaterBodyIndex);
		
	float dispersion = 2 * PI / CurrentWave.Wavelength;
	float2 wavevector = CurrentWave.Direction * dispersion;
	float wavespeed = sqrt(dispersion * Gravity);
	float wavetime = wavespeed * inWaveRenderer.Time;

	float wavepos = dot(inWaveRenderer.WorldPos, wavevector) - wavetime;
		
	float wavesin = sin(wavepos);
	float wavecos = cos(wavepos);
		
	float wKA = CurrentWave.Amplitude * dispersion;
		
	float q = CurrentWave.Steepness / wKA;
		
	OutWave.Normal.xy = wavesin * wKA * CurrentWave.Direction;
		
#if SOLVE_NORMAL_Z
	OutWave.Normal.z = wavecos * CurrentWave.Steepness * saturate((CurrentWave.Amplitude * SteepnessThreshold) / CurrentWave.Wavelength);
		//OutWave.Normal.z = wavecos *  wKA * (q/MaxWaves);
#else
		OutWave.Normal.z = 0;
#endif

	OutWave.WPO.xy = -q * wavesin * CurrentWave.Direction * CurrentWave.Amplitude;
	OutWave.WPO.z = wavecos * CurrentWave.Amplitude;

	return OutWave;
		
}
	
WaveOutput GetAllGerstnerWaves(GerstnerWaveRenderer inWaveRenderer, Texture2D inWaveParamsRT, SamplerState inWaveParamsRTSampler, int inWaterBodyIndex)
{
		
	WaveOutput OutWaves = (WaveOutput)0;
	WaveOutput CurrentWave;
		
	int MaxWaveRange = inWaveRenderer.MaxWaves - 1;
		
	for (int i = 0; i <= MaxWaveRange; i++)
	{
		CurrentWave = GetSingleGerstnerWave(inWaveRenderer, inWaveParamsRT, inWaveParamsRTSampler, i, inWaterBodyIndex);
		OutWaves = AddWaves(OutWaves, CurrentWave);
	}
		
	//The Normal B channel must be inverted after combining waves
	OutWaves.Normal = FinalizeNormal(OutWaves.Normal);
	return OutWaves;
}
	
WaveOutput GetAllGerstnerWavesVectorized(GerstnerWaveRenderer inWaveRenderer, int inWaterBodyIndex)
{
		
	WaveOutput OutWaves = (WaveOutput)0;
	WaveOutput CurrentWave;
		
	int MaxWaveRange = (inWaveRenderer.MaxWaves / 4) - 1;
	for (int i = 0; i <= MaxWaveRange; i++)
	{
		FWaveParams WaveA;
		FWaveParams WaveB;
		FWaveParams WaveC;
		FWaveParams WaveD;
			
		float2 WaveAPosAndDispersion = GetWavePositionAndDispersion(inWaveRenderer, WaveA);
		float2 WaveBPosAndDispersion = GetWavePositionAndDispersion(inWaveRenderer, WaveB);
		float2 WaveCPosAndDispersion = GetWavePositionAndDispersion(inWaveRenderer, WaveC);
		float2 WaveDPosAndDispersion = GetWavePositionAndDispersion(inWaveRenderer, WaveD);
			
		float4 WaveSines = 0;
		float4 WaveCosines = 0;
			
		WaveSines = sin(float4(WaveAPosAndDispersion.x, WaveBPosAndDispersion.x, WaveCPosAndDispersion.x, WaveDPosAndDispersion.x));
		WaveCosines = cos(float4(WaveAPosAndDispersion.x, WaveBPosAndDispersion.x, WaveCPosAndDispersion.x, WaveDPosAndDispersion.x));
			
		//sincos( float4( WaveAPosAndDispersion.x, WaveBPosAndDispersion.x, WaveCPosAndDispersion.x, WaveDPosAndDispersion.x), WaveSines, WaveCosines);
			
		CurrentWave = ComputeSingleWaveVectorized(WaveSines.x, WaveCosines.x, WaveAPosAndDispersion.y, WaveA);
		OutWaves = AddWaves(OutWaves, CurrentWave);
			
		CurrentWave = ComputeSingleWaveVectorized(WaveSines.y, WaveCosines.y, WaveBPosAndDispersion.y, WaveB);
		OutWaves = AddWaves(OutWaves, CurrentWave);
			
		CurrentWave = ComputeSingleWaveVectorized(WaveSines.z, WaveCosines.z, WaveCPosAndDispersion.y, WaveC);
		OutWaves = AddWaves(OutWaves, CurrentWave);
			
		CurrentWave = ComputeSingleWaveVectorized(WaveSines.a, WaveCosines.a, WaveDPosAndDispersion.y, WaveD);
		OutWaves = AddWaves(OutWaves, CurrentWave);
			
	}

	//The Normal B channel must be inverted after combining waves
	OutWaves.Normal = FinalizeNormal(OutWaves.Normal);
	return OutWaves;
}
	
WaveOutput GetRangeGerstnerWaves(GerstnerWaveRenderer inWaveRenderer, Texture2D inWaveParamsRT, SamplerState inWaveParamsRTSampler, int inMinWaveIndex, int inMaxWaveIndex, int inWaterBodyIndex)
{
	WaveOutput OutWaves;
	WaveOutput CurrentWave;
		
	int MaxWaveRange = min(inMaxWaveIndex, inWaveRenderer.MaxWaves - 1);
		
	for (int i = inMinWaveIndex; i <= MaxWaveRange; i++)
	{
		CurrentWave = GetSingleGerstnerWave(inWaveRenderer, inWaveParamsRT, inWaveParamsRTSampler, i, inWaterBodyIndex);
		OutWaves = AddWaves(OutWaves, CurrentWave);
	}
		
	//The Normal B channel must be inverted after combining waves
	OutWaves.Normal = FinalizeNormal(OutWaves.Normal);
		
	return OutWaves;
}
	
// TODO[ jbard] placeholder : Remove ASAP
float ComputeWaveDepthAttenuationFactor(int InWaterBodyIndex, float InWaterDepth)
{
	return 1.0f;
}

GerstnerWaveRenderer InitializeGerstnerWaveRenderer(float2 inWaveParamsRTSize, int inMaxwaves, float2 inWorldPos, float inTime)
{
	GerstnerWaveRenderer OutWaveRenderer;
	
	OutWaveRenderer.WaveParamRTSize = inWaveParamsRTSize;
	OutWaveRenderer.MaxWaves = inMaxwaves;
	OutWaveRenderer.WorldPos = inWorldPos;
	OutWaveRenderer.Time = inTime;
		
	return OutWaveRenderer;
}

WaveOutput AddWavesNew(WaveOutput inWaveA, WaveOutput inWaveB)
{
	inWaveA.Normal += inWaveB.Normal;
	inWaveA.WPO += inWaveB.WPO;
		
	return inWaveA;
}
	
float3 FinalizeNormalNew(float3 inNormal)
{
	return normalize(float3(inNormal.xy, 1.0f - inNormal.z));
}

FWaveParams GetWaveDataByWaveIndexNew(int InWaterBodyIndex, int InWaveIndex)
{
	const FWaterBodyData WaterBodyData = GetWaterBodyData(InWaterBodyIndex);
	return GetWaveDataNew(InWaveIndex, WaterBodyData);
}

float3 PackNormalAndWPONew(WaveOutput inWave)
{
	float3 packedoutput = (float3)0;
		
	packedoutput = floor(inWave.WPO * 100);
	packedoutput += ((inWave.Normal + 1.01) / 2.02);
	
	return packedoutput;
}

WaveOutput GetSingleGerstnerWaveNew(int InWaveIndex, FWaterBodyData InWaterBodyData, float InTime, FDFVector2 InWorldPos)
{
	WaveOutput OutWave = (WaveOutput)0;
		
	FWaveParams CurrentWave = GetWaveDataNew(InWaveIndex, InWaterBodyData);
		
	float Dispersion = 2 * PI / CurrentWave.Wavelength;
	float2 WaveVector = CurrentWave.Direction * Dispersion;
	float WaveSpeed = sqrt(Dispersion * Gravity);
	float WaveTime = WaveSpeed * InTime;

	// Use the offset of the normalized tile as world position.
	// Doing proper LWC math is both more expensive and leads to less smooth/precise results.
	FLWCVector2 LWCWorldPos = DFToTileOffset_Hack(InWorldPos);
	float2 WorldPos = LWCWorldPos.Offset;	
	
	float WavePos = dot(WorldPos, WaveVector) - WaveTime;
	float WaveSin, WaveCos;
	sincos(WavePos, WaveSin, WaveCos);

	// Fix up tile borders by blending between them
	const float2 TileBorderDist = (UE_LWC_RENDER_TILE_SIZE * 0.5f) - abs(WorldPos);
	const float BlendZoneWidth = 400.0f; // Blend over a range of 4 meters on each side of the tile
	if (any(TileBorderDist < BlendZoneWidth))
	{
		const float2 BlendWorldPos = TileBorderDist;

		const float2 BlendAlpha2 = saturate(TileBorderDist / BlendZoneWidth);
		const float BlendAlpha = BlendAlpha2.x * BlendAlpha2.y;

		const float BlendWavePos = dot(BlendWorldPos, WaveVector) - WaveTime;
		WaveSin = lerp(sin(BlendWavePos), WaveSin, BlendAlpha);
		WaveCos = lerp(cos(BlendWavePos), WaveCos, BlendAlpha);
	}
		
	float wKA = CurrentWave.Amplitude * Dispersion;
		
	float q = CurrentWave.Steepness / wKA;
		
	OutWave.Normal.xy = WaveSin * wKA * CurrentWave.Direction;
		
#if SOLVE_NORMAL_Z
		OutWave.Normal.z = WaveCos *  CurrentWave.Steepness * saturate( (CurrentWave.Amplitude * SteepnessThreshold) / CurrentWave.Wavelength );
		//OutWave.Normal.z = WaveCos *  wKA * (q/MaxWaves);
#else
		OutWave.Normal.z = 0;
#endif

	OutWave.WPO.xy = -q * WaveSin * CurrentWave.Direction *  CurrentWave.Amplitude;
	OutWave.WPO.z = WaveCos * CurrentWave.Amplitude;

	return OutWave;
		
}

WaveOutput GetAllGerstnerWavesNew(int InWaterBodyIndex, float InTime, FDFVector2 InWorldPos)
{		
	WaveOutput OutWaves = (WaveOutput)0;
	WaveOutput CurrentWave = (WaveOutput)0;
	
	const FWaterBodyData WaterBodyData = GetWaterBodyData(InWaterBodyIndex);
	const int NumWaves = WaterBodyData.NumWaves;

	for (int i = 0; i < NumWaves; i++)
	{
		CurrentWave = GetSingleGerstnerWaveNew(i, WaterBodyData, InTime, InWorldPos);
		OutWaves = AddWavesNew(OutWaves, CurrentWave);
	}
		
	//The Normal B channel must be inverted after combining waves
	OutWaves.Normal = FinalizeNormalNew(OutWaves.Normal);
	return OutWaves;
}

WaveOutput GetAllGerstnerWavesNew(int InWaterBodyIndex, float InTime, FLWCVector2 InWorldPos) { return GetAllGerstnerWavesNew(InWaterBodyIndex, InTime, DFFromTileOffset(InWorldPos)); }
	
WaveOutput GetRangeGerstnerWavesNew(int InMinWaveIndex, int InMaxWaveIndex, int InWaterBodyIndex, float InTime, FDFVector2 InWorldPos)
{
	WaveOutput OutWaves = (WaveOutput)0;
	WaveOutput CurrentWave = (WaveOutput)0;
		
	const FWaterBodyData WaterBodyData = GetWaterBodyData(InWaterBodyIndex);
	const int NumWaves = WaterBodyData.NumWaves;

	const int MaxWaveRange = min(InMaxWaveIndex + 1, NumWaves);
	const int MinWaveRange = max(0, InMinWaveIndex);

	for (int i = MinWaveRange; i < MaxWaveRange; i++)
	{
		CurrentWave = GetSingleGerstnerWaveNew(i, WaterBodyData, InTime, InWorldPos);
		OutWaves = AddWavesNew(OutWaves, CurrentWave);
	}
		
	//The Normal B channel must be inverted after combining waves
	OutWaves.Normal = FinalizeNormalNew(OutWaves.Normal);
		
	return OutWaves;
}

WaveOutputs GetPerFrequencyGerstnerWavesNew(int LowFrequencyMaxIndex, int MiddleFrequencyMaxIndex, int InWaterBodyIndex, float InTime, FDFVector2 InWorldPos)
{
	const int NumWaves = GetWaterBodyData(InWaterBodyIndex).NumWaves;
	// Correct bad user input. Further input handling is done internally inside GetRangeGerstnerWavesNew to ensure the index ranges are valid for the number of waves in a given water body.
	MiddleFrequencyMaxIndex = max(LowFrequencyMaxIndex + 1, MiddleFrequencyMaxIndex);
	
	WaveOutputs Result = (WaveOutputs)0;
	Result.LowFrequency = GetRangeGerstnerWavesNew(0, LowFrequencyMaxIndex, InWaterBodyIndex, InTime, InWorldPos);
	Result.MiddleFrequency = GetRangeGerstnerWavesNew(LowFrequencyMaxIndex + 1, MiddleFrequencyMaxIndex, InWaterBodyIndex, InTime, InWorldPos);
	Result.HighFrequency = GetRangeGerstnerWavesNew(MiddleFrequencyMaxIndex + 1, NumWaves, InWaterBodyIndex, InTime, InWorldPos);
	Result.AllWaves = AddWavesNew(Result.LowFrequency, AddWavesNew(Result.HighFrequency, Result.MiddleFrequency));

	return Result;
}

WaveOutputs GetPerFrequencyGerstnerWavesNew(int LowFrequencyMaxIndex, int MiddleFrequencyMaxIndex, int InWaterBodyIndex, float InTime, FLWCVector2 InWorldPos)
{
	return GetPerFrequencyGerstnerWavesNew(LowFrequencyMaxIndex, MiddleFrequencyMaxIndex, InWaterBodyIndex, InTime, DFFromTileOffset(InWorldPos));
}

/** Returns the wave attenuation factor according to a given water body and a given water depth. Should match the CPU version (GetWaveAttenuationFactor) */
float ComputeWaveDepthAttenuationFactorNew(int InWaterBodyIndex, float InWaterDepth)
{
	const FWaterBodyData WaterBodyData = GetWaterBodyData(InWaterBodyIndex);

	// Completely erase waves when there are no waves at all (in case waves are computed on a water body with no waves, like a swimming pool) :
	const float TotalErasureFactor = saturate((float)WaterBodyData.NumWaves);
	const float StrengthCoefficient = exp(-max(InWaterDepth, 0.0f) / (WaterBodyData.TargetWaveMaskDepth / 2.0f));
	return saturate(1.0f - StrengthCoefficient) * TotalErasureFactor;
}

