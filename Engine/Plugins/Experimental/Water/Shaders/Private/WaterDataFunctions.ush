// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if VM_SIMULATION==0

#define PER_WAVE_DATA_SIZE 2
#define PER_WATERZONE_DATA_SIZE 2
#define PER_WATERBODY_DATA_SIZE 2
#define PER_WATERZONEVIEW_DATA_SIZE 1

/** Struct containing the decoded water body data  */	
struct FWaterBodyData
{
	// Index of the water zone which owns this body. Points to the position of the water zone in the zone sub-buffer.
	int WaterZoneIndex;

	// Index into the wave data sub-buffer.
	int WaveDataIndex;

	// Number of waves to be read from the wave data buffer
	int NumWaves;

	// Target Wave Mask Depth
	float TargetWaveMaskDepth;
	
	float3 FixedVelocity;
	float FixedZHeight;
	float FixedWaterDepth;
};

/** Struct containing the decoded per-water zone data. */
struct FWaterZoneData
{
	float2 Location;
	float2 Extent;
	float2 HeightExtent;
	float GroundZMin;
};

struct FWaveParams
{
	float2 Direction;
	float Wavelength;
	float Amplitude;
	float Steepness;
};

struct FWaterZoneViewData
{
	float2 Location;
};

int GetWaterZoneDataOffset()
{
	return (int)View.WaterData[0].x;
}

int GetWaveDataOffset()
{
	return (int)View.WaterData[0].y;
}

int GetWaterZoneViewDataOffset()
{
	return (int)View.WaterData[0].z;
}

int GetNumWaterViewData()
{
	return (int)View.WaterData[0].w;
}

bool IsWaterLocalTessellationEnabled(int InWaterZoneIndex)
{
	const int WaterZoneDataBufferOffset = GetWaterZoneDataOffset();
	const int WaterZoneOffset = WaterZoneDataBufferOffset + (InWaterZoneIndex * PER_WATERZONE_DATA_SIZE);

	return View.WaterData[WaterZoneOffset + 1].y > 0;
}

int GetWaterInfoTextureViewIndex(int InWaterZoneIndex)
{
	int Index = IsWaterLocalTessellationEnabled(InWaterZoneIndex) ? View.WaterInfoTextureViewIndex : 0;
	
	return Index;
}

/** Function to decode the raw data from the water indirection buffer */
FWaterBodyData DecodeWaterBodyData(float4 InDataToDecode0, float4 InDataToDecode1)
{
	FWaterBodyData OutWaterBodyData = (FWaterBodyData)0;
	OutWaterBodyData.WaterZoneIndex = InDataToDecode0.x;
	OutWaterBodyData.WaveDataIndex = InDataToDecode0.y;
	OutWaterBodyData.NumWaves = InDataToDecode0.z;
	OutWaterBodyData.TargetWaveMaskDepth = InDataToDecode0.w;
	OutWaterBodyData.FixedVelocity = float3(f16tof32(asuint(InDataToDecode1.x)), f16tof32(asuint(InDataToDecode1.x) >> 16u), InDataToDecode1.y);
	OutWaterBodyData.FixedZHeight = InDataToDecode1.z;
	OutWaterBodyData.FixedWaterDepth = InDataToDecode1.w;
	
	return OutWaterBodyData;
}

FWaterBodyData GetWaterBodyData(int InWaterBodyIndex)
{
	const int WaterBodyOffset = InWaterBodyIndex * PER_WATERBODY_DATA_SIZE;
	const float4 Data0 = View.WaterIndirection[WaterBodyOffset];
	const float4 Data1 = View.WaterIndirection[WaterBodyOffset + 1];

	return DecodeWaterBodyData(Data0, Data1);
}

/** Function to decode the data for an individual gerstner wave from the water data buffer */
FWaveParams DecodeWaveParams(float4 InDataToDecode0, float4 InDataToDecode1)
{
	FWaveParams OutWaveParams = (FWaveParams)0;
	OutWaveParams.Direction = InDataToDecode0.xy;
	OutWaveParams.Wavelength = InDataToDecode0.z;
	OutWaveParams.Amplitude = InDataToDecode0.w;
	OutWaveParams.Steepness = InDataToDecode1.x;

	return OutWaveParams;
}

FWaveParams GetWaveDataNew(int InWaveIndex, FWaterBodyData InWaterBodyData)
{
	const int WaveDataOffset = GetWaveDataOffset();
	const int AbsoluteWaveDataIndex = WaveDataOffset + (InWaterBodyData.WaveDataIndex + InWaveIndex) * PER_WAVE_DATA_SIZE;
	const float4 Data0 = View.WaterData[AbsoluteWaveDataIndex];
	const float4 Data1 = View.WaterData[AbsoluteWaveDataIndex + 1];
		
	return DecodeWaveParams(Data0, Data1);
}

FWaterZoneData DecodeWaterZoneData(float2 Location, float4 Data0, float4 Data1)
{
	FWaterZoneData OutWaterZoneData = (FWaterZoneData)0;
	OutWaterZoneData.Location = Location;
	OutWaterZoneData.Extent = Data0.xy;
	OutWaterZoneData.HeightExtent = Data0.zw;
	OutWaterZoneData.GroundZMin = Data1.x;

	return OutWaterZoneData;
}

FWaterZoneViewData GetWaterZoneViewData(int InWaterZoneIndex)
{
	const int NumViews = GetNumWaterViewData();

	const int ViewIndex = GetWaterInfoTextureViewIndex(InWaterZoneIndex);

	const int WaterZoneViewDataBufferOffset = GetWaterZoneViewDataOffset();
	const int WaterZoneViewOffset = WaterZoneViewDataBufferOffset + (InWaterZoneIndex * NumViews + ViewIndex) * PER_WATERZONEVIEW_DATA_SIZE;

	FWaterZoneViewData WaterZoneViewData = (FWaterZoneViewData)0;
	WaterZoneViewData.Location = View.WaterData[WaterZoneViewOffset].xy;

	return WaterZoneViewData;
}

FWaterZoneData GetWaterZoneData(int InWaterZoneIndex)
{
	const int WaterZoneDataBufferOffset = GetWaterZoneDataOffset();
	const int WaterZoneOffset = WaterZoneDataBufferOffset + (InWaterZoneIndex * PER_WATERZONE_DATA_SIZE);

	FWaterZoneViewData WaterZoneViewData = GetWaterZoneViewData(InWaterZoneIndex);

	const float2 ZoneViewLocation = WaterZoneViewData.Location;
	
	const float4 Data0 = View.WaterData[WaterZoneOffset];
	const float4 Data1 = View.WaterData[WaterZoneOffset + 1];

	return DecodeWaterZoneData(ZoneViewLocation, Data0, Data1);
}

FWaterZoneData GetWaterZoneDataForWaterBody(int InWaterBodyIndex)
{
	const int WaterZoneIndex = GetWaterBodyData(InWaterBodyIndex).WaterZoneIndex;

	return GetWaterZoneData(WaterZoneIndex);
}

float DecodeWaterInfoGroundHeight(float4 WaterInfoSample, int WaterZoneIndex)
{
	const float GroundZNormalized = WaterInfoSample.w;
	const FWaterZoneData WaterZoneData = GetWaterZoneData(WaterZoneIndex);

	const float GroundZScale = WaterZoneData.HeightExtent.y - WaterZoneData.GroundZMin;

	return GroundZNormalized * GroundZScale + WaterZoneData.GroundZMin;
}

float DecodeWaterInfoZHeight(float4 WaterInfoSample, int WaterZoneIndex)
{
	const float WaterZNormalized = WaterInfoSample.z;
	const FWaterZoneData WaterZoneData = GetWaterZoneData(WaterZoneIndex);

	const float WaterZScale = WaterZoneData.HeightExtent.y - WaterZoneData.HeightExtent.x;
	float WaterZ = WaterZNormalized * WaterZScale + WaterZoneData.HeightExtent.x;

	// If below 0, we need to push the height below the height of the terrain
	if (WaterZNormalized <= 0.0)
	{
		WaterZ = DecodeWaterInfoGroundHeight(WaterInfoSample, WaterZoneIndex) - 256.0;
	}

	return WaterZ;
}

float2 DecodeWaterInfoVelocity(float4 WaterInfoSample, float MaxVelocity)
{
	const float2 NormalizedFlow = WaterInfoSample.xy;
	return (NormalizedFlow - 0.5) * 2.0 * MaxVelocity;
}

#endif