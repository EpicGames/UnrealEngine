// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/DeferredShadingCommon.ush"

Texture2D<float4> WaterInfoTexture;

float WaterZMin;
float WaterZMax;
float GroundZMin;
float CaptureZ;

int BlurRadius;

struct FWaterInfo
{
	float2 Velocity;
	float NormalizedWaterZ;
	float NormalizedGroundZ;
};

FWaterInfo MakeWaterInfo(float4 InSampleData)
{
	FWaterInfo Result;
	Result.Velocity = InSampleData.rg;
	Result.NormalizedWaterZ = InSampleData.b;
	Result.NormalizedGroundZ = InSampleData.a;
	return Result;
}

float2 VelocityBlur(float Radius, uint2 TextureCoordinates)
{
	float2 Accum = 0;

	float Neighbors = 0;

	int2 CurrentTextureCoordinates = int2(TextureCoordinates);

	for (int X = -Radius; X <= Radius; X++)
	{
		for(int Y=-Radius; Y <= Radius; Y++)
		{
			CurrentTextureCoordinates = TextureCoordinates + int2(X, Y);

			const FWaterInfo WaterInfo = MakeWaterInfo(WaterInfoTexture.Load(int3(CurrentTextureCoordinates, 0)));
			
			const float WaterZ = WaterInfo.NormalizedWaterZ * (WaterZMax - WaterZMin) + WaterZMin;
			const float GroundZ = WaterInfo.NormalizedGroundZ * (WaterZMax - GroundZMin) + GroundZMin;

			if (WaterZ > GroundZ)
			{
				Accum.xy += WaterInfo.Velocity;
				Neighbors += 1;
			}
		}
	}

	if(Neighbors > 0)
	{
		Accum.xy /= Neighbors;
	}
	
	return float2(Accum);
}
 
void Main(
	in FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0)
{
	const float4 SVPos = Input.Position;
	uint2 TextureCoordinates = floor(SVPos.xy);
	
	const FWaterInfo WaterInfo = MakeWaterInfo(WaterInfoTexture.Load(int3(TextureCoordinates, 0)));
	const float2 BlurredVelocity = VelocityBlur(BlurRadius, TextureCoordinates);

	OutColor = float4(BlurredVelocity.xy, WaterInfo.NormalizedWaterZ, WaterInfo.NormalizedGroundZ);
}
