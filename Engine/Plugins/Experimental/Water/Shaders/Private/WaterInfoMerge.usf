// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/DeferredShadingCommon.ush"

Texture2D<float4> DepthTexture;
SamplerState DepthTextureSampler;
Texture2D<float4> ColorTexture;
SamplerState ColorTextureSampler;
Texture2D<float4> DilationTexture;
SamplerState DilationTextureSampler;


float2 WaterHeightExtents;
float GroundZMin;
float CaptureZ;
float UndergroundDilationDepthOffset;
float DilationOverwriteMinimumDistance;

struct FWaterColorPassResult
{
	float2 Velocity;
	float SceneDepth;
};

FWaterColorPassResult MakeWaterColorPassResult(float4 InSampleData)
{
	FWaterColorPassResult Result;
	Result.Velocity = InSampleData.rg;
	Result.SceneDepth = InSampleData.a;
	return Result;
}

float NormalizeHeight(float Height, float Min, float Max)
{
	float NormalizedHeight = 1.f;
	// It's possible that the min and max are the same value in which case we might end up generating a div by zero error
	// In this case it's okay to use the Normalized value of 1.0 which will just map to the Min value.
	if (Min != Max)
	{
		NormalizedHeight = saturate((Height - Min) / (Max - Min));
	}
	return NormalizedHeight;
}

void Main(
	in FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
	// All input textures MUST be sampled with a point sampler to avoid a driver issue on older Android devices which occurs when using Load instead.
	const float4 ColorSample = ColorTexture.SampleLevel(ColorTextureSampler, Input.UV, 0);
	const float DepthSample = DepthTexture.SampleLevel(DepthTextureSampler, Input.UV, 0).x;
	const float DilationSample = DilationTexture.SampleLevel(DilationTextureSampler, Input.UV, 0).x;
	
	const FWaterColorPassResult WaterInfo = MakeWaterColorPassResult(ColorSample);
	const float GroundSceneDepth = DepthSample;
	const float GroundZ = max(CaptureZ - GroundSceneDepth, GroundZMin);
	
	const float WaterSceneDepth = WaterInfo.SceneDepth;
	const float WaterZ = CaptureZ - WaterSceneDepth;

	const float WaterZMin = WaterHeightExtents.x;
	const float WaterZMax = WaterHeightExtents.y;

	const float DilationSceneDepth = DilationSample;
	float DilatedZ = CaptureZ - DilationSceneDepth;
	const bool bPixelHasDilationData = DilatedZ <= WaterZMax && DilatedZ >= WaterZMin;

	// Always initialize output to avoid bugs in old shader compilers 
	OutColor = 0.0f;
	
	// If the measured WaterZ is below WaterZMin it means that there is no water at this pixel because it is not possible
	// for water to be further than ZMin.
	bool bPixelHasValidWaterData = WaterZ >= WaterZMin;

	// Additionally, if we have dilation data here and the water data would be placed below the landscape, instead draw the dilation data
	if (bPixelHasDilationData && (WaterZ + 128.0f < GroundZ))
	{
		bPixelHasValidWaterData = false;
	}

	if (bPixelHasValidWaterData)
	{
		const float NormalizedWaterZ = NormalizeHeight(WaterZ, WaterZMin, WaterZMax);
		const float NormalizedGroundZ = NormalizeHeight(GroundZ, GroundZMin, WaterZMax);

		OutColor = float4(WaterInfo.Velocity, NormalizedWaterZ, NormalizedGroundZ);
	}
	else if (bPixelHasDilationData)
	{
		// If the dilated water would render on top of terrain, squash it down below
		if (DilatedZ > GroundZ)
		{
			DilatedZ = GroundZ - UndergroundDilationDepthOffset;
		}

		const float NormalizedWaterZ = NormalizeHeight(DilatedZ, WaterZMin, WaterZMax);
		const float NormalizedGroundZ = NormalizeHeight(GroundZ, GroundZMin, WaterZMax);

		OutColor = float4(0.0f, 0.0f, NormalizedWaterZ, NormalizedGroundZ);
	}
	else
	{
		// Pixel may not have any dilation data nor water data. In this case just write 0s to the texture
		const float NormalizedGroundZ = NormalizeHeight(GroundZ, GroundZMin, WaterZMax);
		OutColor = float4(0.0f, 0.0f, 0.0f, NormalizedGroundZ);
	}
}
