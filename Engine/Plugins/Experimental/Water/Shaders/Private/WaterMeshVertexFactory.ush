// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/VertexFactoryCommon.ush"

#ifndef WATER_MESH_DRAW_INDIRECT
#define WATER_MESH_DRAW_INDIRECT 0
#endif

#define WATER_MESH_DRAW_INDIRECT_INTERNAL WATER_MESH_DRAW_INDIRECT && !RAYHITGROUPSHADER && !COMPUTESHADER

#if WATER_MESH_DRAW_INDIRECT_INTERNAL
float3 QuadTreePosition;
float CaptureDepthRange;
uint bLODMorphingEnabled;

#if INSTANCED_STEREO
Buffer<uint> InstanceDataOffsetsBuffer;
Buffer<uint> InstanceData0Buffer;
Buffer<uint> InstanceData1Buffer;
Buffer<uint> InstanceData2Buffer;
#if WITH_WATER_SELECTION_SUPPORT_VF
Buffer<uint> InstanceData3Buffer;
#endif

uint DrawBucketIndex;
uint StereoPassInstanceFactor;
#endif // INSTANCED_STEREO

#endif // WATER_MESH_DRAW_INDIRECT_INTERNAL

struct FVertexFactoryInterpolantsVSToPS
{
#if NUM_TEX_COORD_INTERPOLATORS
	float4	TexCoords[(NUM_TEX_COORD_INTERPOLATORS+1)/2] : TEXCOORD0;
#endif

	nointerpolation uint WaveParamIndex : WAVE_PARAM_INDEX;

#if VF_USE_PRIMITIVE_SCENE_DATA
	nointerpolation uint PrimitiveId : PRIMITIVE_ID;
#endif
};

/**
 * Per-vertex inputs from bound vertex buffers
 */
struct FVertexFactoryInput
{
	float4	Position	: ATTRIBUTE0;

#if WATER_MESH_DRAW_INDIRECT_INTERNAL
#if !INSTANCED_STEREO // ISR uses manual vertex fetching instead
	uint InstanceData0 : ATTRIBUTE8;
	uint InstanceData1 : ATTRIBUTE9;
	uint InstanceData2 : ATTRIBUTE10;
#if HIT_PROXY_SHADER || WITH_WATER_SELECTION_SUPPORT_VF
	uint InstanceData3 : ATTRIBUTE11;
#endif
#endif // !INSTANCED_STEREO

#else // WATER_MESH_DRAW_INDIRECT_INTERNAL
	float4 InstanceData0 : ATTRIBUTE8;
	float4 InstanceData1 : ATTRIBUTE9; 
#if HIT_PROXY_SHADER || WITH_WATER_SELECTION_SUPPORT_VF
	float4 InstanceData2 : ATTRIBUTE10; // Store an HitProxyID per instance
#endif
#endif // WATER_MESH_DRAW_INDIRECT_INTERNAL

	VF_GPUSCENE_DECLARE_INPUT_BLOCK(13)
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()
};

/** 
 * Per-vertex inputs from bound vertex buffers.  Used by passes with a trimmed down position-only shader.
 */
struct FPositionOnlyVertexFactoryInput
{
	float4	Position	: ATTRIBUTE0;

#if WATER_MESH_DRAW_INDIRECT_INTERNAL
#if !INSTANCED_STEREO // ISR uses manual vertex fetching instead
	uint InstanceData0 : ATTRIBUTE8;
	uint InstanceData1 : ATTRIBUTE9;
	uint InstanceData2 : ATTRIBUTE10;
#endif // !INSTANCED_STEREO

#else // WATER_MESH_DRAW_INDIRECT_INTERNAL
	float4 InstanceData0 : ATTRIBUTE8;
	float4 InstanceData1 : ATTRIBUTE9;
#endif // WATER_MESH_DRAW_INDIRECT_INTERNAL

	VF_GPUSCENE_DECLARE_INPUT_BLOCK(1)
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()
};

/** 
 * Per-vertex inputs from bound vertex buffers.  Used by passes with a trimmed down position-and-normal-only shader.
 */
struct FPositionAndNormalOnlyVertexFactoryInput
{
	float4	Position	: ATTRIBUTE0;
	float4	Normal		: ATTRIBUTE2;

#if WATER_MESH_DRAW_INDIRECT_INTERNAL
#if !INSTANCED_STEREO // ISR uses manual vertex fetching instead
	uint InstanceData0 : ATTRIBUTE8;
	uint InstanceData1 : ATTRIBUTE9;
	uint InstanceData2 : ATTRIBUTE10;
#endif // !INSTANCED_STEREO

#else
	float4 InstanceData0 : ATTRIBUTE8;
	float4 InstanceData1 : ATTRIBUTE9;
#endif // WATER_MESH_DRAW_INDIRECT_INTERNAL

	VF_GPUSCENE_DECLARE_INPUT_BLOCK(1)
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()
};

/** 
 * Caches intermediates that would otherwise have to be computed multiple times.  Avoids relying on the compiler to optimize out redundant operations.
 */
struct FVertexFactoryIntermediates
{
	float3 MorphedTranslatedWorldPos;
	
	uint WaveParamIndex;
	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

uint GetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
#if VF_USE_PRIMITIVE_SCENE_DATA
	return Interpolants.PrimitiveId;
#else
	return 0;
#endif
}

void SetPrimitiveId(inout FVertexFactoryInterpolantsVSToPS Interpolants, uint PrimitiveId)
{
#if VF_USE_PRIMITIVE_SCENE_DATA
	Interpolants.PrimitiveId = PrimitiveId;
#endif
}

#if NUM_TEX_COORD_INTERPOLATORS
float2 GetUV(FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex)
{
	float4 UVVector = Interpolants.TexCoords[UVIndex / 2];
	return UVIndex % 2 ? UVVector.zw : UVVector.xy;
}

void SetUV(inout FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex, float2 InValue)
{
	FLATTEN
	if (UVIndex % 2)
	{
		Interpolants.TexCoords[UVIndex / 2].zw = InValue;
	}
	else
	{
		Interpolants.TexCoords[UVIndex / 2].xy = InValue;
	}
}
#endif

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = GetUV(Interpolants, CoordinateIndex);
	}
#endif	//NUM_MATERIAL_TEXCOORDS

	Result.TwoSidedSign = 1;
	Result.PrimitiveId = GetPrimitiveId(Interpolants);

#if WATER_MESH_FACTORY
	Result.WaterWaveParamIndex = Interpolants.WaveParamIndex;
#endif

	return Result;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	half3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
    
	Result.SceneData = Intermediates.SceneData; 
	
	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);		
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

	Result.TangentToWorld = float3x3(1,0,0,0,1,0,0,0,1);
	Result.PreSkinnedPosition = Input.Position.xyz;
	Result.PreSkinnedNormal = float3(0,0,1);

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	// Water doesn't get texcoords from its input verts, so all the vert texcoords will default to the same world position
	// LWC note: this calculation was optimized to avoid a subtraction, which theoretically could have an impact near tile edges, but testing showed no additional artifacts.
	float3 WorldPosOffset = Intermediates.MorphedTranslatedWorldPos - DFToTileOffset_Hack(ResolvedView.PreViewTranslation).Offset;
	UNROLL
	for(int CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS_VERTEX; CoordinateIndex++)
	{
		// LWC_TODO: Using the LWC Offset here will result in seams at the tile edges.
		Result.TexCoords[CoordinateIndex] = WorldPosOffset.xy;
	}
#endif  //NUM_MATERIAL_TEXCOORDS_VERTEX

#if WATER_MESH_FACTORY
	Result.WaterWaveParamIndex = Intermediates.WaveParamIndex;
#endif

	Result.LWCData = MakeMaterialLWCData(Result);

	return Result;
}

float3 MorphTranslatedWorldPosition(float3 OriginalTranslatedWorldPos, float2 InMorphOriginTWS, float LODLevel, float InLODScale, float2 QuadSize, float HeightLODFactor, out float LODFactor)
{
	float3 TranslatedWorldPos = OriginalTranslatedWorldPos;
	
	float DistanceToVert2D = distance(TranslatedWorldPos.xy, ResolvedView.TranslatedWorldCameraOrigin.xy);

	LODFactor = saturate(DistanceToVert2D / (InLODScale * pow(2.0f, LODLevel)) - 1.0f);
	LODFactor = saturate(HeightLODFactor + LODFactor);

	float2 Grid4 = 4.0f * QuadSize;

	// InMorphOriginTWS is the corner of the node being rendered
	const float2 MorphLocalPos = TranslatedWorldPos.xy - InMorphOriginTWS;

	// Offset is like a UV offset (-0.5 to 0.5) within a quadtree node. This is the scalar used to move the vertex
	float2 Offset = frac(MorphLocalPos / Grid4) - float2(0.5, 0.5);

	// Move every other vert either positive or negative towards a neighboring vert
	// This is the smooth continuous sliding which moved to the next LOD level
	const float MinRadius = 0.26f; 
	if (abs(Offset.x) < MinRadius) 
	{
		TranslatedWorldPos.x += Offset.x * LODFactor * Grid4.x;
	}
	if (abs(Offset.y) < MinRadius) 
	{
		TranslatedWorldPos.y += Offset.y * LODFactor * Grid4.y;
	}

	return TranslatedWorldPos;
}

struct FWaterVertexFactoryInstanceInput
{
	float2 Position;
	float3 Translation;
	uint WaterBodyIndex;
	float LODLevel;
	float2 Scale;
	float HeightLODFactor;
	uint NumQuadsPerTileSide;
	bool bShouldMorph;
	bool bCanMorphTwice;
};

#if WATER_MESH_DRAW_INDIRECT_INTERNAL

FWaterVertexFactoryInstanceInput UnpackWaterVertexFactoryInstanceInput(float4 InPosition, uint InData0, uint InData1, uint InData2)
{
	const uint2 NodeCoord = uint2(InData0 & 0x7FFu, (InData0 >> 11u) & 0x7FFu);
	const uint RawLODLevel = (InData0 >> 22u) & 0x1Fu;
	const uint RawDensityIndex = (InData0 >> 27u) & 0x1Fu;
	const uint LODLevel = RawLODLevel + RawDensityIndex;
	const uint DensityIndex = min(RawDensityIndex, (uint)WaterVF.NumDensities - 1);
	const float WaterHeight = f16tof32(InData1) * CaptureDepthRange;
	const uint2 TileCoord = uint2((InData1 >> 16u) & 0xFFu, (InData1 >> 24u) & 0xFFu);
	const float HeightLODFactor = float(InData2 & 0xFFu) / 255.0f;
	const uint WaterBodyIndex = InData2 >> 8u;
	const float Scale = pow(2.0f, (float)RawLODLevel) * WaterVF.LeafSize;

	bool bKilledTriangle = false;

	// Shift and scale the tile according to its tile coord within the node and kill superfluous triangles
	const uint NumQuadsPerLogicalTileSide = max(2u, (uint)WaterVF.NumQuadsLOD0 >> DensityIndex);
	{
		if ((uint)WaterVF.NumQuadsPerTileSide > NumQuadsPerLogicalTileSide)
		{
			InPosition.xy *= (float)WaterVF.NumQuadsPerTileSide / (float)NumQuadsPerLogicalTileSide;
			if (any(InPosition.xy < -0.5f) || any(InPosition.xy > 0.5f))
			{
				// Kill triangle by setting position to NaN
				InPosition = asfloat(0xFFFFFFFF).xxxx;
				bKilledTriangle = true;
			}
		}

		const uint NumTiles = max((uint)WaterVF.NumQuadsPerTileSide, NumQuadsPerLogicalTileSide) / (uint)WaterVF.NumQuadsPerTileSide;
		const float PositionScale = 1.0f / NumTiles;
		const float2 PositionBias = TileCoord * PositionScale;
		InPosition.xy += 0.5f;
		InPosition.xy = InPosition.xy * PositionScale + PositionBias;
		InPosition.xy -= 0.5f;
	}

	FWaterVertexFactoryInstanceInput Result = (FWaterVertexFactoryInstanceInput)0;
	Result.Position = InPosition.xy;
	Result.Translation = float3((NodeCoord + 0.5f) * Scale, WaterHeight) + QuadTreePosition;
	Result.WaterBodyIndex = WaterBodyIndex;
	Result.LODLevel = (float)LODLevel;
	Result.Scale = Scale.xx;
	Result.NumQuadsPerTileSide = NumQuadsPerLogicalTileSide;
	Result.HeightLODFactor = HeightLODFactor;
	// Only allow a tile to morph if it's not the last density level and not the last LOD level, since there is no next level to morph to
	Result.bShouldMorph = !bKilledTriangle && ((bLODMorphingEnabled != 0) && (DensityIndex != (uint)WaterVF.NumDensities - 1)) ? 1 : 0;
	// Tiles can morph twice to be able to morph between 3 LOD levels. Next to last density level can only morph once
	Result.bCanMorphTwice = !bKilledTriangle && (DensityIndex < (uint)WaterVF.NumDensities - 2) ? 1 : 0;

	return Result;
}

#else // WATER_MESH_DRAW_INDIRECT_INTERNAL

FWaterVertexFactoryInstanceInput UnpackWaterVertexFactoryInstanceInput(float4 InPosition, float4 InData0, float4 InData1)
{
	const uint PackedDataChannel = asuint(InData1.x);

	FWaterVertexFactoryInstanceInput Result = (FWaterVertexFactoryInstanceInput)0;
	Result.Position = InPosition.xy;
	Result.Translation = InData0.xyz;
	Result.WaterBodyIndex = asuint(InData0.w);
	Result.LODLevel = (float)(PackedDataChannel & 0xFF);
	Result.Scale = InData1.zw;
	Result.HeightLODFactor = InData1.y;
	Result.NumQuadsPerTileSide = (uint)WaterVF.NumQuadsPerTileSide;
	Result.bShouldMorph = ((PackedDataChannel >> 8u) & 0x1u) != 0;
	Result.bCanMorphTwice = ((PackedDataChannel >> 9u) & 0x1u) != 0;

	return Result;
}

#endif // WATER_MESH_DRAW_INDIRECT_INTERNAL

FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates;

#if WATER_MESH_DRAW_INDIRECT_INTERNAL
#if INSTANCED_STEREO
	const uint InstanceIndex = GetInstanceIdFromVF(Input) / StereoPassInstanceFactor;
	const uint InstanceDataIndex = InstanceDataOffsetsBuffer.Load(DrawBucketIndex) + InstanceIndex;
	const uint InstanceData0 = InstanceData0Buffer.Load(InstanceDataIndex);
	const uint InstanceData1 = InstanceData1Buffer.Load(InstanceDataIndex);
	const uint InstanceData2 = InstanceData2Buffer.Load(InstanceDataIndex);
#else
	const uint InstanceData0 = Input.InstanceData0;
	const uint InstanceData1 = Input.InstanceData1;
	const uint InstanceData2 = Input.InstanceData2;
#endif

	const FWaterVertexFactoryInstanceInput InstanceInput = UnpackWaterVertexFactoryInstanceInput(Input.Position, InstanceData0, InstanceData1, InstanceData2);
#else
	const FWaterVertexFactoryInstanceInput InstanceInput = UnpackWaterVertexFactoryInstanceInput(Input.Position, Input.InstanceData0, Input.InstanceData1);
#endif

	Intermediates.WaveParamIndex = InstanceInput.WaterBodyIndex;

	// Calculate the world pos
	float3 TranslatedWorldPosition = float3(InstanceInput.Position.xy * InstanceInput.Scale, 0.0f) + InstanceInput.Translation;

	Intermediates.SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	
	if (InstanceInput.bShouldMorph)
	{
		// Factor that applies across the entire lowest LOD, based on height above water
		// Then this reaches 1.0, the lowest LOD will pop out and be replaced by the next LOD, starting over at 0.0. Rinse and repeat.
		const float HeightLODFactor = InstanceInput.HeightLODFactor;
	
		const float2 QuadSize = InstanceInput.Scale.xy / InstanceInput.NumQuadsPerTileSide;
	
		float OutLODFactor = 0.0f;
		float3 TempTranslatedWorldPosition = MorphTranslatedWorldPosition(TranslatedWorldPosition, InstanceInput.Translation.xy - InstanceInput.Scale.xy*0.5f, InstanceInput.LODLevel, WaterVF.LODScale, QuadSize, HeightLODFactor, OutLODFactor);
	
		// If the vert is fully morphed, we morph it again as if it was a vert from the next LOD. This effectively means one tile can morph between 3 different LOD levels. 
		// This is needed because the current LOD level tile will stick in to the next LOD level since we move the observer around. Make sure to not add any heightLODFactor to this, since that only applies to lowest LOD
		if (OutLODFactor >= 1.0f && InstanceInput.bCanMorphTwice)
		{
			TempTranslatedWorldPosition = MorphTranslatedWorldPosition(TempTranslatedWorldPosition, InstanceInput.Translation.xy - InstanceInput.Scale.xy*0.5f, InstanceInput.LODLevel + 1, WaterVF.LODScale, QuadSize * 2.0f, 0.0f, OutLODFactor);
		}
	
		Intermediates.MorphedTranslatedWorldPos = TempTranslatedWorldPosition;
	}
	else
	{
		Intermediates.MorphedTranslatedWorldPos = TranslatedWorldPosition;
	}

#if WITH_WATER_SELECTION_SUPPORT_VF
#if WATER_MESH_DRAW_INDIRECT_INTERNAL
	#if INSTANCED_STEREO
	float SelectedValue = UnpackRGBA8(InstanceData3Buffer.Load(InstanceDataIndex)).w;
	#else
	float SelectedValue = UnpackRGBA8(Input.InstanceData3).w;
	#endif // INSTANCED_STEREO
#else // WATER_MESH_DRAW_INDIRECT_INTERNAL
	float SelectedValue = Input.InstanceData2.w;
#endif
	float IsVisible = WaterVF.bRenderSelected * SelectedValue + WaterVF.bRenderUnselected * (1-SelectedValue);
	Intermediates.MorphedTranslatedWorldPos *= IsVisible;
#endif // WITH_WATER_SELECTION_SUPPORT_VF

	return Intermediates;
}

#if HIT_PROXY_SHADER

#define VF_IMPLEMENTED_GET_INSTANCE_HIT_PROXY_ID

float4 VertexFactoryGetInstanceHitProxyId(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
#if WITH_WATER_SELECTION_SUPPORT_VF
#if WATER_MESH_DRAW_INDIRECT_INTERNAL
	#if INSTANCED_STEREO
	const uint InstanceIndex = GetInstanceIdFromVF(Input) / StereoPassInstanceFactor;
	const uint InstanceDataIndex = InstanceDataOffsetsBuffer.Load(DrawBucketIndex) + InstanceIndex;
	return float4(UnpackRGBA8(InstanceData3Buffer.Load(InstanceDataIndex)).bgr, 0);
	#else
	return float4(UnpackRGBA8(Input.InstanceData3).bgr, 0);
	#endif // INSTANCED_STEREO
#else // WATER_MESH_DRAW_INDIRECT_INTERNAL
	return float4(Input.InstanceData2.rgb, 0);
#endif // WATER_MESH_DRAW_INDIRECT_INTERNAL
#else // WITH_WATER_SELECTION_SUPPORT_VF
	return 0;
#endif // !WITH_WATER_SELECTION_SUPPORT_VF
}
#endif

/**
* Get the 3x3 tangent basis vectors for this vertex factory
* this vertex factory will calculate the binormal on-the-fly
*
* @param Input - vertex input stream structure
* @return 3x3 matrix
*/
half3x3 VertexFactoryGetTangentToLocal( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	return half3x3(1,0,0,0,1,0,0,0,1);
}

// @return translated world position
float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return float4(Intermediates.MorphedTranslatedWorldPos, 1);
}

float4 TransformTranslatedWorldToLocal(float4 TranslatedWorldPosition, FDFInverseMatrix WorldToLocal)
{
	float4x4 TranslatedWorldToLocal = DFFastToTranslatedWorld(WorldToLocal, ResolvedView.PreViewTranslation);
	float4 LocalPosition = mul(TranslatedWorldPosition, TranslatedWorldToLocal);
	return LocalPosition;
}

// local position relative to instance
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	float4 PosPrimSpace = TransformTranslatedWorldToLocal(VertexFactoryGetWorldPosition(Input, Intermediates), Intermediates.SceneData.Primitive.WorldToLocal);
	return PosPrimSpace.xyz; // No support for instancing, so instance == primitive
}

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 InWorldPosition)
{
	return InWorldPosition;
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;

	// Initialize the whole struct to 0
	// Really only the last two components of the packed UVs have the opportunity to be uninitialized
	Interpolants = (FVertexFactoryInterpolantsVSToPS)0;

#if  NUM_TEX_COORD_INTERPOLATORS
	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);
	
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		SetUV(Interpolants, CoordinateIndex, CustomizedUVs[CoordinateIndex]);
	}
#endif

	Interpolants.WaveParamIndex = Intermediates.WaveParamIndex;

	SetPrimitiveId(Interpolants, Intermediates.SceneData.PrimitiveId);

	return Interpolants;
}

/** for depth-only pass */
float4 VertexFactoryGetWorldPosition(FPositionOnlyVertexFactoryInput Input)
{
	return Input.Position;
}

// @return previous translated world position
float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return float4(Intermediates.MorphedTranslatedWorldPos + DFFastLocalSubtractDemote(ResolvedView.PrevPreViewTranslation, ResolvedView.PreViewTranslation), 1);
}

// local position relative to instance
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
}

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(GetPrimitiveId(Interpolants));
	return float4(DFFastToTranslatedWorld(PrimitiveData.ObjectWorldPosition, ResolvedView.PreViewTranslation), PrimitiveData.ObjectRadius);
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return GetPrimitiveId(Interpolants);
}

float3 VertexFactoryGetWorldNormal(FPositionAndNormalOnlyVertexFactoryInput Input)
{
	return Input.Normal.xyz;
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	// TODO: Central differencing to figure out the normal
	return float3(0.0f, 0.0f, 1.0f);
}

#if RAYHITGROUPSHADER
FVertexFactoryInput LoadVertexFactoryInputForHGS(uint TriangleIndex, int VertexIndex)
{
	FTriangleBaseAttributes TriangleAttributes = LoadTriangleBaseAttributes(TriangleIndex);

	FVertexFactoryInput Input = (FVertexFactoryInput)0;

	Input.Position = float4(TriangleAttributes.LocalPositions[VertexIndex], 1.0f);

	Input.InstanceData0 = WaterRaytracingVF.InstanceData0;
	Input.InstanceData1 = WaterRaytracingVF.InstanceData1;
#if WITH_WATER_SELECTION_SUPPORT_VF
	Input.InstanceData2 = 0.0f;
#endif

	VF_GPUSCENE_SET_INPUT_FOR_RT(Input, GetInstanceUserData(), 0U);

	return Input;
}
#endif

#if COMPUTESHADER
FVertexFactoryInput LoadVertexFactoryInputForDynamicUpdate(uint TriangleIndex, int VertexIndex, uint PrimitiveId, uint DrawInstanceId)
{
	FVertexFactoryInput Input = (FVertexFactoryInput)0;

	const uint VertexId = TriangleIndex * 3 + VertexIndex;
	const uint VertexOffset = VertexId * 4;
	Input.Position.x = WaterRaytracingVF.VertexBuffer[VertexOffset + 0];
	Input.Position.y = WaterRaytracingVF.VertexBuffer[VertexOffset + 1];
	Input.Position.z = WaterRaytracingVF.VertexBuffer[VertexOffset + 2];
	Input.Position.w = WaterRaytracingVF.VertexBuffer[VertexOffset + 3];

	Input.InstanceData0 = WaterRaytracingVF.InstanceData0;
	Input.InstanceData1 = WaterRaytracingVF.InstanceData1;
#if WITH_WATER_SELECTION_SUPPORT_VF
	Input.InstanceData2 = 0.0f;
#endif
	
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(PrimitiveId);
	VF_GPUSCENE_SET_INPUT_FOR_RT(Input, PrimitiveData.InstanceSceneDataOffset + DrawInstanceId, DrawInstanceId);

	return Input;
}

uint GetNumRayTracingDynamicMeshVerticesIndirect()
{
	return 0;
}
#endif // RAYHITGROUP || COMPUTESHADER

#if NEEDS_VERTEX_FACTORY_INTERPOLATION
	struct FVertexFactoryRayTracingInterpolants
	{
		FVertexFactoryInterpolantsVSToPS InterpolantsVSToPS;
	};

	float2 VertexFactoryGetRayTracingTextureCoordinate( FVertexFactoryRayTracingInterpolants Interpolants )
	{
#if NUM_MATERIAL_TEXCOORDS
		return Interpolants.InterpolantsVSToPS.TexCoords[0].xy;
#else // #if NUM_MATERIAL_TEXCOORDS
		return float2(0,0);
#endif // #if NUM_MATERIAL_TEXCOORDS
	}

	FVertexFactoryInterpolantsVSToPS VertexFactoryAssignInterpolants(FVertexFactoryRayTracingInterpolants Input)
	{
		return Input.InterpolantsVSToPS;
	}

	FVertexFactoryRayTracingInterpolants VertexFactoryGetRayTracingInterpolants(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
	{
		FVertexFactoryRayTracingInterpolants Interpolants;
		Interpolants.InterpolantsVSToPS = VertexFactoryGetInterpolantsVSToPS(Input, Intermediates, VertexParameters);
		return Interpolants;
	}

	FVertexFactoryRayTracingInterpolants VertexFactoryInterpolate(FVertexFactoryRayTracingInterpolants a, float aInterp, FVertexFactoryRayTracingInterpolants b, float bInterp)
	{
		FVertexFactoryRayTracingInterpolants O = a;
	
#if INTERPOLATE_MEMBER
		INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords);
#endif

		return O;
	}
#endif // #if NEEDS_VERTEX_FACTORY_INTERPOLATION

#include "/Engine/Private/VertexFactoryDefaultInterface.ush"
