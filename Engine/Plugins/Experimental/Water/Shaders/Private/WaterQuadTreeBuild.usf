// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/PackUnpack.ush"
#include "WaterQuadTreeCommon.ush"

Texture2D<float4> QuadTreeTexture;
Texture2D<float4> WaterZBoundsTexture;
int InputMipLevelIndex;

void Main(in float4 Position : SV_Position, out float4 OutColor : SV_Target0, out float4 OutWaterZBounds : SV_Target1)
{
	const int2 BaseCoord = int2(Position.xy) * 2;

	uint MinWaterBodyRenderDataIndex = 0xFFFFFFFFu;
	uint MaxWaterBodyRenderDataIndex = 0x0u;
	uint MinTransitionWaterBodyRenderDataIndex = 0xFFFFFFFFu;
	uint MaxTransitionWaterBodyRenderDataIndex = 0x0u;
	bool bHasCompleteSubtree = true;
	float MinZ = 1.0f;
	float MaxZ = 0.0f;
	float SurfaceBaseHeightSum = 0.0f;

	// Downsample from lower mip
	for (int y = 0; y < 2; ++y)
	{
		for (int x = 0; x < 2; ++x)
		{
			const float4 QuadTreeSample = QuadTreeTexture.Load(int3(BaseCoord + int2(x, y), InputMipLevelIndex));
			const FWaterQuadTreeNode ChildNode = WaterQuadTreeUnpackNodeRGBA8(QuadTreeSample);
			
			bHasCompleteSubtree = bHasCompleteSubtree && ChildNode.bHasCompleteSubtree;
			MinWaterBodyRenderDataIndex = min(MinWaterBodyRenderDataIndex, ChildNode.WaterBodyRenderDataIndex);
			MaxWaterBodyRenderDataIndex = max(MaxWaterBodyRenderDataIndex, ChildNode.WaterBodyRenderDataIndex);
			MinTransitionWaterBodyRenderDataIndex = min(MinTransitionWaterBodyRenderDataIndex, ChildNode.TransitionWaterBodyRenderDataIndex);
			MaxTransitionWaterBodyRenderDataIndex = max(MaxTransitionWaterBodyRenderDataIndex, ChildNode.TransitionWaterBodyRenderDataIndex);

			const float3 WaterZBoundsSample = WaterZBoundsTexture.Load(int3(BaseCoord + int2(x, y), InputMipLevelIndex)).xyz;
			MinZ = min(MinZ, WaterZBoundsSample.x);
			MaxZ = max(MaxZ, WaterZBoundsSample.y);
			SurfaceBaseHeightSum += WaterZBoundsSample.z;
		}
	}

	const bool bCanMerge = (MinWaterBodyRenderDataIndex == MaxWaterBodyRenderDataIndex) && (MinTransitionWaterBodyRenderDataIndex == MaxTransitionWaterBodyRenderDataIndex) && MaxWaterBodyRenderDataIndex > 0;

	FWaterQuadTreeNode ResultNode = (FWaterQuadTreeNode)0;
	ResultNode.WaterBodyRenderDataIndex = bCanMerge ? MinWaterBodyRenderDataIndex : 0;
	ResultNode.TransitionWaterBodyRenderDataIndex = bCanMerge ? MinTransitionWaterBodyRenderDataIndex : 0;
	ResultNode.bHasCompleteSubtree = bHasCompleteSubtree;
	ResultNode.bIsSubtreeSameWaterBody = bCanMerge;

	OutColor = WaterQuadTreePackNodeRGBA8(ResultNode);
	OutWaterZBounds = bCanMerge ? float4(MinZ, MaxZ, SurfaceBaseHeightSum * 0.25f, 1.0f) : float4(0.0f, 0.0f, 0.0f, 1.0f);
}