// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/PackUnpack.ush"

#define WATER_BODY_TYPE_RIVER 0
#define WATER_BODY_TYPE_LAKE 1
#define WATER_BODY_TYPE_OCEAN 2

struct FWaterBodyRenderData
{
	uint WaterBodyIndex;
	uint MaterialIndex;
	uint RiverToLakeMaterialIndex;
	uint RiverToOceanMaterialIndex;
	uint WaterBodyType;
	uint HitProxyColorAndIsSelected;
	float SurfaceBaseHeight;
	float MinZ;
	float MaxZ;
	float MaxWaveHeight;
};

struct FWaterQuadTreeNode
{
	uint WaterBodyRenderDataIndex;
	uint TransitionWaterBodyRenderDataIndex;
	bool bHasCompleteSubtree;
	bool bIsSubtreeSameWaterBody;
};

uint WaterQuadTreePackNode(FWaterQuadTreeNode Node)
{
	uint Result = 0;
	Result |= Node.WaterBodyRenderDataIndex & 0x7FFFu; // 15 bits
	Result |= (Node.TransitionWaterBodyRenderDataIndex & 0x7FFFu) << 15u;
	Result |= uint(Node.bHasCompleteSubtree ? 1 : 0) << 30u;
	Result |= uint(Node.bIsSubtreeSameWaterBody ? 1 : 0) << 31u;

	return Result;
}

float4 WaterQuadTreePackNodeRGBA8(FWaterQuadTreeNode Node)
{
	uint PackedUI = WaterQuadTreePackNode(Node);
	float4 Packed4;
	Packed4.x = ((PackedUI >> 0u) & 0xFFu) / 255.0f;
	Packed4.y = ((PackedUI >> 8u) & 0xFFu) / 255.0f;
	Packed4.z = ((PackedUI >> 16u) & 0xFFu) / 255.0f;
	Packed4.w = ((PackedUI >> 24u) & 0xFFu) / 255.0f;
	return Packed4;
}

FWaterQuadTreeNode WaterQuadTreeUnpackNode(uint Packed)
{
	FWaterQuadTreeNode Result;
	Result.WaterBodyRenderDataIndex = Packed & 0x7FFFu;
	Result.TransitionWaterBodyRenderDataIndex = (Packed >> 15u) & 0x7FFFu;
	Result.bHasCompleteSubtree = (Packed & (1u << 30u)) != 0;
	Result.bIsSubtreeSameWaterBody = (Packed & (1u << 31u)) != 0;

	return Result;
}

FWaterQuadTreeNode WaterQuadTreeUnpackNodeRGBA8(float4 Packed)
{
	uint PackedUI = 0;
	PackedUI |= uint(saturate(Packed.x) * 255.0f + 0.5f) << 0u;
	PackedUI |= uint(saturate(Packed.y) * 255.0f + 0.5f) << 8u;
	PackedUI |= uint(saturate(Packed.z) * 255.0f + 0.5f) << 16u;
	PackedUI |= uint(saturate(Packed.w) * 255.0f + 0.5f) << 24u;
	return WaterQuadTreeUnpackNode(PackedUI);
}
