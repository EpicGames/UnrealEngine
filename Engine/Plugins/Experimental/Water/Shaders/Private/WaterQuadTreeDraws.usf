// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/PackUnpack.ush"
#include "WaterQuadTreeCommon.ush"

uint PackNodeCoord(uint4 InNode)
{
	uint Result = 0;
	Result |= uint(InNode.x & 0x7FFu);
	Result |= uint(InNode.y & 0x7FFu) << 11u;
	Result |= uint(InNode.z & 0x1Fu) << 22u;
	Result |= uint(InNode.w & 0x1Fu) << 27u;
	return Result;
}

uint4 UnpackNodeCoord(uint InPacked)
{
	uint4 Result;
	Result.x = InPacked & 0x7FFu;
	Result.y = (InPacked >> 11u) & 0x7FFu;
	Result.z = (InPacked >> 22u) & 0x1Fu;
	Result.w = (InPacked >> 27u) & 0x1Fu;
	return Result;
}

struct FWaterLODParams
{
	int LowestLOD;
	float HeightLODFactor;
};

FWaterLODParams GetWaterLODParams(float InObserverHeight, float InWaterHeightForLOD, float InLODScale, uint InTreeDepth)
{
	float DistToWater = abs(InObserverHeight - InWaterHeightForLOD) / InLODScale;
	DistToWater = max(DistToWater - 2.0f, 0.0f);
	DistToWater *= 2.0f;

	const float FloatLOD = clamp(log2(DistToWater), 0.0f, InTreeDepth - 1.0f);

	FWaterLODParams WaterLODParams;
	WaterLODParams.LowestLOD = clamp(floor(FloatLOD), 0, InTreeDepth - 1);
	WaterLODParams.HeightLODFactor = frac(FloatLOD);

	return WaterLODParams;
}

float4 GetNodeAABB2D(uint3 InNodeCoord, float3 InQuadTreePosition, float InLeafSize)
{
	const float Scale = (1u << InNodeCoord.z) * InLeafSize;
	return float4(InNodeCoord.xy, InNodeCoord.xy + 1.0f) * Scale + InQuadTreePosition.xyxy;
}

#ifdef INITIALIZE_INDIRECT_ARGS

#ifndef PRECISE_OCCLUSION_QUERIES
#define PRECISE_OCCLUSION_QUERIES 0
#endif

RWBuffer<uint> IndirectArgs;
#if PRECISE_OCCLUSION_QUERIES
RWBuffer<uint> OcclusionQueryArgs;
#endif
uint NumDrawBuckets;
uint NumViews;
uint NumQuads;

[numthreads(64, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint BucketIndex = DispatchThreadId.x;

	if (BucketIndex >= NumDrawBuckets)
	{
		return;
	}

	const uint IndexCountPerInstance = NumQuads * NumQuads * 6;

	for (uint ViewIndex = 0; ViewIndex < NumViews; ++ViewIndex)
	{
		const uint IndirectArgIndex = NumDrawBuckets * ViewIndex + BucketIndex;
		IndirectArgs[IndirectArgIndex * 5 + 0] = IndexCountPerInstance;
		IndirectArgs[IndirectArgIndex * 5 + 1] = 0; // InstanceCount
		IndirectArgs[IndirectArgIndex * 5 + 2] = 0; // StartIndexLocation
		IndirectArgs[IndirectArgIndex * 5 + 3] = 0; // BaseVertexLocation
		IndirectArgs[IndirectArgIndex * 5 + 4] = 0; // StartInstanceLocation
	}

#if PRECISE_OCCLUSION_QUERIES
	if (all(DispatchThreadId == 0))
	{
		for (uint ViewIndex = 0; ViewIndex < NumViews; ++ViewIndex)
		{
			OcclusionQueryArgs[ViewIndex * 5 + 0] = 36;// IndexCountPerInstance; 12 triangles per cube
			OcclusionQueryArgs[ViewIndex * 5 + 1] = 0; // InstanceCount
			OcclusionQueryArgs[ViewIndex * 5 + 2] = 0; // StartIndexLocation
			OcclusionQueryArgs[ViewIndex * 5 + 3] = 0; // BaseVertexLocation
			OcclusionQueryArgs[ViewIndex * 5 + 4] = 0; // StartInstanceLocation
		}
	}
#endif // PRECISE_OCCLUSION_QUERIES
}

#endif // INITIALIZE_INDIRECT_ARGS

#ifdef CLEAR_PER_VIEW_BUFFERS

RWByteAddressBuffer BucketCounts;
RWByteAddressBuffer PackedNodes;
uint NumDrawBuckets;

[numthreads(64, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (all(DispatchThreadId == 0))
	{
		// PackedNodes stores a counter at index 0 which we need to clear.
		PackedNodes.Store(0, 0);
	}

	if (DispatchThreadId.x < NumDrawBuckets)
	{
		BucketCounts.Store(DispatchThreadId.x << 2u, 0);
	}
}

#endif // CLEAR_PER_VIEW_BUFFERS

#ifdef QUAD_TREE_TRAVERSE

#include "/Engine/Private/Nanite/NaniteHZBCull.ush" // BoxCullFrustum()

#ifndef PRECISE_OCCLUSION_QUERIES
#define PRECISE_OCCLUSION_QUERIES 0
#endif

RWByteAddressBuffer PackedNodes;

#if PRECISE_OCCLUSION_QUERIES
RWBuffer<float4> OcclusionQueryBoxes;
RWBuffer<uint> OcclusionVisibility;
RWBuffer<uint> OcclusionQueryArgs;
#endif // PRECISE_OCCLUSION_QUERIES

Texture2D<float4> QuadTreeTexture;
Texture2D<float4> WaterZBoundsTexture;
StructuredBuffer<FWaterBodyRenderData> WaterBodyRenderData;
float4 CullingBoundsAABB;
float3 QuadTreePosition;
float3 ObserverPosition;
uint QuadTreeResolutionX;
uint QuadTreeResolutionY;
uint ViewIndex;
float LeafSize;
float LODScale;
float CaptureDepthRange;
int ForceCollapseDensityLevel;
uint NumLODs;
uint NumDispatchedThreads;
uint bHZBOcclusionCullingEnabled;

float ComputeSquaredDistanceToPoint(float2 InBox2DMin, float2 InBox2DMax, float2 InPoint)
{
	// Accumulates the distance as we iterate axis
	float DistSquared = 0.0f;

	if (InPoint.x < InBox2DMin.x)
	{
		DistSquared += Square(InPoint.x - InBox2DMin.x);
	}
	else if (InPoint.x > InBox2DMax.x)
	{
		DistSquared += Square(InPoint.x - InBox2DMax.x);
	}

	if (InPoint.y < InBox2DMin.y)
	{
		DistSquared += Square(InPoint.y - InBox2DMin.y);
	}
	else if (InPoint.y > InBox2DMax.y)
	{
		DistSquared += Square(InPoint.y - InBox2DMax.y);
	}

	return DistSquared;
}

float GetLODDistance(int InLODLevel, float InLODScale)
{
	return pow(2.0f, (float)(InLODLevel + 1)) * InLODScale;
}

bool CanRender(FWaterQuadTreeNode Node, int InDensityLevel, int InForceCollapseDensityLevel)
{
	int MaterialIndex = -1;
	// There is a dummy entry at index 0 but we know that it has no valid MaterialIndex, so we can skip this case
	if (Node.WaterBodyRenderDataIndex > 0)
	{
		const FWaterBodyRenderData WBRenderData = WaterBodyRenderData[Node.WaterBodyRenderDataIndex];
		MaterialIndex = WBRenderData.MaterialIndex;
	}
	// Can render if the density level is (in addition to same water bodies in all descendants) either above the force collapse level or if the subtree is complete
	return MaterialIndex >= 0 && Node.bIsSubtreeSameWaterBody && ((InDensityLevel > InForceCollapseDensityLevel) || Node.bHasCompleteSubtree);
}

[numthreads(64, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (DispatchThreadId.x >= NumDispatchedThreads)
	{
		return;
	}

	ResolvedView = ResolveView();

	// Compute which LOD level this thread belongs to and then find the node index relative to that LOD level
	uint3 NodeCoord = 0;
	{
		uint LODLevelIndex = 0;
		uint2 Resolution = uint2(QuadTreeResolutionX, QuadTreeResolutionY);
		uint NumPreviousNodes = 0;
		while (LODLevelIndex < NumLODs && DispatchThreadId.x >= (Resolution.x * Resolution.y + NumPreviousNodes))
		{
			NumPreviousNodes += Resolution.x * Resolution.y;
			Resolution = uint2(max(1u, Resolution.x / 2), max(1u, Resolution.y / 2));
			++LODLevelIndex;
		}

		const uint LocalNodeIndex = DispatchThreadId.x - NumPreviousNodes;
		NodeCoord = uint3(LocalNodeIndex % Resolution.x, LocalNodeIndex / Resolution.x, LODLevelIndex);
	}

	const uint LODLevel = NodeCoord.z;
	const float4 NodeAABB2D = GetNodeAABB2D(NodeCoord, QuadTreePosition, LeafSize);
	
	// Bounds culling
	{
		const float4 CullingBoundsAABBTWS = CullingBoundsAABB + QuadTreePosition.xyxy;
		if (any(NodeAABB2D.zw < CullingBoundsAABBTWS.xy) || any(NodeAABB2D.xy > CullingBoundsAABBTWS.zw))
		{
			return;
		}
	}

	// Frustum and HZB occlusion culling
	float2 WaterZBounds = 0.0f;
	bool bCrossesNearPlane = false;
	{
		WaterZBounds = WaterZBoundsTexture.Load(NodeCoord).xy * CaptureDepthRange + QuadTreePosition.z;
		const float3 NodeAABBMin = float3(NodeAABB2D.xy, WaterZBounds.x);
		const float3 NodeAABBMax = float3(NodeAABB2D.zw, WaterZBounds.y);
		const float3 NodeCenter = (NodeAABBMin + NodeAABBMax) * 0.5f;
		const float3 NodeExtent = float3((NodeAABBMax.xy - NodeAABBMin.xy) * 0.5f, (NodeAABBMax.z - NodeAABBMin.z) * 0.5f + 1e-5f);
	
		const bool bIsOrtho = IsOrthoProjection(ResolvedView.ViewToClip);
		FFrustumCullData Cull = BoxCullFrustum(NodeCenter, NodeExtent, ResolvedView.TranslatedWorldToClip, ResolvedView.ViewToClip, bIsOrtho, true /* near clip */, false /* skip culling */);

		if (bHZBOcclusionCullingEnabled && !Cull.bCrossesNearPlane)
		{
			const FScreenRect Rect = GetScreenRect(int4(0, 0, HZBViewSize), Cull, 4);
			Cull.bIsVisible = IsVisibleHZB(Rect, true);
		}

		if (!Cull.bIsVisible)
		{
			return;
		}

		bCrossesNearPlane = Cull.bCrossesNearPlane;
	}

	// Compute the 2D distance between the observer and the tile. Will be zero if the observer is within the tile
	const float ClosestDistanceToTile = sqrt(ComputeSquaredDistanceToPoint(NodeAABB2D.xy, NodeAABB2D.zw, ObserverPosition.xy));

	// Compute the lowest LOD we want to render, based on the observer height above the water surface
	const float3 ObserverPos = ObserverPosition - QuadTreePosition;
	const int2 TextureLoadCoord = (int2)clamp(ObserverPos.xy / LeafSize, 0.0f, float2(QuadTreeResolutionX, QuadTreeResolutionY) - 0.5f);
	const float WaterHeightForLOD = WaterZBoundsTexture.Load(int3(TextureLoadCoord, 0)).z * CaptureDepthRange;
	const FWaterLODParams WaterLODParams = GetWaterLODParams(ObserverPos.z, WaterHeightForLOD, LODScale, NumLODs - 1);

	uint DensityLevel = 0;
	bool bShouldRender = false;

	// We might need to draw this tile even if it is outside the LOD range for tiles in this level of the quadtree.
	// This is the case if tiles at a higher LOD can't be rendered, so we need to emulate a higher LOD tile with
	// multiple lower LOD tiles, but using a lower density/tessellation.
	const bool bOutsideLODRange = ClosestDistanceToTile > GetLODDistance(LODLevel, LODScale) || LODLevel < WaterLODParams.LowestLOD;
	if ((LODLevel < (NumLODs - 1)) && bOutsideLODRange)
	{
		// Does the parent tile intersect the LOD range of tiles at this LOD?
		uint3 ParentNodeCoord = uint3(NodeCoord.xy / 2, NodeCoord.z + 1);
		float4 ParentNodeAABB2D = GetNodeAABB2D(ParentNodeCoord, QuadTreePosition, LeafSize);
		const float ClosestDistanceToParentTile = sqrt(ComputeSquaredDistanceToPoint(ParentNodeAABB2D.xy, ParentNodeAABB2D.zw, ObserverPosition.xy));
		const bool bParentTileIntersectsThisLOD = ClosestDistanceToParentTile <= GetLODDistance(LODLevel, LODScale) && LODLevel >= WaterLODParams.LowestLOD;

		// Is the parent tile renderable?
		const FWaterQuadTreeNode ParentNode = WaterQuadTreeUnpackNodeRGBA8(QuadTreeTexture.Load(ParentNodeCoord));
		const bool bParentTileCanRender = CanRender(ParentNode, DensityLevel, ForceCollapseDensityLevel);

		// If the parent tile intersects this LOD, we don't render the parent tile and instead render its children.
		// If the parent tile can't render at all, then we need to emulate it by rendering its children at the highest renderable LOD.
		if (bParentTileIntersectsThisLOD || !bParentTileCanRender)
		{
			// In both cases we need to compute the density we need to use to make it appear like we rendered the higher LOD tile.
			for (uint ParentLOD = LODLevel + 1; ParentLOD < NumLODs; ++ParentLOD)
			{
				DensityLevel = ParentLOD - LODLevel;

				ParentNodeCoord = uint3(NodeCoord.xy >> (ParentLOD - LODLevel), ParentLOD);
				ParentNodeAABB2D = GetNodeAABB2D(ParentNodeCoord, QuadTreePosition, LeafSize);

				const float Dist = sqrt(ComputeSquaredDistanceToPoint(ParentNodeAABB2D.xy, ParentNodeAABB2D.zw, ObserverPosition.xy));
				if (Dist <= GetLODDistance(ParentLOD, LODScale) && ParentLOD >= WaterLODParams.LowestLOD)
				{
					break;
				}
			}
			bShouldRender = true;
		}
	}
	// The tile is fully within its LOD range and does not intersect the LOD range of its children (Distance <= GetLODDistance(LODLevel) && Distance > GetLODDistance(LODLevel - 1)), 
	// so we can render it as is.
	else if (LODLevel == WaterLODParams.LowestLOD || ClosestDistanceToTile > GetLODDistance(LODLevel - 1, LODScale))
	{
		bShouldRender = true;
	}

	if (bShouldRender)
	{
		const FWaterQuadTreeNode Node = WaterQuadTreeUnpackNodeRGBA8(QuadTreeTexture.Load(NodeCoord));
		if (CanRender(Node, DensityLevel, ForceCollapseDensityLevel))
		{
			uint WritePos = 0;
			PackedNodes.InterlockedAdd(0, 1, WritePos);
			PackedNodes.Store((WritePos + 1) << 2u, PackNodeCoord(uint4(NodeCoord, DensityLevel))); // Num is stored at index 0, so we offset all writes by 1

#if PRECISE_OCCLUSION_QUERIES
			// Increase the InstanceCount by 1.
			uint Dummy = 0;
			InterlockedAdd(OcclusionQueryArgs[ViewIndex * 5 + 1], 1, Dummy);

			// Write out bounding box and store a flag indicating if the box crosses the near plane.
			const float3 NodeAABBMin = float3(NodeAABB2D.xy, WaterZBounds.x);
			const float3 NodeAABBMax = float3(NodeAABB2D.zw, WaterZBounds.y);
			const float3 NodeCenter = (NodeAABBMin + NodeAABBMax) * 0.5f;
			const float3 NodeExtent = float3((NodeAABBMax.xy - NodeAABBMin.xy) * 0.5f, (NodeAABBMax.z - NodeAABBMin.z) * 0.5f + 1e-5f);
			OcclusionQueryBoxes[WritePos * 2 + 0] = float4(NodeCenter, bCrossesNearPlane ? 1.0f : 0.0f);
			OcclusionQueryBoxes[WritePos * 2 + 1] = float4(NodeExtent, 0.0f);

			// Set bounding boxes crossing the near plane as always visible and initialize to 0 otherwise.
			OcclusionVisibility[WritePos] = bCrossesNearPlane ? 1 : 0;
#endif // PRECISE_OCCLUSION_QUERIES
		}
	}
}

#endif // QUAD_TREE_TRAVERSE

#ifdef OCCLUSION_QUERY_RASTER_VS

Buffer<float4> OcclusionQueryBoxes;

void MainVS(
	in float3 InPosition : ATTRIBUTE0,
	in uint InInstanceId : SV_InstanceID,
	out nointerpolation uint OutQueryIndex : QUERY_INDEX,
	out float4 OutPosition : SV_Position
)
{
	ResolvedView = ResolveView();

	OutQueryIndex = InInstanceId;

	const float4 CenterAndCrossesNearPlane = OcclusionQueryBoxes[InInstanceId * 2 + 0];
	const bool bCrossesNearPlane = CenterAndCrossesNearPlane.w != 0.0f;

	if (!bCrossesNearPlane)
	{
		const float3 Center = CenterAndCrossesNearPlane.xyz;
		const float3 Extent = OcclusionQueryBoxes[InInstanceId * 2 + 1].xyz;
		const float3 TranslatedWorldPosition = (InPosition * Extent) + Center;
		OutPosition = mul(float4(TranslatedWorldPosition, 1.0f), ResolvedView.TranslatedWorldToClip);
	}
	else
	{
		// Kill instance by setting all vertices to NaN. If the box crosses the near plane, we already set it as visible in the result buffer.
		OutPosition = asfloat(0xFFFFFFFF).xxxx;
	}
}

#endif // OCCLUSION_QUERY_RASTER_VS

#ifdef OCCLUSION_QUERY_RASTER_PS

RWBuffer<uint> Visibility;

EARLYDEPTHSTENCIL
void MainPS(
	in uint InQueryIndex : QUERY_INDEX, 
	out float4 OutColor : SV_Target0
)
{
	Visibility[InQueryIndex] = 1;
	OutColor = float4(1.0f, 0.0f, 0.0f, 1.0f);
}

#endif // OCCLUSION_QUERY_RASTER_PS

#ifdef COMPUTE_BUCKET_COUNTS

#ifndef PRECISE_OCCLUSION_QUERIES
#define PRECISE_OCCLUSION_QUERIES 0
#endif

RWByteAddressBuffer BucketCounts;
Texture2D<float4> QuadTreeTexture;
StructuredBuffer<FWaterBodyRenderData> WaterBodyRenderData;
ByteAddressBuffer PackedNodes;
#if PRECISE_OCCLUSION_QUERIES
Buffer<uint> OcclusionResults;
#endif // PRECISE_OCCLUSION_QUERIES
uint NumDispatchedThreads;
uint NumDensities;
uint NumQuadsLOD0;
uint NumQuadsPerDraw;

[numthreads(64, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint NumDraws = PackedNodes.Load(0);

	// We are limited to 65535 threads, but might end up with more draws than that
	for (uint DrawIndex = DispatchThreadId.x; DrawIndex < NumDraws; DrawIndex += NumDispatchedThreads)
	{
#if PRECISE_OCCLUSION_QUERIES
		const bool bIsVisible = OcclusionResults[DrawIndex] != 0;
		if (!bIsVisible)
		{
			continue;
		}
#endif

		// Load packed node coord and density index
		const uint4 NodeCoordAndDensity = UnpackNodeCoord(PackedNodes.Load((DrawIndex + 1) << 2));
		const uint DensityIndexClamped = min(NodeCoordAndDensity.w, NumDensities - 1);

		// Sample the quadtree to access the water body render data
		const float4 QuadTreeSample = QuadTreeTexture.Load(NodeCoordAndDensity.xyz);
		const FWaterQuadTreeNode Node = WaterQuadTreeUnpackNodeRGBA8(QuadTreeSample);
		const FWaterBodyRenderData WBRenderData = WaterBodyRenderData[Node.WaterBodyRenderDataIndex];

		// Determine the material
		uint MaterialIndex = WBRenderData.MaterialIndex;
		// Rivers can have transitions to other water bodies
		if (WBRenderData.WaterBodyType == WATER_BODY_TYPE_RIVER && Node.TransitionWaterBodyRenderDataIndex > 0)
		{
			const FWaterBodyRenderData TransitionWBRenderData = WaterBodyRenderData[Node.TransitionWaterBodyRenderDataIndex];
			if (TransitionWBRenderData.WaterBodyType == WATER_BODY_TYPE_LAKE)
			{
				MaterialIndex = WBRenderData.RiverToLakeMaterialIndex;
			}
			else if (TransitionWBRenderData.WaterBodyType == WATER_BODY_TYPE_OCEAN)
			{
				MaterialIndex = WBRenderData.RiverToOceanMaterialIndex;
			}
		}

		// Increment bucket counter
		const uint BucketIndex = MaterialIndex;
		const uint NumTilesPerEdge = max(NumQuadsPerDraw, NumQuadsLOD0 >> DensityIndexClamped) / NumQuadsPerDraw;
		const uint NumDraws = NumTilesPerEdge * NumTilesPerEdge;
		uint Dummy;
		BucketCounts.InterlockedAdd(BucketIndex << 2, NumDraws, Dummy);
	}
}

#endif // COMPUTE_BUCKET_COUNTS

#ifdef COMPUTE_BUCKET_PREFIX_SUM

#ifndef PARALLEL_PREFIX_SUM
#define PARALLEL_PREFIX_SUM 0
#endif

RWBuffer<uint> BucketPrefixSums;
ByteAddressBuffer BucketCounts;
uint NumBuckets;
uint OutputOffset;
uint bWriteTotalSumAtBufferEnd;

#if PARALLEL_PREFIX_SUM

#include "/Engine/Private/WaveOpUtil.ush"

#define GROUP_SIZE 128
#define ARRAY_SIZE (GROUP_SIZE * 2)
groupshared uint SharedData[ARRAY_SIZE];
groupshared uint BlockPrefixSum;
groupshared uint TotalGlobalSum;

uint LoadFromBuffer(uint InIndex)
{
	if (InIndex < NumBuckets)
	{
		return BucketCounts.Load(InIndex << 2);
	}
	else
	{
		return 0;
	}
}

void WriteToBuffer(uint InIndex, uint InWriteOffset, uint InGlobalPrefixSum)
{
	if (InIndex < NumBuckets)
	{
		BucketPrefixSums[InIndex + InWriteOffset] = SharedData[InIndex] + InGlobalPrefixSum;
	}
}

#endif

#if PARALLEL_PREFIX_SUM
[numthreads(GROUP_SIZE, 1, 1)]
#else
[numthreads(1, 1, 1)]
#endif
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
#if PARALLEL_PREFIX_SUM

	const uint ThreadID = DispatchThreadId.x;
	const uint NumBlocks = (NumBuckets + (ARRAY_SIZE - 1)) / ARRAY_SIZE;

	if (ThreadID == 0)
	{
		// Apply global offset if this is not the first view. All views share the same prefix sum and instance data buffers.
		if (OutputOffset > 0)
		{
			// Previous view must have written the total sum after the end of its range. This corresponds to the first element in this range.
			BlockPrefixSum = BucketPrefixSums[OutputOffset];
		}
		else
		{
			BlockPrefixSum = 0;
		}
		TotalGlobalSum = BlockPrefixSum;
	}

	// Process array in blocks of fixed size, keeping track of the sum of all elements in previous blocks
	for (uint BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
	{
		GroupMemoryBarrierWithGroupSync();

		const uint BlockOffset = BlockIndex * ARRAY_SIZE;
		uint Offset = 1;

		// Load into LDS
		const uint Value0 = LoadFromBuffer(2 * ThreadID + 0 + BlockOffset);
		const uint Value1 = LoadFromBuffer(2 * ThreadID + 1 + BlockOffset);
		const uint LocalSum = Value0 + Value1;
		SharedData[2 * ThreadID + 0] = Value0;
		SharedData[2 * ThreadID + 1] = Value1;

		WaveInterlockedAdd(TotalGlobalSum, LocalSum);

		// Up-sweep
		for (uint d = ARRAY_SIZE >> 1; d > 0; d >>= 1)
		{
			GroupMemoryBarrierWithGroupSync();

			if (ThreadID < d)
			{
				const uint IndexA = Offset * (2 * ThreadID + 1) - 1;
				const uint IndexB = Offset * (2 * ThreadID + 2) - 1;

				SharedData[IndexB] += SharedData[IndexA];
			}
			Offset <<= 1;
		}

		// Clear the last element
		if (ThreadID == 0)
		{
			SharedData[ARRAY_SIZE - 1] = 0;
		}

		// Down-sweep
		for (uint d = 1; d < ARRAY_SIZE; d <<= 1)
		{
			Offset >>= 1;
			GroupMemoryBarrierWithGroupSync();

			if (ThreadID < d)
			{
				const uint IndexA = Offset * (2 * ThreadID + 1) - 1;
				const uint IndexB = Offset * (2 * ThreadID + 2) - 1;

				const uint Temp = SharedData[IndexA];
				SharedData[IndexA] = SharedData[IndexB];
				SharedData[IndexB] += Temp;
			}
		}

		const uint GlobalPrefixSum = BlockPrefixSum;

		GroupMemoryBarrierWithGroupSync();

		// Write results to output buffer
		WriteToBuffer(2 * ThreadID + 0, BlockOffset + OutputOffset, GlobalPrefixSum);
		WriteToBuffer(2 * ThreadID + 1, BlockOffset + OutputOffset, GlobalPrefixSum);

		if (NumBlocks > 1)
		{
			WaveInterlockedAdd(BlockPrefixSum, LocalSum);
		}
	}

	// Write total of all values (including those of prior views) at element 0 of the next view.
	// This way we can propagate the prefix sum offsets across views.
	if (bWriteTotalSumAtBufferEnd != 0)
	{
		if (ThreadID == 0)
		{
			BucketPrefixSums[OutputOffset + NumBuckets] = TotalGlobalSum;
		}
	}

#else

	uint PrefixSum = 0;

	// Apply global offset if this is not the first view. All views share the same prefix sum and instance data buffers.
	if (OutputOffset > 0)
	{
		// Previous view must have written the total sum after the end of its range. This corresponds to the first element in this range.
		PrefixSum = BucketPrefixSums[OutputOffset];
	}

	for (uint BucketIndex = 0; BucketIndex < NumBuckets; ++BucketIndex)
	{
		BucketPrefixSums[OutputOffset + BucketIndex] = PrefixSum;
		const uint Count = BucketCounts.Load(BucketIndex << 2);
		PrefixSum += Count;
	}

	// Write total of all values (including those of prior views) at element 0 of the next view.
	// This way we can propagate the prefix sum offsets across views.
	if (bWriteTotalSumAtBufferEnd != 0)
	{
		BucketPrefixSums[OutputOffset + NumBuckets] = PrefixSum;
	}

#endif // PARALLEL_PREFIX_SUM
}

#endif // COMPUTE_BUCKET_PREFIX_SUM

#ifdef GENERATE_INSTANCE_DATA

#ifndef PRECISE_OCCLUSION_QUERIES
#define PRECISE_OCCLUSION_QUERIES 0
#endif

RWBuffer<uint> IndirectArgs;
RWBuffer<uint> InstanceData0;
RWBuffer<uint> InstanceData1;
RWBuffer<uint> InstanceData2;
RWBuffer<uint> InstanceData3;
Texture2D<float4> QuadTreeTexture;
Texture2D<float4> WaterZBoundsTexture;
StructuredBuffer<FWaterBodyRenderData> WaterBodyRenderData;
ByteAddressBuffer PackedNodes;
Buffer<uint> InstanceDataOffsets;
#if PRECISE_OCCLUSION_QUERIES
Buffer<uint> OcclusionResults;
#endif // PRECISE_OCCLUSION_QUERIES

float3 QuadTreePosition;
float3 ObserverPosition;
uint QuadTreeResolutionX;
uint QuadTreeResolutionY;
uint NumDensities;
uint NumMaterials;
uint NumDispatchedThreads;
uint BucketIndexOffset;
uint NumLODs;
uint NumQuadsLOD0;
uint NumQuadsPerDraw;
float LeafSize;
float LODScale;
float CaptureDepthRange;
uint StereoPassInstanceFactor;
uint bWithWaterSelectionSupport;
uint bLODMorphingEnabled;
uint bInstancedStereoRendering;

[numthreads(64, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint NumDraws = PackedNodes.Load(0);
	// We are limited to 65535 threads, but might end up with more draws than that
	for (uint DrawIndex = DispatchThreadId.x; DrawIndex < NumDraws; DrawIndex += NumDispatchedThreads)
	{
#if PRECISE_OCCLUSION_QUERIES
		const bool bIsVisible = OcclusionResults[DrawIndex] != 0;
		if (!bIsVisible)
		{
			continue;
		}
#endif

		// Load packed node coord and density index
		const uint4 NodeCoordAndDensity = UnpackNodeCoord(PackedNodes.Load((DrawIndex + 1) << 2));
		const uint2 NodeCoord = NodeCoordAndDensity.xy;
		const uint LODLevel = NodeCoordAndDensity.z;
		const uint DensityIndex = NodeCoordAndDensity.w;
		const uint DensityIndexClamped = min(DensityIndex, NumDensities - 1);

		// Sample the quadtree to access the water body render data
		const float4 QuadTreeSample = QuadTreeTexture.Load(int3(NodeCoord, LODLevel));
		const FWaterQuadTreeNode Node = WaterQuadTreeUnpackNodeRGBA8(QuadTreeSample);
		const FWaterBodyRenderData WBRenderData = WaterBodyRenderData[Node.WaterBodyRenderDataIndex];

		// Determine the material
		uint MaterialIndex = WBRenderData.MaterialIndex;
		uint WaterBodyIndex = WBRenderData.WaterBodyIndex;
		// Rivers can have transitions to other water bodies
		if (WBRenderData.WaterBodyType == WATER_BODY_TYPE_RIVER && Node.TransitionWaterBodyRenderDataIndex > 0)
		{
			const FWaterBodyRenderData TransitionWBRenderData = WaterBodyRenderData[Node.TransitionWaterBodyRenderDataIndex];
			if (TransitionWBRenderData.WaterBodyType == WATER_BODY_TYPE_LAKE)
			{
				MaterialIndex = WBRenderData.RiverToLakeMaterialIndex;
				WaterBodyIndex = TransitionWBRenderData.WaterBodyIndex;
			}
			else if (TransitionWBRenderData.WaterBodyType == WATER_BODY_TYPE_OCEAN)
			{
				MaterialIndex = WBRenderData.RiverToOceanMaterialIndex;
				WaterBodyIndex = TransitionWBRenderData.WaterBodyIndex;
			}
		}

		const uint BucketIndex = MaterialIndex + BucketIndexOffset;

		const uint BucketInstanceDataOffset = InstanceDataOffsets.Load(BucketIndex);
		const uint NumTilesPerEdge = max(NumQuadsPerDraw, NumQuadsLOD0 >> DensityIndexClamped) / NumQuadsPerDraw;
		const uint NumInstances = NumTilesPerEdge * NumTilesPerEdge;
		const uint NumInstancesStereo = NumInstances * StereoPassInstanceFactor;
		
		// Increment InstanceCount
		uint LocalInstanceDataOffsetStereo;
		InterlockedAdd(IndirectArgs[BucketIndex * 5 + 1], NumInstancesStereo, LocalInstanceDataOffsetStereo);
		const uint LocalInstanceDataOffset = LocalInstanceDataOffsetStereo / StereoPassInstanceFactor;
		const uint InstanceDataBaseIndex = BucketInstanceDataOffset + LocalInstanceDataOffset;

		// On the first write to this bucket, set StartInstanceLocation to offset where the per-instance vertex attributes are read from.
		// With ISR, we use the InstanceId to manually fetch instance data buffers in the vertex factory.
		// StartInstanceLocation affects InstanceId differently on different platforms, so when using InstanceId, StartInstanceLocation must be 0.
		if (LocalInstanceDataOffset == 0 && bInstancedStereoRendering == 0)
		{
			IndirectArgs[BucketIndex * 5 + 4] = BucketInstanceDataOffset;
		}

		// Write packed instance data
		{
			// Data0:
			// uint NodeCoord.x : 11;
			// uint NodeCoord.y : 11;
			// uint LODLevel : 5;
			// uint DensityIndex : 5;
			//
			// Data1:
			// half WaterSurfaceBaseHeight;
			// uint TileX : 8;
			// uint TileY : 8;
			//
			// Data2:
			// uint HeightLODFactorUnorm : 8;
			// uint WaterBodyIndex : 24; // Could potentially use fewer bits for this to cram additional data in here in the future

			const float3 ObserverPos = ObserverPosition - QuadTreePosition;
			const int2 TextureLoadCoord = (int2)clamp(ObserverPos.xy / LeafSize, 0.0f, float2(QuadTreeResolutionX, QuadTreeResolutionY) - 0.5f);
			const float WaterHeightForLOD = WaterZBoundsTexture.Load(int3(TextureLoadCoord, 0)).z * CaptureDepthRange;
			const FWaterLODParams WaterLODParams = GetWaterLODParams(ObserverPos.z, WaterHeightForLOD, LODScale, NumLODs - 1);
			const uint LogicalLODLevel = LODLevel + DensityIndex;

			// Lowest LOD isn't always 0, this increases with the height distance 
			const bool bIsLowestLOD = (LogicalLODLevel == WaterLODParams.LowestLOD);
			const float HeightLODFactor = bIsLowestLOD ? WaterLODParams.HeightLODFactor : 0.0f;
			const uint HeightLODFactorUnorm = uint(saturate(HeightLODFactor) * 255.0f);
			const float WaterSurfaceBaseHeight = WaterZBoundsTexture.Load(int3(NodeCoord, LODLevel)).z;

			const uint Data0 = (NodeCoord.x & 0x7FFu) | ((NodeCoord.y & 0x7FFu) << 11u) | ((LODLevel & 0x1Fu) << 22u) | ((DensityIndex & 0x1Fu) << 27u);
			const uint WaterHeightF16 = f32tof16(WaterSurfaceBaseHeight);
			const uint Data2 = (HeightLODFactorUnorm & 0xFFu) | (WaterBodyIndex << 8u);

			// Render a single quadtree node by drawing one or multiple tile instances
			uint TileX = 0;
			uint TileY = 0;
			for (uint InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
			{
				const uint Data1 = WaterHeightF16 | ((TileX & 0xFFu) << 16u) | ((TileY & 0xFFu) << 24u);
				InstanceData0[InstanceDataBaseIndex + InstanceIndex] = Data0;
				InstanceData1[InstanceDataBaseIndex + InstanceIndex] = Data1;
				InstanceData2[InstanceDataBaseIndex + InstanceIndex] = Data2;
				
				TileY = (TileX + 1) >= NumTilesPerEdge ? (TileY + 1) : TileY;
				TileX = (TileX + 1) >= NumTilesPerEdge ? 0 : (TileX + 1);
			}
		}

		// Instance Hit Proxy ID
		if (bWithWaterSelectionSupport)
		{
			for (uint InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
			{
				InstanceData3[InstanceDataBaseIndex + InstanceIndex] = WBRenderData.HitProxyColorAndIsSelected;
			}
		}
	}
}

#endif // GENERATE_INSTANCE_DATA

#if DEBUG_SHOW_TILES

#include "/Engine/Private/ShaderPrint.ush"

#ifndef PRECISE_OCCLUSION_QUERIES
#define PRECISE_OCCLUSION_QUERIES 0
#endif

Texture2D<float4> QuadTreeTexture;
Texture2D<float4> WaterZBoundsTexture;
StructuredBuffer<FWaterBodyRenderData> WaterBodyRenderData;
ByteAddressBuffer PackedNodes;
#if PRECISE_OCCLUSION_QUERIES
Buffer<uint> OcclusionResults;
#endif // PRECISE_OCCLUSION_QUERIES

float3 QuadTreePosition;
uint NumDispatchedThreads;
float LeafSize;
float CaptureDepthRange;

[numthreads(64, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint NumDraws = PackedNodes.Load(0);
	// We are limited to 65535 threads, but might end up with more draws than that
	for (uint DrawIndex = DispatchThreadId.x; DrawIndex < NumDraws; DrawIndex += NumDispatchedThreads)
	{
#if PRECISE_OCCLUSION_QUERIES
		const bool bIsVisible = OcclusionResults[DrawIndex] != 0;
		if (!bIsVisible)
		{
			continue;
		}
#endif

		// Load packed node coord and density index
		const uint4 NodeCoordAndDensity = UnpackNodeCoord(PackedNodes.Load((DrawIndex + 1) << 2));
		const uint3 NodeCoord = NodeCoordAndDensity.xyz;

		// Sample the quadtree to access the water body render data
		const float4 QuadTreeSample = QuadTreeTexture.Load(int3(NodeCoord));
		const FWaterQuadTreeNode Node = WaterQuadTreeUnpackNodeRGBA8(QuadTreeSample);
		const FWaterBodyRenderData WBRenderData = WaterBodyRenderData[Node.WaterBodyRenderDataIndex];

		// TODO: Support the two missing modes for visualizing the LOD level and the density
		float4 DebugColor;
		switch (WBRenderData.WaterBodyType)
		{
		case WATER_BODY_TYPE_RIVER: DebugColor = ColorRed; break;
		case WATER_BODY_TYPE_LAKE: DebugColor = ColorGreen; break;
		case WATER_BODY_TYPE_OCEAN: DebugColor = ColorBlue; break;
		default: DebugColor = ColorWhite;
		}

		if (WBRenderData.WaterBodyType == WATER_BODY_TYPE_RIVER && Node.TransitionWaterBodyRenderDataIndex > 0)
		{
			const FWaterBodyRenderData TransitionWBRenderData = WaterBodyRenderData[Node.TransitionWaterBodyRenderDataIndex];
			if (TransitionWBRenderData.WaterBodyType == WATER_BODY_TYPE_LAKE)
			{
				DebugColor = ColorYellow;
			}
			else if (TransitionWBRenderData.WaterBodyType == WATER_BODY_TYPE_OCEAN)
			{
				DebugColor = ColorPurple;
			}
		}

		const float4 NodeAABB2D = GetNodeAABB2D(NodeCoord, QuadTreePosition, LeafSize);
		const float2 WaterZBounds = WaterZBoundsTexture.Load(NodeCoord).xy * CaptureDepthRange + QuadTreePosition.z;
		const float3 AABBMin = float3(NodeAABB2D.xy + 20.0f, WaterZBounds.x);
		const float3 AABBMax = float3(NodeAABB2D.zw - 20.0f, WaterZBounds.y);

		AddOBBTWS(InitShaderPrintContext(), AABBMin, AABBMax, DebugColor, float4x4(
			1, 0, 0, 0,
			0, 1, 0, 0,
			0, 0, 1, 0,
			0, 0, 0, 1));
	}
}

#endif // DEBUG_SHOW_TILES