// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/PackUnpack.ush"
#include "WaterQuadTreeCommon.ush"

#ifndef NUM_MSAA_SAMPLES
#define NUM_MSAA_SAMPLES 1
#endif

#if NUM_MSAA_SAMPLES > 1
Texture2DMS<float4, NUM_MSAA_SAMPLES> WaterBodyRasterTextureMS;
Texture2DMS<float4, NUM_MSAA_SAMPLES> ZBoundsRasterTextureMS;
#else
Texture2D<float4> WaterBodyRasterTexture;
Texture2D<float4> ZBoundsRasterTexture;
#endif
StructuredBuffer<FWaterBodyRenderData> WaterBodyRenderData;
int SuperSamplingFactor;
float RcpCaptureDepthRange;
float ZBoundsPadding; // We base our tile Z bounds on rasterization, however with sloped rivers we will likely miss part of the Z extent. Add some padding to account for that.

uint UnpackWaterBodyRenderDataIndex(float Packed)
{
	const uint PackedUI = saturate(Packed) * 1024.0f;
	return PackedUI & 0x7Fu;
}

void Main(
	in float4 Position : SV_Position, 
	out float4 OutPackedNode : SV_Target0, 
	out float4 OutWaterZBounds : SV_Target1)
{
	// Downsample super- and multisampled input texture. The data is set up to correctly resolve with MAX blending.
	float NonRiverWaterBodyPacked = 0.0f;
	float RiverWaterBodyPacked = 0.0f;
	float MinZ = 0.0f;
	float MaxZ = 0.0f;
	for (int y = 0; y < SuperSamplingFactor; ++y)
	{
		for (int x = 0; x < SuperSamplingFactor; ++x)
		{
#if NUM_MSAA_SAMPLES > 1
			for (int SampleIdx = 0; SampleIdx < NUM_MSAA_SAMPLES; ++SampleIdx)
#endif
			{
				const int2 Coord = int2(Position.xy) * SuperSamplingFactor + int2(x, y);

#if NUM_MSAA_SAMPLES > 1
				const float2 WaterBodySample = WaterBodyRasterTextureMS.Load(Coord, SampleIdx).xy;
				const float2 ZBoundsSample = ZBoundsRasterTextureMS.Load(Coord, SampleIdx).xy;
#else
				const float2 WaterBodySample = WaterBodyRasterTexture.Load(int3(Coord, 0)).xy;
				const float2 ZBoundsSample = ZBoundsRasterTexture.Load(int3(Coord, 0)).xy;
#endif
				NonRiverWaterBodyPacked = max(NonRiverWaterBodyPacked, WaterBodySample.x);
				RiverWaterBodyPacked = max(RiverWaterBodyPacked, WaterBodySample.y);
				
				MinZ = max(MinZ, ZBoundsSample.x); // MinZ is actually 1 - MinZ
				MaxZ = max(MaxZ, ZBoundsSample.y);
			}
		}
	}

	const uint NonRiverWaterBodyRenderDataIndex = UnpackWaterBodyRenderDataIndex(NonRiverWaterBodyPacked);
	const uint RiverWaterBodyRenderDataIndex = UnpackWaterBodyRenderDataIndex(RiverWaterBodyPacked);

	const bool bIsNonRiverWaterBodyValid = NonRiverWaterBodyRenderDataIndex > 0;
	const bool bIsRiverWaterBodyValid = RiverWaterBodyRenderDataIndex > 0;
	const bool bIsAnyValid = bIsRiverWaterBodyValid || bIsNonRiverWaterBodyValid;

	FWaterQuadTreeNode ResultNode;
	ResultNode.WaterBodyRenderDataIndex = 0;
	ResultNode.TransitionWaterBodyRenderDataIndex = 0;
	ResultNode.bHasCompleteSubtree = bIsAnyValid;
	ResultNode.bIsSubtreeSameWaterBody = bIsAnyValid;

	// Rivers always have priority and can have transitions to non-river water bodies
	if (bIsRiverWaterBodyValid)
	{
		ResultNode.WaterBodyRenderDataIndex = RiverWaterBodyRenderDataIndex;
		ResultNode.TransitionWaterBodyRenderDataIndex = bIsNonRiverWaterBodyValid ? NonRiverWaterBodyRenderDataIndex : 0;
	}
	else if (bIsNonRiverWaterBodyValid)
	{
		ResultNode.WaterBodyRenderDataIndex = NonRiverWaterBodyRenderDataIndex;
		ResultNode.TransitionWaterBodyRenderDataIndex = 0;
	}

	OutPackedNode = WaterQuadTreePackNodeRGBA8(ResultNode);

	const float2 MinMaxZ = bIsAnyValid ? saturate(float2((1.0f - MinZ) - ZBoundsPadding, MaxZ + ZBoundsPadding)) : float2(0.0f, 0.0f);
	float SurfaceBaseHeight = 0.0f;
	// In case of rivers transitioning to other water bodies, we want to use the SurfaceBaseHeight of that other water body
	if (bIsNonRiverWaterBodyValid)
	{
		const FWaterBodyRenderData WBRenderData = WaterBodyRenderData[NonRiverWaterBodyRenderDataIndex];
		SurfaceBaseHeight = WBRenderData.SurfaceBaseHeight * RcpCaptureDepthRange;
	}
	else if (bIsRiverWaterBodyValid)
	{
		SurfaceBaseHeight = MinMaxZ.y;
	}

	OutWaterZBounds = float4(MinMaxZ, SurfaceBaseHeight, 1.0f);
}