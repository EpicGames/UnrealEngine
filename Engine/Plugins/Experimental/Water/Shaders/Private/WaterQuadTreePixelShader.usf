// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

#ifndef CLIP_CONSERVATIVE_TRIANGLE
#define CLIP_CONSERVATIVE_TRIANGLE 0
#endif

uint WaterBodyRenderDataIndex;
uint Priority;
float WaterBodyMinZ;
float WaterBodyMaxZ;
float MaxWaveHeight;
uint bIsRiver;

void Main(
	in float4 InPosition : SV_Position,
#if CLIP_CONSERVATIVE_TRIANGLE
	in float4 InClipPosition : CLIP_POSITION,
	in float4 InTriangleAABB : TRIANGLE_AABB,
#endif
	out float4 OutWaterBody : SV_Target0,
	out float4 OutZBounds : SV_Target1)
{
#if CLIP_CONSERVATIVE_TRIANGLE
	const float2 Pos = InClipPosition.xy / InClipPosition.w;
	if (any(Pos < InTriangleAABB.xy) || any(Pos > InTriangleAABB.zw))
	{
		discard;
	}
#endif

	uint PackedPriorityAndWaterBodyRenderDataIndex = 0;
	PackedPriorityAndWaterBodyRenderDataIndex |= WaterBodyRenderDataIndex & 0x7Fu; // 7bits
	PackedPriorityAndWaterBodyRenderDataIndex |= min(Priority, 7u) << 7u; // 3bits

	// Pack into 10bit unorm
	const float PackedF = (PackedPriorityAndWaterBodyRenderDataIndex + 0.5f) / 1024.0f;
	
	// RT0 R: Packed non-river water body priority and render data index
	// RT0 G: Packed river water body priority and render data index
	// RT1 R: Inverse normalized minimum bounds Z value
	// RT1 G: Normalized maximum bounds Z value
	if (bIsRiver != 0)
	{
		const float RiverDepth = InPosition.z;
		const float RiverMinZ = saturate(RiverDepth - MaxWaveHeight);
		const float RiverMaxZ = saturate(RiverDepth + MaxWaveHeight);
		OutWaterBody = float4(0.0f, PackedF, 0.0f, 1.0f);
		OutZBounds = float4(1.0f - RiverMinZ, RiverMaxZ, 0.0f, 1.0f);
	}
	else
	{
		OutWaterBody = float4(PackedF, 0.0f, 0.0f, 1.0f);
		OutZBounds = float4(1.0f - WaterBodyMinZ, WaterBodyMaxZ, 0.0f, 1.0f);
	}
}
