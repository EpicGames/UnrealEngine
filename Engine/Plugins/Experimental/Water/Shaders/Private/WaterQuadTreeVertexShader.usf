// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

#define RASTER_MODE_REGULAR 0
#define RASTER_MODE_JITTER 1
#define RASTER_MODE_CONSERVATIVE 2

#ifndef RASTER_MODE
#define RASTER_MODE RASTER_MODE_REGULAR
#endif

float4x4 Transform;

#if RASTER_MODE == RASTER_MODE_JITTER
float2 JitterScale;
#elif RASTER_MODE == RASTER_MODE_CONSERVATIVE
float2 HalfPixelSize;
#endif

float4 AddPoint(float4 InPos, float4 InAABB)
{
	if (InPos.w < 0.0f)
	{
		InAABB.xy = min(InAABB.xy, InPos.xy);
		InAABB.zw = max(InAABB.zw, InPos.xy);
	}
	else
	{
		InAABB.xy = min(InAABB.xy, InPos.xy / InPos.w);
		InAABB.zw = max(InAABB.zw, InPos.xy / InPos.w);
	}
	return InAABB;
}

void Main(
	in float3 InPosition : ATTRIBUTE0,
#if RASTER_MODE == RASTER_MODE_JITTER
	in float2 InJitter : ATTRIBUTE1,
#elif RASTER_MODE == RASTER_MODE_CONSERVATIVE
	in float2 InPrevNextVertex0 : ATTRIBUTE1,
	in float2 InPrevNextVertex1 : ATTRIBUTE2,
	in float2 InPrevNextVertex2 : ATTRIBUTE3,
#endif
	out float4 OutPosition : SV_Position
#if RASTER_MODE == RASTER_MODE_CONSERVATIVE
	,out float4 OutClipPosition : CLIP_POSITION
	,out float4 OutTriangleAABB : TRIANGLE_AABB
#endif
)
{
#if RASTER_MODE == RASTER_MODE_REGULAR
	
	OutPosition = mul(float4(InPosition, 1.0f), Transform);

#elif RASTER_MODE == RASTER_MODE_JITTER
	
	float4x4 JitteredTransform = Transform;
	JitteredTransform[3].xy += InJitter * JitterScale;
	OutPosition = mul(float4(InPosition, 1.0f), JitteredTransform);

#elif RASTER_MODE == RASTER_MODE_CONSERVATIVE

	// See "GPU Gems 2 Chapter 42. Conservative Rasterization".
	// We construct a slightly bigger triangle by displacing the edges by HalfPixelSize. 
	// This new triangle should then conservatively cover all pixels touched by the original triangle.
	// We pass a triangle AABB to the pixel shader to discard false positive pixels.

	const float3 PrevVertexPos = float3(InPrevNextVertex0.x, InPrevNextVertex0.y, InPrevNextVertex1.x);
	const float3 NextVertexPos = float3(InPrevNextVertex1.y, InPrevNextVertex2.x, InPrevNextVertex2.y);

	const float4 PrevPosition = mul(float4(PrevVertexPos, 1.0f), Transform);
	const float4 CurrPosition = mul(float4(InPosition, 1.0f), Transform);
	const float4 NextPosition = mul(float4(NextVertexPos, 1.0f), Transform);

	// Compute the triangle AABB
	OutTriangleAABB = float4(9999999.0f, 9999999.0f, -9999999.0f, -9999999.0f);
	OutTriangleAABB = AddPoint(PrevPosition, OutTriangleAABB);
	OutTriangleAABB = AddPoint(CurrPosition, OutTriangleAABB);
	OutTriangleAABB = AddPoint(NextPosition, OutTriangleAABB);
	OutTriangleAABB += float4(-HalfPixelSize, HalfPixelSize);

	// Compute equations of the planes through the two edges  
	float3 Planes[2];
	Planes[0] = cross(CurrPosition.xyw - PrevPosition.xyw, PrevPosition.xyw);
	Planes[1] = cross(NextPosition.xyw - CurrPosition.xyw, CurrPosition.xyw);

	// Move the planes by the appropriate semidiagonal  
	Planes[0].z -= dot(HalfPixelSize, abs(Planes[0].xy));
	Planes[1].z -= dot(HalfPixelSize, abs(Planes[1].xy));

	// Compute the intersection point of the planes.
	OutPosition = CurrPosition;
	OutPosition.xyw = cross(Planes[0], Planes[1]);
	OutPosition /= abs(OutPosition.w);
	OutPosition.z = CurrPosition.z / CurrPosition.w; // Keep the depth of the non-dilated vertex. Slightly incorrect, but seems to work.

	OutClipPosition = OutPosition;

#endif
}
