// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_WaterWaves.h"
#include "Modules/ModuleManager.h"
#include "WaterWaves.h"
#include "WaterEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_WaterWaves)

#define LOCTEXT_NAMESPACE "WaterWaves"

FText UAssetDefinition_WaterWaves::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "FAssetTypeActions_WaterWaves", "Water Waves");
}

TSoftClassPtr<UObject> UAssetDefinition_WaterWaves::GetAssetClass() const 
{
	return UWaterWavesAsset::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_WaterWaves::GetAssetCategories() const
{
	static TArray<FAssetCategoryPath> AssetCategories = { FAssetCategoryPath(LOCTEXT("Water", "Water"))};
	return AssetCategories;
}

EAssetCommandResult UAssetDefinition_WaterWaves::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	FWaterEditorModule* WaterEditorModule = &FModuleManager::LoadModuleChecked<FWaterEditorModule>("WaterEditor");

	for (UWaterWavesAsset* WavesAsset : OpenArgs.LoadObjects<UWaterWavesAsset>())
	{
		WaterEditorModule->CreateWaterWaveAssetEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, WavesAsset);
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
