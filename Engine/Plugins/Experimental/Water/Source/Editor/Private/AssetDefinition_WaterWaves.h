// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_WaterWaves.generated.h"

UCLASS()
class UAssetDefinition_WaterWaves : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:

	// Begin IAssetTypeActions Interface
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(0, 85, 200)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// End IAssetTypeActions Interface
};