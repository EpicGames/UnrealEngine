// Copyright Epic Games, Inc. All Rights Reserved.

#include "CausticsGeneratorActor.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CausticsGeneratorActor)


ACausticsGeneratorActor::ACausticsGeneratorActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DefaultSceneRoot = CreateDefaultSubobject<USceneComponent>(TEXT("DefaultSceneRoot"));
	RootComponent = DefaultSceneRoot;
	DefaultSceneRoot->CreationMethod = EComponentCreationMethod::Native;

	//WaterPreviewGridHISMC = CreateDefaultSubobject<UHierarchicalInstancedStaticMeshComponent>(TEXT("WaterPreviewGridHISMC"));
	//WaterPreviewGridHISMC->CreationMethod = EComponentCreationMethod::Native;
	//WaterPreviewGridHISMC->AttachToComponent(DefaultSceneRoot, FAttachmentTransformRules::KeepRelativeTransform);
	//WaterPreviewGridHISMC->SetStaticMesh(Cast< UStaticMesh >(FSoftObjectPath(TEXT("StaticMesh'/Water/Meshes/S_WaterPlane_512_LOD0.S_WaterPlane_512_LOD0'")).TryLoad()));

	//CausticParticlesGridHISMC = CreateDefaultSubobject<UHierarchicalInstancedStaticMeshComponent>(TEXT("CausticParticlesGridHISMC"));
	//CausticParticlesGridHISMC->CreationMethod = EComponentCreationMethod::Native;
	//CausticParticlesGridHISMC->AttachToComponent(DefaultSceneRoot, FAttachmentTransformRules::KeepRelativeTransform);
	//CausticParticlesGridHISMC->SetStaticMesh(Cast< UStaticMesh >(FSoftObjectPath(TEXT("StaticMesh'/Water/Caustics/Meshes/CausticPhotonMeshGrid.CausticPhotonMeshGrid'")).TryLoad()));

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.TickGroup = TG_DuringPhysics;
	PrimaryActorTick.bStartWithTickEnabled = true;
	PrimaryActorTick.SetTickFunctionEnable(true);
	bIsEditorOnlyActor = true;
}

void ACausticsGeneratorActor::Tick(float DeltaSeconds)
{
	this->EditorTick(DeltaSeconds);
	//UE_LOGF(LogTemp, Warning, "Actor Tick was called");
}

void ACausticsGeneratorActor::EditorTick_Implementation(float DeltaSeconds)
{

}

bool ACausticsGeneratorActor::ShouldTickIfViewportsOnly() const
{
	return EditorTickIsEnabled;
}

void ACausticsGeneratorActor::SpawnWaterPreviewGrid(UHierarchicalInstancedStaticMeshComponent* HISMC, float GridSize, int GridTiles)
{
	HISMC->ClearInstances();

	const float CellSizeF = GridSize / (float)GridTiles;
	const float HalfCellsX = (float)GridTiles / 2.0f;
	const float HalfCellsY = (float)GridTiles / 2.0f;
	const FVector ActorLocation(GetActorLocation());
	const FVector CellSizeVector(CellSizeF / 256.f);
	const FVector HalfVector(0.5f, 0.5f, 0.0f);

	for (int32 yy = 0; yy < GridTiles; ++yy)
	{
		const float PolyCenterY = (float)yy - HalfCellsY;

		for (int32 xx = 0; xx < GridTiles; ++xx)
		{
			const float PolyCenterX = (float)xx - HalfCellsX;

			FVector PolyCenterVector(PolyCenterX, PolyCenterY, 0.f);
			PolyCenterVector *= CellSizeF;
			PolyCenterVector += HalfVector * CellSizeF;
			PolyCenterVector += ActorLocation;

			HISMC->AddInstance(FTransform(FRotator::ZeroRotator, PolyCenterVector, CellSizeVector));
		}
	}
}

void ACausticsGeneratorActor::SpawnCausticParticleGrid(UHierarchicalInstancedStaticMeshComponent* HISMC, float GridSize, int GridTiles)
{
	HISMC->ClearInstances();

	const float CellSizeF = GridSize / (float)GridTiles;
	const float HalfCellsX = (float)GridTiles / 2.0f;
	const float HalfCellsY = (float)GridTiles / 2.0f;
	const FVector ActorLocation(GetActorLocation());
	const FVector CellSizeVector(CellSizeF / 12700.0f);
	const FVector HalfVector(0.5f, 0.5f, 0.0f);

	for (int32 yy = 0; yy < GridTiles; ++yy)
	{
		const float PolyCenterY = (float)yy - HalfCellsY;

		for (int32 xx = 0; xx < GridTiles; ++xx)
		{
			const float PolyCenterX = (float)xx - HalfCellsX;

			FVector PolyCenterVector(PolyCenterX, PolyCenterY, 0.f);
			PolyCenterVector *= CellSizeF;
			PolyCenterVector += HalfVector * CellSizeF;
			PolyCenterVector += ActorLocation;

			HISMC->AddInstance(FTransform(FRotator::ZeroRotator, PolyCenterVector, FVector(CellSizeVector.X, CellSizeVector.Y, 0.0001)), /*bWorldSpace*/true);
		}
	}
}
