// Copyright Epic Games, Inc. All Rights Reserved.

#include "JumpFloodComponent2D.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "WaterEditorModule.h"
#include "WaterUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(JumpFloodComponent2D)

UJumpFloodComponent2D::UJumpFloodComponent2D(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
	, BlurPasses(1)
	, CompletedBlurPasses(0)
{
	bCanEverAffectNavigation = false;
}

bool UJumpFloodComponent2D::ValidateJumpFloodRenderTargets()
{
	if ((RTA == nullptr) 
		|| (RTB == nullptr) 
		|| (RTA->GetFormat() != RTB->GetFormat()) 
		|| (RTA->SizeX != RTB->SizeX) 
		|| (RTA->SizeY != RTB->SizeY))
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Render Target used in Jump Flood Component 2D.");
		return false;
	}

	return true;
}

bool UJumpFloodComponent2D::ValidateJumpFloodRequirements()
{
	if (JumpStepMID == nullptr)
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Jump Step material used in Jump Flood Component 2D.");
		return false;		
	}

	if (UseBlur && (BlurEdgesMID == nullptr))
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Blur Edges material used in Jump Flood Component 2D.");
		return false;
	}

	if (FindEdgesMID == nullptr)
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Find Edges material used in Jump Flood Component 2D.");
		return false;
	}

	return ValidateJumpFloodRenderTargets();
}

void UJumpFloodComponent2D::JumpFlood(UTextureRenderTarget2D* SeedRT, float SceneCaptureZ, FLinearColor Curl, bool UseDepth, float ZLocationT)
{
	CreateMIDs();
	
	if (!ValidateJumpFloodRequirements())
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid setup for Jump Flood Component 2D. Aborting JumpFlood.");
		return;
	}

	check(JumpStepMID != nullptr);

	FindEdges(SeedRT, SceneCaptureZ, Curl, UseDepth, ZLocationT);

	// Edge Blur
	if (UseBlur)
	{
		for(int32 BlurPassIndex = 0, NumBlurPasses = FMath::Min(BlurPasses, 9) ; BlurPassIndex < NumBlurPasses; ++BlurPassIndex)
		{
			SingleBlurStep();
		}
	}

	JumpStepMID->SetVectorParameterValue(FName(TEXT("TextureSize")), FLinearColor((float)RTA->SizeX, (float)RTA->SizeY, 0.0f, 0.0f));

	CompletedPasses = 0;
	RequiredPasses = FMath::CeilToInt(FMath::Loge((float)FMath::Max(RTA->SizeX, RTA->SizeY)) / FMath::Loge(2.0f));

	while (CompletedPasses < RequiredPasses)
	{
		SingleJumpStep();
	}
}

bool UJumpFloodComponent2D::CreateMIDs()
{
	JumpStepMID = FWaterUtils::GetOrCreateTransientMID(JumpStepMID, TEXT("JumpStepMID"), JumpStepMaterial);
	BlurEdgesMID = FWaterUtils::GetOrCreateTransientMID(BlurEdgesMID, TEXT("BlurEdgesMID"), BlurEdgesMaterial);
	FindEdgesMID = FWaterUtils::GetOrCreateTransientMID(FindEdgesMID, TEXT("FindEdgesMID"), FindEdgesMaterial);

	if ((JumpStepMID == nullptr) || (BlurEdgesMID == nullptr) || (FindEdgesMID == nullptr))
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid JumpFlood materials.");
		return false;
	}

	return true;
}

void UJumpFloodComponent2D::AssignRenderTargets(UTextureRenderTarget2D* InRTA, UTextureRenderTarget2D* InRTB)
{
	RTA = InRTA;
	RTB = InRTB;
}

UTextureRenderTarget2D* UJumpFloodComponent2D::SingleJumpStep()
{
	if (JumpStepMID == nullptr)
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Jump Step material for Jump Flood Component 2D. Aborting SingleJumpStep.");
		return nullptr;
	}

	if (!ValidateJumpFloodRenderTargets())
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Render Targetfor Jump Flood Component 2D. Aborting SingleJumpStep.");
		return nullptr;
	}

	check(JumpStepMID)

	JumpStepMID->SetScalarParameterValue(FName(TEXT("Index")), float(CompletedPasses+1));

	const int32 Offset = CompletedBlurPasses % 2;

	JumpStepMID->SetTextureParameterValue(FName(TEXT("RT")), PingPongSource(Offset));

	UTextureRenderTarget2D* RenderTarget = PingPongTarget(Offset);
	UKismetRenderingLibrary::ClearRenderTarget2D(this, RenderTarget, FLinearColor::Black);
	UKismetRenderingLibrary::DrawMaterialToRenderTarget(this, RenderTarget, JumpStepMID);
	
	++CompletedPasses;

	return PingPongSource(Offset);
}

UTextureRenderTarget2D* UJumpFloodComponent2D::FindEdges(UTextureRenderTarget2D* InSeed, float InCaptureZ, FLinearColor InCurl, bool InUseDepth, float InZLocation)
{
	if (FindEdgesMID == nullptr)
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Find Edges material for Jump Flood Component 2D. Aborting FindEdges.");
		return nullptr;
	}

	if (!ValidateJumpFloodRenderTargets())
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Render Targetfor Jump Flood Component 2D. Aborting FindEdges.");
		return nullptr;
	}

	check(FindEdgesMID);

	CompletedBlurPasses = 0;
	CompletedPasses = 0;

	FindEdgesMID->SetVectorParameterValue(FName(TEXT("TextureSize")), FLinearColor((float)InSeed->SizeX, (float)InSeed->SizeY, 0.0f, 0.0f));
	FindEdgesMID->SetTextureParameterValue(FName(TEXT("SeedEdgesRT")), InSeed);
	FindEdgesMID->SetVectorParameterValue(FName(TEXT("Curl")), InCurl);
	FindEdgesMID->SetScalarParameterValue(FName(TEXT("SceneCaptureZ")), InCaptureZ);
	FindEdgesMID->SetScalarParameterValue(FName(TEXT("UsesDepth")), InUseDepth ? 1.0f : 0.0f);
	FindEdgesMID->SetScalarParameterValue(FName(TEXT("Z")), InZLocation);
	UKismetRenderingLibrary::ClearRenderTarget2D(this, RTA, FLinearColor::Black);
	UKismetRenderingLibrary::ClearRenderTarget2D(this, RTB, FLinearColor::Black);
	UKismetRenderingLibrary::DrawMaterialToRenderTarget(this, RTA, FindEdgesMID);
	
	return PingPongSource(0);
}

void UJumpFloodComponent2D::FindEdges_Debug(UTextureRenderTarget2D* InSeed, float InCaptureZ, FLinearColor InCurl, UTextureRenderTarget2D* inDest, float InZOffset)
{
	if (FindEdgesMID == nullptr)
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Find Edges material for Jump Flood Component 2D. Aborting FindEdges_Debug.");
		return;
	}

	if (!ValidateJumpFloodRenderTargets())
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Render Targetfor Jump Flood Component 2D. Aborting FindEdges_Debug.");
		return;
	}

	check(FindEdgesMID);

	CompletedPasses = 0;

	FindEdgesMID->SetVectorParameterValue(FName(TEXT("TextureSize")), FLinearColor((float)InSeed->SizeX, (float)InSeed->SizeY, 0.0f, 0.0f));
	FindEdgesMID->SetTextureParameterValue(FName(TEXT("SeedEdgesRT")), InSeed);
	FindEdgesMID->SetVectorParameterValue(FName(TEXT("Curl")), InCurl);
	FindEdgesMID->SetScalarParameterValue(FName(TEXT("SceneCaptureZ")), InCaptureZ);
	FindEdgesMID->SetScalarParameterValue(FName(TEXT("UsesDepth")), FMath::IsNearlyEqual(InCaptureZ, 0.0f) ? 0.0f : 1.0f);
	FindEdgesMID->SetScalarParameterValue(FName(TEXT("Z")), InZOffset);

	UKismetRenderingLibrary::ClearRenderTarget2D(this, inDest, FLinearColor::Black);
	UKismetRenderingLibrary::DrawMaterialToRenderTarget(this, inDest, FindEdgesMID);
}

UTextureRenderTarget2D* UJumpFloodComponent2D::SingleBlurStep()
{
	if (BlurEdgesMID == nullptr)
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Blur Edges material for Jump Flood Component 2D. Aborting SingleBlurStep.");
		return nullptr;
	}

	if (!ValidateJumpFloodRenderTargets())
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Render Targetfor Jump Flood Component 2D. Aborting SingleBlurStep.");
		return nullptr;
	}

	check(BlurEdgesMID);

	UTextureRenderTarget2D* Source = PingPongSource(0);
	check(Source != nullptr);
	UTextureRenderTarget2D* Target = PingPongTarget(0);
	check(Target != nullptr);

	FVector2D TextureSize((float)Source->SizeX, (float)Source->SizeY);
	BlurEdgesMID->SetVectorParameterValue(FName(TEXT("TextureResolution")), FLinearColor(TextureSize.X, TextureSize.Y, 0.0f, 0.0f));
	BlurEdgesMID->SetTextureParameterValue(FName(TEXT("SeedEdgesRT")), Source);
	UKismetRenderingLibrary::ClearRenderTarget2D(this, Target, FLinearColor::Black);
	UKismetRenderingLibrary::DrawMaterialToRenderTarget(this, Target, BlurEdgesMID);

	++CompletedPasses;
	++CompletedBlurPasses;

 	return Source;
}

UTextureRenderTarget2D* UJumpFloodComponent2D::PingPongSource(int32 Offset) const
{
	if ((Offset + CompletedPasses) % 2)
	{
		return RTB;
	} 
	else
	{
		return RTA;
	}
}

UTextureRenderTarget2D* UJumpFloodComponent2D::PingPongTarget(int32 Offset) const
{
	if ((Offset + CompletedPasses + 1) % 2)
	{
		return RTB;
	}
	else
	{
		return RTA;
	}
}

