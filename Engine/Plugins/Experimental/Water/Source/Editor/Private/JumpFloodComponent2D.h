// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "JumpFloodComponent2D.generated.h"

class UTextureRenderTarget2D;
class UMaterialInstanceDynamic;
class UMaterialInterface;

UCLASS(config = Engine, Blueprintable, BlueprintType)
class UJumpFloodComponent2D : public UActorComponent
{
public:
	GENERATED_BODY()

	UJumpFloodComponent2D(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	UFUNCTION(BlueprintCallable, meta = (Category))
	virtual bool CreateMIDs();

	UFUNCTION(BlueprintCallable, meta = (Category))
	void AssignRenderTargets(UTextureRenderTarget2D* InRTA, UTextureRenderTarget2D* InRTB);

	UFUNCTION(BlueprintCallable, meta = (Category))
	void JumpFlood(UTextureRenderTarget2D* SeedRT, float SceneCaptureZ, FLinearColor Curl, bool UseDepth, float ZxLocationT);

	UFUNCTION(BlueprintCallable, meta = (Category))
	UTextureRenderTarget2D* SingleJumpStep();

	UFUNCTION(BlueprintCallable, meta = (Category))
	UTextureRenderTarget2D* FindEdges(UTextureRenderTarget2D* SeedRT, float CaptureZ, FLinearColor Curl, bool UseDepth, float ZxLocationT);

	UFUNCTION(BlueprintCallable, meta = (Category))
	void FindEdges_Debug(UTextureRenderTarget2D* SeedRT, float CaptureZ, FLinearColor Curl, UTextureRenderTarget2D* DestRT, float ZOffset);

	UFUNCTION(BlueprintCallable, meta = (Category))
	UTextureRenderTarget2D* SingleBlurStep();

public:
	UPROPERTY(EditInstanceOnly, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Materials"))
	TObjectPtr<UMaterialInterface> JumpStepMaterial = nullptr;

	UPROPERTY(EditInstanceOnly, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Materials"))
	TObjectPtr<UMaterialInterface> FindEdgesMaterial = nullptr;

	UPROPERTY(EditInstanceOnly, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Materials"))
	TObjectPtr<UMaterialInterface> BlurEdgesMaterial = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(Category="Default"))
	bool UseBlur = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta=(Category="Default"))
	int32 BlurPasses = 1;

private:
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Default"))
	TObjectPtr<UTextureRenderTarget2D> RTA;

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Default"))
	TObjectPtr<UTextureRenderTarget2D> RTB;

	// Transient properties (exposed only for debugging reasons) :
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Transient, meta = (Category = "Debug"))
	TObjectPtr<UMaterialInstanceDynamic> JumpStepMID;

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Transient, meta = (Category = "Debug"))
	TObjectPtr<UMaterialInstanceDynamic> FindEdgesMID;

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Transient, meta = (Category = "Debug"))
	TObjectPtr<UMaterialInstanceDynamic> BlurEdgesMID;

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Transient, meta = (Category = "Debug"))
	int32 RequiredPasses = 0;

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Transient, meta = (Category = "Debug"))
	int32 CompletedPasses = 0;

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Transient, meta = (Category = "Debug"))
	int32 CompletedBlurPasses = 0;

	bool ValidateJumpFloodRenderTargets();
	bool ValidateJumpFloodRequirements();
	UTextureRenderTarget2D* PingPongSource(int32 Offset) const;
	UTextureRenderTarget2D* PingPongTarget(int32 Offset) const;
};
