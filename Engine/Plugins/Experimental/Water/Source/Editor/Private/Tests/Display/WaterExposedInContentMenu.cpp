// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "IPlacementModeModule.h"

#if WITH_AUTOMATION_TESTS
// TestRail: 27425803
// TestRail: 36679397
// TestRail: 36679398
// TestRail: 36679399
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FWaterExposedInContentMenuTest, "Editor.Plugins.Tools.Water", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

void FWaterExposedInContentMenuTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	OutTestCommands.Add(TEXT("Water Body River"));
	OutBeautifiedNames.Add(TEXT("RiverBodyExposedInContentMenu"));
	OutTestCommands.Add(TEXT("Water Body Lake"));
	OutBeautifiedNames.Add(TEXT("LakeBodyExposedInContentMenu"));
	OutTestCommands.Add(TEXT("Water Body Ocean"));
	OutBeautifiedNames.Add(TEXT("OceanBodyExposedInContentMenu"));
	OutTestCommands.Add(TEXT("Water Body Island"));
	OutBeautifiedNames.Add(TEXT("IslandBodyExposedInContentMenu"));
}

bool FWaterExposedInContentMenuTest::RunTest(const FString& Parameters)
{
	// Getting the module
	const IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
	// Water Body Actors are only accessible through All Classes
	const FName PlacementModeCategoryHandle = TEXT("AllClasses");
	FString WaterBodyName = Parameters;

	TArray<TSharedPtr<FPlaceableItem>> OutItems;
	// We need to manually load All Classes to search for a Water Body Actor
	PlacementModeModule.Get().RegenerateItemsForCategory(PlacementModeCategoryHandle);

	PlacementModeModule.GetItemsForCategory(PlacementModeCategoryHandle, OutItems);
	bool WaterBodyFound = OutItems.ContainsByPredicate([&WaterBodyName](const TSharedPtr<FPlaceableItem>& Item) {
		return Item->DisplayName.ToString().Contains(WaterBodyName);
		});

	TestTrue("Water Body " + WaterBodyName + " is not found in Create Menu", WaterBodyFound);

	return true;
}

#endif