// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "Engine/World.h"
#include "Misc/AutomationTest.h"
#include "IPlacementModeModule.h"
#include "EditorWorldUtils.h"
#include "EngineUtils.h"
#include "Landscape.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "WaterBodyIslandActor.h"
#include "WaterBodyLakeActor.h"
#include "WaterBodyOceanActor.h"
#include "WaterBodyRiverActor.h"
#include "Editor/TemplateMapInfo.h"

/**
 * Implements a complex automation test for verifying water bodies in landscape edit layers.
 */
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FWaterInLandscapeEditLayerTest, "Plugins.Water.Landscape", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FWaterInLandscapeEditLayerTest::RunTest(const FString& Parameters)
{
	// Clean up after the test to ensure no leftover selections or objects.
	ON_SCOPE_EXIT
	{
		GEditor->SelectNone(true, true, false);
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	};
	// Retrieve template map information.
	TArray<FTemplateMapInfo> TemplateMaps;
	TemplateMaps = GUnrealEd ? GUnrealEd->GetTemplateMapInfos() : TemplateMaps;
	const FText TargetDisplayName = FText::FromString("Open World");
	// Find the "Open World" template map.
	const FTemplateMapInfo* FoundTemplate = TemplateMaps.FindByPredicate([&](const FTemplateMapInfo& TemplateMapInfo) {
		return TemplateMapInfo.DisplayName.EqualTo(TargetDisplayName);
		});
	if (!TestNotNull(TEXT("OpenWorld map found successfully"), FoundTemplate))
	{
		return false;
	}
	// Set up the editor world using the found template.
	const FScopedEditorWorld ScopedEditorWorld(
		FoundTemplate->Map.ToString(),
		UWorld::InitializationValues()
		.RequiresHitProxies(false)
		.ShouldSimulatePhysics(false)
		.EnableTraceCollision(false)
		.CreateNavigation(false)
		.CreateAISystem(false)
		.AllowAudioPlayback(false)
		.CreatePhysicsScene(true)
	);
	const UWorld* World = ScopedEditorWorld.GetWorld();
	if (!TestNotNull(TEXT("World is valid"), World))
	{
		return false;
	}
	// Helper lambda function to create water body actors.
	auto CreateActor = [&](const UClass* ActorClass) -> AActor*
    {
        if (auto* ActorFactory = GEditor->FindActorFactoryForActorClass(ActorClass))
        {
            return ActorFactory->CreateActor(nullptr, World->GetCurrentLevel(), FTransform({ 0.f, 0.f, 0.f }, { 0.f, 0.f, 0.f }));
        }
        return nullptr;
    };
	// Create actors based on the parameters.
	if (Parameters.Contains("River"))
	{
		CreateActor(AWaterBodyRiver::StaticClass());
	}
	if (Parameters.Contains("Lake"))
	{
		CreateActor(AWaterBodyLake::StaticClass());
	}
	if (Parameters.Contains("Ocean"))
	{
		CreateActor(AWaterBodyOcean::StaticClass());
	}
	if (Parameters.Contains("Island"))
	{
		CreateActor(AWaterBodyIsland::StaticClass());
	}
	// Check for the presence of the "Water" layer in landscapes.
	const FName WaterLayerName = TEXT("Water");
	bool bWaterLayerFound = false;
	for (const ALandscape* Landscape : TActorRange<ALandscape>(World))
	{
		if (Landscape && Landscape->GetLayerIndex(WaterLayerName) != INDEX_NONE)
		{
			bWaterLayerFound = true;
			break;
		}
	}
	return TestTrue(TEXT("Water layer was found"), bWaterLayerFound);
}

void FWaterInLandscapeEditLayerTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	const TArray<FString> WaterLandscapeNamesGetter
	{
		TEXT("VerifyLandscapeRiverWaterLayer"),
		TEXT("VerifyLandscapeLakeWaterLayer"),
		TEXT("VerifyLandscapeOceanWaterLayer"),
		TEXT("VerifyLandscapeIslandWaterLayer")
	};

	const TArray<FString> WaterLandscapeCommandsGetter
	{
		TEXT("Water Body River"),
		TEXT("Water Body Lake"),
		TEXT("Water Body Ocean"),
		TEXT("Water Body Island")
	};

	FAutomationTestFramework& Framework = FAutomationTestFramework::Get();

	for (const FString& Test : WaterLandscapeNamesGetter)
	{
		OutBeautifiedNames.Add(Test);
		Framework.RegisterComplexAutomationTestTags(this, Test, TEXT("[GraphicsTools][Terrain][Water][DailyEssential]"));
	}

	for (const FString& TestCmd : WaterLandscapeCommandsGetter)
	{
		OutTestCommands.Add(TestCmd);
	}
}

#endif

#endif