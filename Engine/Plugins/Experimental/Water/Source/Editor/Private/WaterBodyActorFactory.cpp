// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyActorFactory.h"
#include "WaterEditorSettings.h"
#include "WaterBodyRiverActor.h"
#include "WaterBodyOceanActor.h"
#include "WaterBodyLakeActor.h"
#include "WaterBodyCustomActor.h"
#include "WaterBodyOceanComponent.h"
#include "WaterBodyRiverComponent.h"
#include "WaterSplineComponent.h"
#include "WaterZoneActor.h"
#include "WaterRuntimeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyActorFactory)

#define LOCTEXT_NAMESPACE "WaterBodyActorFactory"

// --------------------------------------------------
// WaterBody Factory
// --------------------------------------------------
UWaterBodyActorFactory::UWaterBodyActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUseSurfaceOrientation = true;
}

void UWaterBodyActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	UWaterBodyComponent* WaterBodyComponent = CastChecked<AWaterBody>(NewActor)->GetWaterBodyComponent();
	check(WaterBodyComponent);

	if (const FWaterBrushActorDefaults* WaterBrushActorDefaults = GetWaterBrushActorDefaults())
	{
		WaterBodyComponent->CurveSettings = WaterBrushActorDefaults->CurveSettings;
		WaterBodyComponent->WaterHeightmapSettings = WaterBrushActorDefaults->HeightmapSettings;
		WaterBodyComponent->LayerWeightmapSettings = WaterBrushActorDefaults->LayerWeightmapSettings;
	}

	if (const FWaterBodyDefaults* WaterBodyDefaults = GetWaterBodyDefaults())
	{
		WaterBodyComponent->SetWaterMaterial(WaterBodyDefaults->GetWaterMaterial());
		WaterBodyComponent->SetWaterStaticMeshMaterial(WaterBodyDefaults->GetWaterStaticMeshMaterial());
		WaterBodyComponent->SetHLODMaterial(WaterBodyDefaults->GetWaterHLODMaterial());
		WaterBodyComponent->SetUnderwaterPostProcessMaterial(WaterBodyDefaults->GetUnderwaterPostProcessMaterial());

		WaterBodyComponent->SetWaterInfoMaterial(GetDefault<UWaterRuntimeSettings>()->GetDefaultWaterInfoMaterial());

		UWaterSplineComponent* WaterSpline = WaterBodyComponent->GetWaterSpline();
		if (ShouldOverrideWaterSplineDefaults(WaterSpline))
		{
			WaterSpline->WaterSplineDefaults = WaterBodyDefaults->SplineDefaults;
		}
	}

	// If the water body is spawned into a zone which is using local only tessellation, we must default to enabling static meshes.
	if (const AWaterZone* WaterZone = WaterBodyComponent->GetWaterZone())
	{
		if (WaterZone->IsLocalOnlyTessellationEnabled())
		{
			WaterBodyComponent->SetWaterBodyStaticMeshEnabled(true);
		}
	}
}

// If WaterSpline's owning actor class is a BP class, don't allow to override WaterSplineDefaults
bool UWaterBodyActorFactory::ShouldOverrideWaterSplineDefaults(const UWaterSplineComponent* WaterSpline) const
{
	check(WaterSpline);
	AWaterBody* OwningBody = WaterSpline->GetTypedOuter<AWaterBody>();
	return OwningBody && OwningBody->GetClass()->ClassGeneratedBy == nullptr;
}

// --------------------------------------------------
// WaterBodyRiver Factory
// --------------------------------------------------
UWaterBodyRiverActorFactory::UWaterBodyRiverActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("WaterBodyRiverActorDisplayName", "Water Body River");
	NewActorClass = AWaterBodyRiver::StaticClass();
}

const FWaterBodyDefaults* UWaterBodyRiverActorFactory::GetWaterBodyDefaults() const
{
	return &GetDefault<UWaterEditorSettings>()->WaterBodyRiverDefaults;
}

const FWaterBrushActorDefaults* UWaterBodyRiverActorFactory::GetWaterBrushActorDefaults() const
{
	return &GetDefault<UWaterEditorSettings>()->WaterBodyRiverDefaults.BrushDefaults;
}

void UWaterBodyRiverActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);
	
	AWaterBodyRiver* WaterBodyActor = CastChecked<AWaterBodyRiver>(NewActor);
	UWaterBodyRiverComponent* WaterBodyRiverComponent = CastChecked<UWaterBodyRiverComponent>(WaterBodyActor->GetWaterBodyComponent());
	WaterBodyRiverComponent->SetLakeTransitionMaterial(GetDefault<UWaterEditorSettings>()->WaterBodyRiverDefaults.GetRiverToLakeTransitionMaterial());
	WaterBodyRiverComponent->SetOceanTransitionMaterial(GetDefault<UWaterEditorSettings>()->WaterBodyRiverDefaults.GetRiverToOceanTransitionMaterial());

	UWaterSplineComponent* WaterSpline = WaterBodyRiverComponent->GetWaterSpline();
	WaterSpline->ResetSpline({ FVector(0, 0, 0), FVector(5000, 0, 0), FVector(10000, 5000, 0) });
}

// --------------------------------------------------
// WaterBodyOcean Factory
// --------------------------------------------------
UWaterBodyOceanActorFactory::UWaterBodyOceanActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("WaterBodyOceanActorDisplayName", "Water Body Ocean");
	NewActorClass = AWaterBodyOcean::StaticClass();
}

const FWaterBodyDefaults* UWaterBodyOceanActorFactory::GetWaterBodyDefaults() const
{
	return &GetDefault<UWaterEditorSettings>()->WaterBodyOceanDefaults;
}

const FWaterBrushActorDefaults* UWaterBodyOceanActorFactory::GetWaterBrushActorDefaults() const
{
	return &GetDefault<UWaterEditorSettings>()->WaterBodyOceanDefaults.BrushDefaults;
}

void UWaterBodyOceanActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	AWaterBodyOcean* WaterBodyOcean = CastChecked<AWaterBodyOcean>(NewActor);
	UWaterBodyComponent* WaterBodyComponent = WaterBodyOcean->GetWaterBodyComponent();
	check(WaterBodyComponent != nullptr);
	
	if (const UWaterWavesBase* DefaultWaterWaves = GetDefault<UWaterEditorSettings>()->WaterBodyOceanDefaults.WaterWaves)
	{
		UWaterWavesBase* WaterWaves = DuplicateObject(DefaultWaterWaves, NewActor, MakeUniqueObjectName(NewActor, DefaultWaterWaves->GetClass(), TEXT("OceanWaterWaves")));
		WaterBodyOcean->SetWaterWaves(WaterWaves);
	}

	UWaterSplineComponent* WaterSpline = WaterBodyComponent->GetWaterSpline();
	WaterSpline->ResetSpline({ FVector(10000, -10000, 0), FVector(10000,  10000, 0), FVector(-10000,  10000, 0), FVector(-10000, -10000, 0) });

	if (const AWaterZone* OwningWaterZone = WaterBodyComponent->GetWaterZone())
	{
		if (UWaterBodyOceanComponent* OceanComponent = Cast<UWaterBodyOceanComponent>(WaterBodyComponent))
		{
			const double ExistingCollisionHeight = OceanComponent->GetCollisionExtents().Z;
			OceanComponent->SetCollisionExtents(FVector(OwningWaterZone->GetZoneExtent() / 2.0, ExistingCollisionHeight));
			OceanComponent->FillWaterZoneWithOcean();
		}
	}
}

// --------------------------------------------------
// WaterBodyLake Factory
// --------------------------------------------------
UWaterBodyLakeActorFactory::UWaterBodyLakeActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("WaterBodyLakeActorDisplayName", "Water Body Lake");
	NewActorClass = AWaterBodyLake::StaticClass();
}

const FWaterBodyDefaults* UWaterBodyLakeActorFactory::GetWaterBodyDefaults() const
{
	return &GetDefault<UWaterEditorSettings>()->WaterBodyLakeDefaults;
}

const FWaterBrushActorDefaults* UWaterBodyLakeActorFactory::GetWaterBrushActorDefaults() const
{
	return &GetDefault<UWaterEditorSettings>()->WaterBodyLakeDefaults.BrushDefaults;
}

void UWaterBodyLakeActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	AWaterBodyLake* WaterBodyLake = CastChecked<AWaterBodyLake>(NewActor);
	UWaterBodyComponent* WaterBodyComponent = WaterBodyLake->GetWaterBodyComponent();
	check(WaterBodyComponent != nullptr);
	
	if (const UWaterWavesBase* DefaultWaterWaves = GetDefault<UWaterEditorSettings>()->WaterBodyLakeDefaults.WaterWaves)
	{
		UWaterWavesBase* WaterWaves = DuplicateObject(DefaultWaterWaves, NewActor, MakeUniqueObjectName(NewActor, DefaultWaterWaves->GetClass(), TEXT("LakeWaterWaves")));
		WaterBodyLake->SetWaterWaves(WaterWaves);
	}

	UWaterSplineComponent* WaterSpline = WaterBodyLake->GetWaterSpline();
	WaterSpline->ResetSpline({ FVector(0, 0, 0), FVector(7000, -3000, 0),  FVector(6500, 6500, 0) });
}

// --------------------------------------------------
// WaterBodyCustom Factory
// --------------------------------------------------
UWaterBodyCustomActorFactory::UWaterBodyCustomActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("WaterBodyCustomActorDisplayName", "Water Body Custom");
	NewActorClass = AWaterBodyCustom::StaticClass();
}

const FWaterBodyDefaults* UWaterBodyCustomActorFactory::GetWaterBodyDefaults() const
{
	return &GetDefault<UWaterEditorSettings>()->WaterBodyCustomDefaults;
}

void UWaterBodyCustomActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	AWaterBodyCustom* WaterBodyCustom = CastChecked<AWaterBodyCustom>(NewActor);
	WaterBodyCustom->GetWaterBodyComponent()->SetWaterMeshOverride(GetDefault<UWaterEditorSettings>()->WaterBodyCustomDefaults.GetWaterMesh());

	UWaterSplineComponent* WaterSpline = WaterBodyCustom->GetWaterSpline();
	WaterSpline->ResetSpline({ FVector(0, 0, 0) });
}

#undef LOCTEXT_NAMESPACE
