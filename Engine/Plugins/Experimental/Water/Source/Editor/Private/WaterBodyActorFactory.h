// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ActorFactories/ActorFactory.h"
#include "WaterBodyActorFactory.generated.h"

struct FWaterBodyDefaults;
struct FWaterBrushActorDefaults;

UCLASS(Abstract, MinimalAPI, config = Editor)
class UWaterBodyActorFactory : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;

protected:
	virtual const FWaterBodyDefaults* GetWaterBodyDefaults() const { return nullptr; }
	virtual const FWaterBrushActorDefaults* GetWaterBrushActorDefaults() const { return nullptr; }
	virtual bool ShouldOverrideWaterSplineDefaults(const class UWaterSplineComponent* WaterSpline) const;
};

UCLASS(MinimalAPI, config = Editor)
class UWaterBodyRiverActorFactory : public UWaterBodyActorFactory
{
	GENERATED_UCLASS_BODY()

	virtual const FWaterBodyDefaults* GetWaterBodyDefaults() const override;
	virtual const FWaterBrushActorDefaults* GetWaterBrushActorDefaults() const override;
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
};

UCLASS(MinimalAPI, config = Editor)
class UWaterBodyOceanActorFactory : public UWaterBodyActorFactory
{
	GENERATED_UCLASS_BODY()

	virtual const FWaterBodyDefaults* GetWaterBodyDefaults() const override;
	virtual const FWaterBrushActorDefaults* GetWaterBrushActorDefaults() const override;
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
};

UCLASS(MinimalAPI, config = Editor)
class UWaterBodyLakeActorFactory : public UWaterBodyActorFactory
{
	GENERATED_UCLASS_BODY()

	virtual const FWaterBodyDefaults* GetWaterBodyDefaults() const override;
	virtual const FWaterBrushActorDefaults* GetWaterBrushActorDefaults() const override;
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
};

UCLASS(MinimalAPI, config = Editor)
class UWaterBodyCustomActorFactory : public UWaterBodyActorFactory
{
	GENERATED_UCLASS_BODY()

	virtual const FWaterBodyDefaults* GetWaterBodyDefaults() const override;
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
};
