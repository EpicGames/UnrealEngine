// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyBrushCacheContainerThumbnailRenderer.h"
#include "Engine/TextureRenderTarget2D.h"
#include "WaterBrushCacheContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyBrushCacheContainerThumbnailRenderer)

UWaterBodyBrushCacheContainerThumbnailRenderer::UWaterBodyBrushCacheContainerThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UWaterBodyBrushCacheContainerThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	return Super::CanVisualizeAsset(GetRenderTarget(Object));
}

void UWaterBodyBrushCacheContainerThumbnailRenderer::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	Super::GetThumbnailSize(GetRenderTarget(Object), Zoom, OutWidth, OutHeight);
}

void UWaterBodyBrushCacheContainerThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	Super::Draw(GetRenderTarget(Object), X, Y, Width, Height, RenderTarget, Canvas, bAdditionalViewFamily);
}

UTextureRenderTarget2D* UWaterBodyBrushCacheContainerThumbnailRenderer::GetRenderTarget(UObject* Object) const
{
	UWaterBodyBrushCacheContainer* WaterBodyBrushCacheContainer = Cast<UWaterBodyBrushCacheContainer>(Object);
	return WaterBodyBrushCacheContainer ? WaterBodyBrushCacheContainer->Cache.CacheRenderTarget : nullptr;
}
