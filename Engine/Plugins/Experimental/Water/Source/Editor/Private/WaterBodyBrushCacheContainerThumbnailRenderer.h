// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ThumbnailRendering/TextureThumbnailRenderer.h"
#include "WaterBodyBrushCacheContainerThumbnailRenderer.generated.h"

UCLASS(MinimalAPI)
class UWaterBodyBrushCacheContainerThumbnailRenderer : public UTextureThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	//~ Begin UThumbnailRenderer Interface.
	virtual bool CanVisualizeAsset(UObject* Object) override;
	virtual void GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const override;
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	//~ EndUThumbnailRenderer Interface.

private:
	class UTextureRenderTarget2D* GetRenderTarget(UObject* Object) const;
};
