// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyIslandActorFactory.h"
#include "WaterBodyIslandActor.h"
#include "WaterEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyIslandActorFactory)

#define LOCTEXT_NAMESPACE "WaterBodyIslandActorFactory"

UWaterBodyIslandActorFactory::UWaterBodyIslandActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("WaterBodyIslandActorDisplayName", "Water Body Island");
	NewActorClass = AWaterBodyIsland::StaticClass();
	bUseSurfaceOrientation = true;
}

void UWaterBodyIslandActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);
	
	AWaterBodyIsland* WaterBodyIsland = CastChecked<AWaterBodyIsland>(NewActor);

	WaterBodyIsland->WaterCurveSettings = GetDefault<UWaterEditorSettings>()->WaterBodyIslandDefaults.BrushDefaults.CurveSettings;
	WaterBodyIsland->WaterHeightmapSettings = GetDefault<UWaterEditorSettings>()->WaterBodyIslandDefaults.BrushDefaults.HeightmapSettings;
	WaterBodyIsland->WaterWeightmapSettings = GetDefault<UWaterEditorSettings>()->WaterBodyIslandDefaults.BrushDefaults.LayerWeightmapSettings;
}

#undef LOCTEXT_NAMESPACE
