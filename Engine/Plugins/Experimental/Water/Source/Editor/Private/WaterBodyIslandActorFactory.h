// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ActorFactories/ActorFactory.h"
#include "WaterBodyIslandActorFactory.generated.h"

UCLASS(MinimalAPI, config = Editor)
class UWaterBodyIslandActorFactory : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
};

