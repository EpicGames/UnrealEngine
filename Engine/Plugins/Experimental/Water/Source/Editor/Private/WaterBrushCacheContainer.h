// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterBrushCacheContainer.generated.h"

class UTextureRenderTarget2D;

// TODO [jonathan.bard] : rename : this is not a WaterBodyBrushCache, this a simple RenderTarget with a boolean to force an update on it
//  This is also used for caching curves...
USTRUCT(BlueprintType)
struct FWaterBodyBrushCache
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cache")
	TObjectPtr<UTextureRenderTarget2D> CacheRenderTarget = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cache")
	bool CacheIsValid = false;
};

UCLASS(config = Engine, Blueprintable, BlueprintType)
class UWaterBodyBrushCacheContainer : public UObject
{
	GENERATED_BODY()

public:

	UWaterBodyBrushCacheContainer(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, meta=(DisplayName="Cache", Category="Default"))
	FWaterBodyBrushCache Cache;
};
