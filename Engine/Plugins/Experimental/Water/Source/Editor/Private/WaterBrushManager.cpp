// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBrushManager.h"
#include "Camera/CameraTypes.h"
#include "JumpFloodComponent2D.h"
#include "Components/SceneCaptureComponent2D.h"
#include "Editor.h"
#include "WaterBodyIslandActor.h"
#include "Engine/Canvas.h"
#include "Engine/Texture2D.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "WaterBodyActor.h"
#include "WaterSplineComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "Materials/MaterialParameterCollection.h"
#include "WaterEditorModule.h"
#include "WaterEditorSubsystem.h"
#include "WaterEditorSettings.h"
#include "WaterSubsystem.h"
#include "WaterUtils.h"
#include "WaterVersion.h"
#include "Curves/CurveFloat.h"
#include "EngineUtils.h"
#include "Landscape.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "Logging/MessageLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBrushManager)

#define LOCTEXT_NAMESPACE "WaterBrushManager"

AWaterBrushManager::AWaterBrushManager(const FObjectInitializer& ObjectInitializer)
	: Super()
	, WorldSize(FVector::ZeroVector)
	, LandscapeRTRes(0, 0)
	, LandscapeTransform(FTransform::Identity)
{
	JumpFloodComponent2D = CreateDefaultSubobject<UJumpFloodComponent2D>(TEXT("JumpFloodComponent2D"));

	SceneCaptureComponent2D = CreateDefaultSubobject<USceneCaptureComponent2D>(TEXT("SceneCaptureComponent2D"));
	SceneCaptureComponent2D->CreationMethod = EComponentCreationMethod::Native;
	SceneCaptureComponent2D->AttachToComponent(RootComponent, FAttachmentTransformRules::KeepRelativeTransform);
	SceneCaptureComponent2D->ProjectionType = ECameraProjectionMode::Type::Orthographic;
	SceneCaptureComponent2D->PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList;
	SceneCaptureComponent2D->CaptureSource = ESceneCaptureSource::SCS_SceneDepth;
	SceneCaptureComponent2D->bCaptureEveryFrame = false;
	SceneCaptureComponent2D->bCaptureOnMovement = false;
	SceneCaptureComponent2D->bExcludeFromSceneTextureExtents = true;  // Don't let this influence the render target size used for other purposes.
	SceneCaptureComponent2D->SetRelativeRotation(FRotator(-90.0f, 0.0f, -90.0f));
	SceneCaptureComponent2D->SetRelativeScale3D(FVector(0.01f, 0.01f, 0.01f));

	TArray<FEngineShowFlagsSetting> ShowFlagSettings;
	// HACK [jonathan.bard] : Nanite doesn't support USceneCaptureComponent's ShowOnlyComponents ATM so just disable Nanite during captures : 
	ShowFlagSettings.Add(FEngineShowFlagsSetting { TEXT("NaniteMeshes"), false } );
	// These also need to be disabled to get a clean capture of just the water info material output
	ShowFlagSettings.Add(FEngineShowFlagsSetting { TEXT("Atmosphere"), false } );
	ShowFlagSettings.Add(FEngineShowFlagsSetting { TEXT("Bloom"), false } );
	ShowFlagSettings.Add(FEngineShowFlagsSetting { TEXT("Lighting"), false } );
	ShowFlagSettings.Add(FEngineShowFlagsSetting { TEXT("Fog"), false } );
	SceneCaptureComponent2D->SetShowFlagSettings(ShowFlagSettings);

	PrimaryActorTick.TickGroup = ETickingGroup::TG_PrePhysics;
	bIsEditorOnlyActor = false;
}

void AWaterBrushManager::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FWaterCustomVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
}

void AWaterBrushManager::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FWaterCustomVersion::GUID) < FWaterCustomVersion::MoveBrushMaterialsToWaterBrushManager)
	{
		// Only setup some default materials for water brushes that were NOT instantiated by blueprint, otherwise we would override the default values contained in the BP : 
		if (GetClass()->ClassGeneratedBy == nullptr)
		{
			SetupDefaultMaterials();
		}
	}

	if (GetLinkerCustomVersion(FWaterCustomVersion::GUID) < FWaterCustomVersion::MoveJumpFloodMaterialsToWaterBrushManager)
	{
		if (JumpFloodComponent2D != nullptr)
		{
			if (JumpFloodComponent2D->BlurEdgesMaterial != nullptr)
			{
				BlurEdgesMaterial = JumpFloodComponent2D->BlurEdgesMaterial;
			}
			if (JumpFloodComponent2D->FindEdgesMaterial != nullptr)
			{
				FindEdgesMaterial = JumpFloodComponent2D->FindEdgesMaterial;
			}
			if (JumpFloodComponent2D->JumpStepMaterial != nullptr)
			{
				JumpStepMaterial = JumpFloodComponent2D->JumpStepMaterial;
			}
		}
	}

#if WITH_EDITOR
	if (GetLinkerCustomVersion(FWaterCustomVersion::GUID) < FWaterCustomVersion::MoveWaterMPCParamsToWaterMesh)
	{
		// OnPostLoad, the world is not set so we cannot retrieve the water mesh actor, we have to delay it to post-init :  
		OnWorldPostInitHandle = FWorldDelegates::OnPostWorldInitialization.AddLambda([this](UWorld* World, const UWorld::InitializationValues IVS)
		{
			if (World == GetWorld())
			{
				TActorIterator<AWaterZone> It(World);
				if (AWaterZone* WaterZoneActor = It ? *It : nullptr)
				{
					FVector RTWorldLocation, RTWorldSizeVector;
					if (DeprecateWaterLandscapeInfo(RTWorldLocation, RTWorldSizeVector))
					{
						SetMPCParams();
					}
				}
			}
		});
		
		// Each time a level is added, it might contain a landscape component, hence we need to deprecate RTWorldLocationand RTWorldSizeVector accordingly :
		OnLevelAddedToWorldHandle = FWorldDelegates::LevelAddedToWorld.AddLambda([this](ULevel* Level, UWorld* World)
		{
			if ((World == GetWorld()) && (Level != nullptr))
			{
				TActorIterator<AWaterZone> It(World);
				if (AWaterZone* WaterZoneActor = It ? *It : nullptr)
				{
					FVector RTWorldLocation, RTWorldSizeVector;
					if (DeprecateWaterLandscapeInfo(RTWorldLocation, RTWorldSizeVector))
					{
						SetMPCParams();
					}
				}
			}
		});

	}

	if (!IsTemplate() 
		&& (bNeedsForceUpdate || (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::RemoveLandscapeWaterInfo)))
	{
		// The removal of LandscapeWaterInfo is accompanied by a change in how the water velocity height texture is encoded so we need to regenerate it :
		bNeedsForceUpdate = true;

		ShowForceUpdateMapCheckError();

		// Show MapCheck window
		FMessageLog("MapCheck").Open(EMessageSeverity::Warning);
	}
#endif // WITH_EDITOR
}

void AWaterBrushManager::BeginDestroy()
{
	Super::BeginDestroy();

	FWorldDelegates::OnPostWorldInitialization.Remove(OnWorldPostInitHandle);
	OnWorldPostInitHandle.Reset();

	FWorldDelegates::LevelAddedToWorld.Remove(OnLevelAddedToWorldHandle);
	OnLevelAddedToWorldHandle.Reset();
}
	

void AWaterBrushManager::PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph)
{
	Super::PostLoadSubobjects(OuterInstanceGraph);

	if (JumpFloodComponent2D)
	{
		JumpFloodComponent2D->CreationMethod = EComponentCreationMethod::Native;
	}
	if (SceneCaptureComponent2D)
	{
		SceneCaptureComponent2D->CreationMethod = EComponentCreationMethod::Native;
	}
}

UCurveFloat* AWaterBrushManager::GetElevationCurveAsset(const FWaterCurveSettings& CurveSettings)
{
	if (CurveSettings.ElevationCurveAsset)
	{
		return CurveSettings.ElevationCurveAsset;
	}

	static FSoftObjectPath DefaultCurve(TEXT("/Water/Curves/FloatCurve.FloatCurve"));
	return Cast<UCurveFloat>(DefaultCurve.TryLoad());
}

UTextureRenderTarget2D* AWaterBrushManager::VelocityPingPongRead(const FBrushRenderContext& BrushRenderContext) const
{
	if (BrushRenderContext.VelocityRTIndex % 2)
	{
		return CombinedVelocityAndHeightRTA;
	}
	else
	{
		return CombinedVelocityAndHeightRTB;
	}
}


UTextureRenderTarget2D* AWaterBrushManager::VelocityPingPongWrite(const FBrushRenderContext& BrushRenderContext) const
{
	if (BrushRenderContext.VelocityRTIndex % 2)
	{
		return CombinedVelocityAndHeightRTB;
	}
	else
	{
		return CombinedVelocityAndHeightRTA;
	}
}


UTextureRenderTarget2D* AWaterBrushManager::HeightPingPongRead(const FBrushRenderContext& BrushRenderContext) const
{
	if (BrushRenderContext.RTIndex == 0)
	{
		return LandscapeRTRef;
	}
	else if (BrushRenderContext.RTIndex % 2)// Odd
	{
		return HeightmapRTA;
	}
	else // Even
	{
		return HeightmapRTB;
	}
}


UTextureRenderTarget2D* AWaterBrushManager::HeightPingPongWrite(const FBrushRenderContext& BrushRenderContext) const
{
	if (BrushRenderContext.RTIndex % 2)// Odd
	{
		return HeightmapRTB;
	}
	else // Even
	{
		return HeightmapRTA;
	}
}


UTextureRenderTarget2D* AWaterBrushManager::WeightPingPongRead(const FBrushRenderContext& BrushRenderContext) const
{
	if (BrushRenderContext.RTIndex == 0)
	{
		return LandscapeRTRef;
	}
	else if (BrushRenderContext.RTIndex % 2)// Odd
	{
		return WeightmapRTA;
	}
	else // Even
	{
		return WeightmapRTB;
	}
}


UTextureRenderTarget2D* AWaterBrushManager::WeightPingPongWrite(const FBrushRenderContext& BrushRenderContext) const
{
	if (BrushRenderContext.RTIndex % 2)// Odd
	{
		return WeightmapRTB;
	}
	else // Even
	{
		return WeightmapRTA;
	}
}

void AWaterBrushManager::AddDependencyIfValid(UObject* Dependency, TSet<UObject*>& OutDependencies)
{
	if (IsValid(Dependency))
	{
		OutDependencies.Add(Dependency);
	}
}

void AWaterBrushManager::ClearCurveCache()
{
	for (auto& CacheEntry : BrushCurveRTCache)
	{
		// Stop listening to OnUpdateCurve events on all curves in the cache : 
		CacheEntry.Key->OnUpdateCurve.RemoveAll(this);
	}
	BrushCurveRTCache.Empty();
}

void AWaterBrushManager::OnCurveUpdated(UCurveBase* Curve, EPropertyChangeType::Type ChangeType)
{
	UCurveFloat* CurveFloat = CastChecked<UCurveFloat>(Curve);

	// Rebuild the cache entry for this curve : 
	FWaterBodyBrushCache* CacheEntry = BrushCurveRTCache.Find(CurveFloat);
	check(CacheEntry != nullptr);
	CacheEntry->CacheIsValid = false;

	// And trigger a rebuild of all water brush actor cache entries using this curve : 
	for (TWeakInterfacePtr<IWaterBrushActorInterface> BrushActor : GetActorsAffectingLandscape())
	{
		if (BrushActor.IsValid())
		{
			const FWaterCurveSettings& LocalCurveSettings = BrushActor->GetWaterCurveSettings();
			UCurveFloat* ElevationCurveAsset = GetElevationCurveAsset(LocalCurveSettings);
			if (ElevationCurveAsset == CurveFloat)
			{
				UWaterBodyBrushCacheContainer* CacheContainer = nullptr;
				FWaterBodyBrushCache WaterBrushCache;
				GetWaterCacheKey(CastChecked<AActor>(BrushActor.GetObject()), /*out*/ CacheContainer, /*out*/ WaterBrushCache);
				if (CacheContainer != nullptr)
				{
					CacheContainer->Cache.CacheIsValid = false;
				}
			}
		}
	}

	RequestLandscapeUpdate();
}

void AWaterBrushManager::BlueprintOnRenderTargetTexturesUpdated_Native(UTexture2D* VelocityTexture)
{
	VelocityTexture->PreEditChange(nullptr);
	VelocityTexture->LODBias = 0;
	VelocityTexture->PostEditChange();

	UseDynamicPreviewRT = false;
}

void AWaterBrushManager::ForceUpdate()
{
#if WITH_EDITOR
	if (bNeedsForceUpdate)
	{
		// We need to mark our own package as dirty to force save the water brush manager and stop dispaying the ForceUpdate message
		Modify();
	}
#endif // WITH_EDITOR

	bKillCache = true;	
	ClearCurveCache();
	ALandscapeBlueprintBrushBase::RequestLandscapeUpdate();
}

void AWaterBrushManager::SingleJumpStep()
{
	if (!::IsValid(DebugDistanceFieldMID))
	{
		UE_LOGF(LogWaterEditor, Error, "DebugDistanceFieldMaterial must be set to use this debug function");
		return;
	}

	UTextureRenderTarget2D* RenderTarget = JumpFloodComponent2D->SingleJumpStep();
	DebugDistanceFieldMID->UMaterialInstanceDynamic::SetTextureParameterValue(FName(TEXT("RT")), RenderTarget);
}

void AWaterBrushManager::SingleBlurStep()
{
	if (!::IsValid(DebugDistanceFieldMID))
	{
		UE_LOGF(LogWaterEditor, Error, "DebugDistanceFieldMaterial must be set to use this debug function");
		return;
	}

	UTextureRenderTarget2D* RenderTarget = JumpFloodComponent2D->SingleBlurStep();
	DebugDistanceFieldMID->UMaterialInstanceDynamic::SetTextureParameterValue(FName(TEXT("RT")), RenderTarget);
}

void AWaterBrushManager::FindEdges()
{
	if (!::IsValid(DebugDistanceFieldMID))
	{
		UE_LOGF(LogWaterEditor, Error, "DebugDistanceFieldMaterial must be set to use this debug function");
		return;
	}

	UTextureRenderTarget2D* RenderTarget = JumpFloodComponent2D->FindEdges(DepthAndShapeRT, 50000.0f, FLinearColor(0.0f, 0.0f, 0.0f, 1.0), false, 99.0f);
	DebugDistanceFieldMID->UMaterialInstanceDynamic::SetTextureParameterValue(FName(TEXT("RT")), RenderTarget);
}

void AWaterBrushManager::BlueprintWaterBodyChanged_Native(AActor* Actor)
{
	if (::IsValid(Actor))
	{
		UWaterBodyBrushCacheContainer* ContainerObject;
		FWaterBodyBrushCache WaterBrushCache;
		GetWaterCacheKey(Actor, /*out*/ ContainerObject, /*out*/ WaterBrushCache);

		if (::IsValid(ContainerObject))
		{
			ContainerObject->Cache.CacheIsValid = false;
		}
	}
}

void AWaterBrushManager::Initialize_Native(FTransform const& InLandscapeTransform, FIntPoint const& InLandscapeSize, FIntPoint const& InLandscapeRenderTargetSize)
{
	UE_LOGF(LogWaterEditor, Verbose, "Updated Landscape Transform");

	bool bNeedsFullUpdate = false;
	if (LandscapeQuads != InLandscapeSize)
	{
		LandscapeQuads = InLandscapeSize;
		bNeedsFullUpdate = true;
	}
	if (LandscapeRTRes != InLandscapeRenderTargetSize)
	{
		LandscapeRTRes = InLandscapeRenderTargetSize;
		bNeedsFullUpdate = true;
	}
	if (!InLandscapeTransform.Equals(LandscapeTransform))
	{
		LandscapeTransform = InLandscapeTransform;
		bNeedsFullUpdate = true;
	}

	if (bNeedsFullUpdate)
	{
		check(SceneCaptureComponent2D != nullptr);

		FVector Scale = LandscapeTransform.GetScale3D();
		WorldSize.Set(Scale.X * (float)LandscapeQuads.X, Scale.Y * (float)LandscapeQuads.Y, 0.512f);

		const FVector Temp(Scale.X * (float)LandscapeRTRes.X, Scale.Y * (float)LandscapeRTRes.Y, 0.512f);
		SceneCaptureComponent2D->OrthoWidth = FMath::Max(Temp.X, Temp.Y);

		FVector LocationVector(Temp - Scale);
		LocationVector *= 0.5f;
		LocationVector = LandscapeTransform.GetRotation().RotateVector(LocationVector);
		LocationVector += LandscapeTransform.GetLocation();
		LocationVector.Z = 50000.0f;
		SceneCaptureComponent2D->SetWorldLocation(LocationVector);

		// If the transform or resolution changes, the distance fields need to be recomputed entirely : 
		bKillCache = true;
	}
}

void AWaterBrushManager::CaptureMeshDepth(const TArrayView<UStaticMeshComponent*>& MeshComponents)
{
	SceneCaptureComponent2D->ClearShowOnlyComponents();
	SceneCaptureComponent2D->ShowOnlyActors.Empty();

	TArray<bool> PreviousVisibilities;
	TArray<bool> PreviousHiddenInGame;
	PreviousVisibilities.SetNum(MeshComponents.Num());
	PreviousHiddenInGame.SetNum(MeshComponents.Num());

	for (int32 PrimitiveComponentIndex = 0; PrimitiveComponentIndex < MeshComponents.Num(); ++PrimitiveComponentIndex)
	{
		UStaticMeshComponent* PrimitiveComponent = MeshComponents[PrimitiveComponentIndex];
		PreviousVisibilities[PrimitiveComponentIndex] = PrimitiveComponent->GetVisibleFlag();
		PreviousHiddenInGame[PrimitiveComponentIndex] = PrimitiveComponent->bHiddenInGame;

		PrimitiveComponent->SetVisibility(true);
		PrimitiveComponent->SetHiddenInGame(false);

		SceneCaptureComponent2D->ShowOnlyComponent(PrimitiveComponent);
	}

	SceneCaptureComponent2D->CaptureScene();

	for (int32 PrimitiveComponentIndex = 0; PrimitiveComponentIndex < MeshComponents.Num(); ++PrimitiveComponentIndex)
	{
		UStaticMeshComponent* PrimitiveComponent = MeshComponents[PrimitiveComponentIndex];
		PrimitiveComponent->SetVisibility(PreviousVisibilities[PrimitiveComponentIndex]);
		PrimitiveComponent->SetHiddenInGame(PreviousHiddenInGame[PrimitiveComponentIndex]);
	}

	// Avoid keeping references to Captured components
	SceneCaptureComponent2D->ClearShowOnlyComponents();
}

void AWaterBrushManager::GetWaterCacheKey(AActor* WaterBrush, /*out*/ UWaterBodyBrushCacheContainer*& ContainerObject, /*out*/ FWaterBodyBrushCache& Value)
{
	ContainerObject = CastChecked<UWaterBodyBrushCacheContainer>(GetActorCache(WaterBrush, UWaterBodyBrushCacheContainer::StaticClass()), ECastCheckedType::NullAllowed);
	if (IsValid(ContainerObject))
	{
		Value = ContainerObject->Cache;
	}
}

void AWaterBrushManager::CacheBrushDistanceField(const FBrushActorRenderContext& BrushActorRenderContext)
{
	check(::IsValid(BrushActorRenderContext.CacheContainer));
	UKismetRenderingLibrary::ClearRenderTarget2D(this, BrushActorRenderContext.CacheContainer->Cache.CacheRenderTarget, FLinearColor::Black);
	UKismetRenderingLibrary::DrawMaterialToRenderTarget(this, BrushActorRenderContext.CacheContainer->Cache.CacheRenderTarget, DistanceFieldCacheMID);
	BrushActorRenderContext.CacheContainer->Cache.CacheIsValid = true;
}

void AWaterBrushManager::GetRenderDependencies(TSet<UObject*>& OutDependencies)
{
	Super::GetRenderDependencies(OutDependencies);

	// Add all materials as depencies : we don't want to render the brush if one of those materials isn't compiled yet : 
	AddDependencyIfValid(BrushAngleFalloffMaterial, OutDependencies);
	AddDependencyIfValid(BrushWidthFalloffMaterial, OutDependencies);
	AddDependencyIfValid(DistanceFieldCacheMaterial, OutDependencies);
	AddDependencyIfValid(RenderRiverSplineDepthMaterial, OutDependencies);
	AddDependencyIfValid(DebugDistanceFieldMaterial, OutDependencies);
	AddDependencyIfValid(WeightmapMaterial, OutDependencies);
	AddDependencyIfValid(DrawCanvasMaterial, OutDependencies);
	AddDependencyIfValid(CompositeWaterBodyTextureMaterial, OutDependencies);
	AddDependencyIfValid(IslandFalloffMaterial, OutDependencies);
	AddDependencyIfValid(JumpStepMaterial, OutDependencies);
	AddDependencyIfValid(FindEdgesMaterial, OutDependencies);
	AddDependencyIfValid(BlurEdgesMaterial, OutDependencies);
}

bool AWaterBrushManager::SetupRiverSplineRenderMIDs(const FBrushActorRenderContext& BrushActorRenderContext, bool bRestoreMIDs, TArray<UMaterialInterface*>& InOutMIDs)
{
	AWaterBody* WaterBody = BrushActorRenderContext.GetActorAs<AWaterBody>();
	check(WaterBody->GetWaterBodyType() == EWaterBodyType::River);

	const UWaterSplineComponent* SplineComponent = WaterBody->GetWaterSpline();
	const int32 NumSplineMids = SplineComponent->GetNumberOfSplinePoints() - 1;

	TArray<UPrimitiveComponent*> BrushRenderableComponents = WaterBody->GetBrushRenderableComponents();
	TInlineComponentArray<USplineMeshComponent*> SplineMeshComponents;
	SplineMeshComponents.Reserve(BrushRenderableComponents.Num());
	Algo::Transform(BrushRenderableComponents, SplineMeshComponents, [](UPrimitiveComponent* PrimitiveComponent) { return StaticCast<USplineMeshComponent*>(PrimitiveComponent); });

	if (SplineMeshComponents.Num() != NumSplineMids)
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid River spline mesh component count.");
		return false;
	}

	if (NumSplineMids > RiverSplineMIDs.Num())
	{
		RiverSplineMIDs.SetNumZeroed(NumSplineMids);
	}

	if (bRestoreMIDs)
	{
		check(InOutMIDs.Num() == NumSplineMids);
	}
	else
	{
		InOutMIDs.SetNumZeroed(NumSplineMids);
	}

	for (int32 MIDIndex = 0; MIDIndex < NumSplineMids; ++MIDIndex)
	{
		USplineMeshComponent* SplineMeshComponent = SplineMeshComponents[MIDIndex];
		check(SplineMeshComponent != nullptr);
		if (bRestoreMIDs)
		{
			SplineMeshComponent->SetMaterial(0, InOutMIDs[MIDIndex]);
		}
		else
		{
			RiverSplineMIDs[MIDIndex] = FWaterUtils::GetOrCreateTransientMID(RiverSplineMIDs[MIDIndex], TEXT("RiverSplineMID"), RenderRiverSplineDepthMaterial);
			UMaterialInstanceDynamic* TempMID = RiverSplineMIDs[MIDIndex];
			if (TempMID != nullptr)
			{
				const float Offset = BrushActorRenderContext.WaterBrushActor->GetWaterHeightmapSettings().FalloffSettings.ZOffset;
				TempMID->SetScalarParameterValue(FName(TEXT("DepthA")), SplineComponent->GetFloatPropertyAtSplinePoint(MIDIndex, FName(TEXT("Depth"))) + Offset);
				TempMID->SetScalarParameterValue(FName(TEXT("DepthB")), SplineComponent->GetFloatPropertyAtSplinePoint(MIDIndex + 1, FName(TEXT("Depth"))) + Offset);
				TempMID->SetScalarParameterValue(FName(TEXT("VelA")), SplineComponent->GetFloatPropertyAtSplinePoint(MIDIndex, FName(TEXT("WaterVelocityScalar"))));
				TempMID->SetScalarParameterValue(FName(TEXT("VelB")), SplineComponent->GetFloatPropertyAtSplinePoint(MIDIndex + 1, FName(TEXT("WaterVelocityScalar"))));

				// Save the previous material for restoring it further on : 
				InOutMIDs[MIDIndex] = (SplineMeshComponent->GetMaterial(0));
				SplineMeshComponent->SetMaterial(0, TempMID);
			}
			else
			{
				UE_LOGF(LogWaterEditor, Error, "Invalid River spline material for Water Brush.");
				return false;
			}
		}
	}

	return true;
}

void AWaterBrushManager::CaptureRiverDepthAndVelocity(const FBrushActorRenderContext& BrushActorRenderContext)
{
	TArray<UMaterialInterface*> MIDsToRestore;
	if (!SetupRiverSplineRenderMIDs(BrushActorRenderContext, /*bRestoreMIDs  = */false, MIDsToRestore))
	{
		UE_LOGF(LogWaterEditor, Error, "Error in setup River spline render material for Water Brush. Aborting CaptureRiverDepthAndVelocity.");
		return;
	}

	AWaterBody* WaterBody = BrushActorRenderContext.GetActorAs<AWaterBody>();
	const bool Hidden = WaterBody->IsTemporarilyHiddenInEditor();
	WaterBody->SetIsTemporarilyHiddenInEditor(false);

	TArray<UPrimitiveComponent*> BrushRenderableComponents = WaterBody->GetBrushRenderableComponents();
	TArray<UStaticMeshComponent*, TInlineAllocator<32>> SplineMeshComponents;
	SplineMeshComponents.Reserve(BrushRenderableComponents.Num());
	Algo::Transform(BrushRenderableComponents, SplineMeshComponents, [](UPrimitiveComponent* PrimitiveComponent) { return StaticCast<UStaticMeshComponent*>(PrimitiveComponent); });

	SceneCaptureComponent2D->TextureTarget = WaterDepthAndVelocityRT;
	SceneCaptureComponent2D->CaptureSource = ESceneCaptureSource::SCS_SceneColorSceneDepth;
	CaptureMeshDepth(SplineMeshComponents);

	SceneCaptureComponent2D->CaptureSource = ESceneCaptureSource::SCS_SceneDepth;
	SceneCaptureComponent2D->TextureTarget = DepthAndShapeRT;
	CaptureMeshDepth(SplineMeshComponents);

	WaterBody->SetIsTemporarilyHiddenInEditor(Hidden);

	// Cleanup the spline components at the end (we're not supposed to have modified the water actors) :
	SetupRiverSplineRenderMIDs(BrushActorRenderContext, /*bRestoreMIDs  = */true, MIDsToRestore);
}

void AWaterBrushManager::DrawCanvasShape(const FBrushActorRenderContext& BrushActorRenderContext)
{
	TArray<FCanvasUVTri> CanvasUVTris;

	UE_LOGF(LogWaterEditor, Verbose, "Actor used for Spline Canvas Render: %ls", *UKismetSystemLibrary::GetDisplayName(BrushActorRenderContext.WaterBrushActor.GetObject()));

	UWaterSplineComponent* SplineComponent = CastChecked<UWaterSplineComponent>(BrushActorRenderContext.GetActor()->GetComponentByClass(UWaterSplineComponent::StaticClass()));
	ensure(SplineComponent);

	int32 TruncSegments = FMath::TruncToInt(SplineComponent->GetSplineLength() / CanvasSegmentSize);
	UE_LOGF(LogWaterEditor, Verbose, "Spline Segment Canvas Segments: %d", TruncSegments);

	for (int32 ii = 0; ii < TruncSegments; ++ii)
	{
		FVector LocationA = SplineComponent->GetLocationAtDistanceAlongSpline(0.0f, ESplineCoordinateSpace::World);
		FVector LocationB = SplineComponent->GetLocationAtDistanceAlongSpline(float(ii + 1) * CanvasSegmentSize, ESplineCoordinateSpace::World);
		FVector LocationC = SplineComponent->GetLocationAtDistanceAlongSpline(float(ii + 2) * CanvasSegmentSize, ESplineCoordinateSpace::World);

		LocationA = LandscapeTransform.InverseTransformPosition(LocationA) + 0.5f;
		LocationB = LandscapeTransform.InverseTransformPosition(LocationB) + 0.5f;
		LocationC = LandscapeTransform.InverseTransformPosition(LocationC) + 0.5f;

		FCanvasUVTri& CanvasUVTri = CanvasUVTris.AddDefaulted_GetRef();
		CanvasUVTri.V0_Pos = FVector2D(LocationA);
		CanvasUVTri.V0_UV = FVector2D::ZeroVector;
		CanvasUVTri.V0_Color = FLinearColor::Red;
		CanvasUVTri.V1_Pos = FVector2D(LocationC);
		CanvasUVTri.V1_UV = FVector2D::ZeroVector;
		CanvasUVTri.V1_Color = FLinearColor::Green;
		CanvasUVTri.V2_Pos = FVector2D(LocationB);
		CanvasUVTri.V2_UV = FVector2D::ZeroVector;
		CanvasUVTri.V2_Color = FLinearColor::Blue;
	}

	// TODO [jonathan.bard] : missing DynamicPointActor/DynamicPreviewPoint from BP here

	UKismetRenderingLibrary::ClearRenderTarget2D(this, DepthAndShapeRT, FLinearColor::Black);

	UCanvas* Canvas;
	FVector2D CanvasToRenderTargetSize;
	FDrawToRenderTargetContext RenderTargetContext;
	UKismetRenderingLibrary::BeginDrawCanvasToRenderTarget(this, DepthAndShapeRT, /*out*/ Canvas, /*out*/ CanvasToRenderTargetSize, /*out*/ RenderTargetContext);
	check(::IsValid(Canvas));
	check(::IsValid(DrawCanvasMID));

	Canvas->UCanvas::K2_DrawMaterialTriangle(DrawCanvasMID, CanvasUVTris);

	UKismetRenderingLibrary::EndDrawCanvasToRenderTarget(this, RenderTargetContext);
}

void AWaterBrushManager::DrawBrushMaterial(const FBrushRenderContext& BrushRenderContext, const FBrushActorRenderContext& BrushActorRenderContext)
{
	if (BrushRenderContext.bHeightmapRender)
	{
		UKismetRenderingLibrary::ClearRenderTarget2D(this, HeightPingPongWrite(BrushRenderContext), FLinearColor::Black);
		UKismetRenderingLibrary::DrawMaterialToRenderTarget(this, HeightPingPongWrite(BrushRenderContext), BrushActorRenderContext.MID);

		UE_LOGF(LogWaterEditor, Verbose, "Render Target Write Target: %ls", *UKismetSystemLibrary::GetDisplayName(HeightPingPongWrite(BrushRenderContext)));
		UE_LOGF(LogWaterEditor, Verbose, "Brush MID Parent: %ls", *UKismetSystemLibrary::GetDisplayName(BrushActorRenderContext.MID));
	}
	else
	{
		UKismetRenderingLibrary::ClearRenderTarget2D(this, WeightPingPongWrite(BrushRenderContext), FLinearColor::Black);
		UKismetRenderingLibrary::DrawMaterialToRenderTarget(this, WeightPingPongWrite(BrushRenderContext), WeightmapMID);
	}
}

void AWaterBrushManager::UpdateCurveCacheKeys()
{
	for (TWeakInterfacePtr<IWaterBrushActorInterface> BrushActor : GetActorsAffectingLandscape())
	{
		if (BrushActor.IsValid())
		{
			const FWaterCurveSettings& LocalCurveSettings = BrushActor->GetWaterCurveSettings();
			UCurveFloat* ElevationCurveAsset = GetElevationCurveAsset(LocalCurveSettings);
			if (ensure(ElevationCurveAsset != nullptr))
			{
				FWaterBodyBrushCache* WaterBrushCache = BrushCurveRTCache.Find(ElevationCurveAsset);
				if ((WaterBrushCache == nullptr) || (WaterBrushCache->CacheRenderTarget == nullptr))
				{
					UTextureRenderTarget2D* CurveRT = FWaterUtils::GetOrCreateTransientRenderTarget2D(nullptr, TEXT("CurveRT"), FIntPoint(256, 1), ETextureRenderTargetFormat::RTF_R16f);
					BrushCurveRTCache.Add(ElevationCurveAsset, FWaterBodyBrushCache{ CurveRT, false });
					ElevationCurveAsset->OnUpdateCurve.AddUObject(this, &AWaterBrushManager::OnCurveUpdated);
				}
			}
		}
	}
}

void AWaterBrushManager::UpdateCurves()
{
	TArray<typename decltype(BrushCurveRTCache)::KeyType> Keys;
	BrushCurveRTCache.GetKeys(Keys);

	for (const TObjectPtr<UCurveFloat>& CurCurveFloat : Keys)
	{
		if (CurCurveFloat)
		{
			const FWaterBodyBrushCache& CurveCache = *BrushCurveRTCache.Find(CurCurveFloat);

			// Curve cache needs to be refreshed : 
			if (!CurveCache.CacheIsValid && CurveCache.CacheRenderTarget)
			{
				check(CurCurveFloat);
				UE_LOGF(LogWaterEditor, Verbose, "Water Body Curve Cache Invalid : Refreshing Curve RT");
				UKismetRenderingLibrary::ClearRenderTarget2D(this, CurveCache.CacheRenderTarget, FLinearColor::Black);

				check(CurveCache.CacheRenderTarget);

				UCanvas* Canvas = nullptr;
				FVector2D CanvasToRenderTargetSize;
				FDrawToRenderTargetContext RenderTargetContext;
				UKismetRenderingLibrary::BeginDrawCanvasToRenderTarget(this, CurveCache.CacheRenderTarget, /*out*/ Canvas, /*out*/ CanvasToRenderTargetSize, /*out*/ RenderTargetContext);

				// TODO [jonathan.bard] : this is just an upload of texture data, this shouldn't be done this way : 
				check(Canvas);
				const FVector2D ScreenSize(1.0f, 2.0f);
				for (int32 ii = 0; ii < 256; ++ii)
				{
					float LinearValue = CurCurveFloat->GetFloatValue((float)ii / 255.0f);
					const FLinearColor Color(LinearValue, LinearValue, LinearValue);
					Canvas->K2_DrawBox(FVector2D((float)ii, 0.0f), ScreenSize, 1.0f, Color);
				}

				BrushCurveRTCache[CurCurveFloat].CacheIsValid = true;
				UKismetRenderingLibrary::EndDrawCanvasToRenderTarget(this, RenderTargetContext);
			}
		}
	}
}

bool AWaterBrushManager::BrushRenderSetup()
{
	if (!AllocateRTs())
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Render Target for Water Brush. Aborting BrushRenderSetup.");
		return false;
	}

	JumpFloodComponent2D->BlurEdgesMaterial = BlurEdgesMaterial;
	JumpFloodComponent2D->FindEdgesMaterial = FindEdgesMaterial;
	JumpFloodComponent2D->JumpStepMaterial = JumpStepMaterial;

	// TODO [jonathan.bard] make sure that this works : (probably do the MID setup in CreateMIDs and use AWaterUtils::GetOrCreateTransientMID
	if (::IsValid(DebugDistanceFieldMaterial))
	{
		UStaticMeshComponent* StaticMeshComponent = CastChecked<UStaticMeshComponent>(AActor::AddComponent(FName(TEXT("NODE_AddStaticMeshComponent-0")), false, FTransform(FRotator::ZeroRotator, FVector::ZeroVector, WorldSize), this), ECastCheckedType::NullAllowed);
		if (DebugDistanceFieldMaterial->IsA<UMaterialInstanceDynamic>())
		{
			UE_LOGF(LogWaterEditor, Error, "Invalid DebugDistanceFieldMaterial Material : must be either a Material Instance Constant or a Material");
		}
		else
		{
			// Transient MID : no outer, no name : 
			DebugDistanceFieldMID = UMaterialInstanceDynamic::Create(DebugDistanceFieldMaterial, nullptr);
			check((DebugDistanceFieldMID != nullptr) && (DebugDistanceFieldMID->GetMaterial() == DebugDistanceFieldMaterial->GetMaterial()));
			StaticMeshComponent->SetMaterial(0, DebugDistanceFieldMID);

			DebugDistanceFieldMID->SetScalarParameterValue(FName(TEXT("ShowGrid")), (float)ShowGrid);
			DebugDistanceFieldMID->SetScalarParameterValue(FName(TEXT("ShowDistance")), (float)ShowDistance);
			DebugDistanceFieldMID->SetScalarParameterValue(FName(TEXT("ShowGradient")), (float)ShowGradient);
			DebugDistanceFieldMID->SetScalarParameterValue(FName(TEXT("DistanceDivisor")), DistanceDivisor);
		}
	}

	UpdateBrushCacheKeys();
	UpdateCurveCacheKeys();
	UpdateCurves();
	SetMPCParams();

	if (!CreateMIDs())
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid material setup for Water Brush. Aborting BrushRenderSetup.");
		return false;
	}

	// Success at last!
	return true;
}

void AWaterBrushManager::DistanceFieldCaching(const FBrushActorRenderContext& BrushActorRenderContext)
{
	check(::IsValid(BrushActorRenderContext.MID));
	check(::IsValid(DistanceFieldCacheMID));
	check(::IsValid(BrushActorRenderContext.CacheContainer));

	BrushActorRenderContext.MID->SetTextureParameterValue(FName(TEXT("CachedDistanceFieldHeight")), BrushActorRenderContext.CacheContainer->Cache.CacheRenderTarget);
	DistanceFieldCacheMID->SetTextureParameterValue(FName(TEXT("MeshDepth")), DepthAndShapeRT);

	const FWaterBodyHeightmapSettings& HeightmapSettings = BrushActorRenderContext.WaterBrushActor->GetWaterHeightmapSettings();

	float JumpFloodRTValue = LandscapeRTRes.GetMax();
	JumpFloodRTValue = FMath::Log2(JumpFloodRTValue);

	int32 Ceiling = FMath::CeilToInt(JumpFloodRTValue);
	Ceiling += HeightmapSettings.Effects.Blurring.Radius;

	DistanceFieldCacheMID->SetTextureParameterValue(FName(TEXT("JumpFloodRT")), (Ceiling % 2 == 0) ? JumpFloodRTA : JumpFloodRTB);

	DistanceFieldCacheMID->SetScalarParameterValue(FName(TEXT("UseBlur")), HeightmapSettings.Effects.Blurring.bBlurShape ? 1.0f : 0.0f);
	DistanceFieldCacheMID->SetScalarParameterValue(FName(TEXT("Bluroffset")), (float)HeightmapSettings.Effects.Blurring.Radius);

	AWaterBody* WaterBody = BrushActorRenderContext.TryGetActorAs<AWaterBody>();

	bool bDoInvert = (WaterBody != nullptr) && (WaterBody->GetWaterBodyType() == EWaterBodyType::Ocean);
	DistanceFieldCacheMID->SetScalarParameterValue(FName(TEXT("Invert")), bDoInvert ? 1.0f : 0.0f);

	const FWaterBrushEffectCurlNoise& CurlNoise = HeightmapSettings.Effects.CurlNoise;
	FLinearColor NoiseColor(CurlNoise.Curl1Tiling, CurlNoise.Curl1Amount, CurlNoise.Curl2Tiling, CurlNoise.Curl2Amount);
	DistanceFieldCacheMID->SetVectorParameterValue(FName(TEXT("Curl")), NoiseColor);

	DistanceFieldCacheMID->SetScalarParameterValue(FName(TEXT("ZOffset")), HeightmapSettings.FalloffSettings.ZOffset);
	DistanceFieldCacheMID->SetTextureParameterValue(FName(TEXT("ChannelDepth")), WaterDepthAndVelocityRT);

	bool bDoMeshDepth = (WaterBody != nullptr) && (WaterBody->GetWaterBodyType() == EWaterBodyType::River);
	DistanceFieldCacheMID->SetScalarParameterValue(FName(TEXT("UseMeshDepth")), bDoMeshDepth ? 1.0f : 0.0f);
}

void AWaterBrushManager::CurvesSmoothingAndTerracing(const FBrushActorRenderContext& BrushActorRenderContext)
{
	const FWaterBodyHeightmapSettings& HeightmapSettings = BrushActorRenderContext.WaterBrushActor->GetWaterHeightmapSettings();
	const FWaterBrushEffectSmoothBlending& SmoothBlending = HeightmapSettings.Effects.SmoothBlending;
	const FWaterFalloffSettings& FalloffSettings = HeightmapSettings.FalloffSettings;
	const FWaterBrushEffectTerracing& Terracing = HeightmapSettings.Effects.Terracing;
	const FWaterCurveSettings& CurveSettings = BrushActorRenderContext.WaterBrushActor->GetWaterCurveSettings();

	check(::IsValid(BrushActorRenderContext.MID));
	check(::IsValid(DistanceFieldCacheMID));

	const FLinearColor SmoothingParamsColor(FMath::Max(SmoothBlending.InnerSmoothDistance, 0.01f), FMath::Max(SmoothBlending.OuterSmoothDistance, 0.01f), 0.0f, 0.0f);
	BrushActorRenderContext.MID->SetVectorParameterValue(FName(TEXT("SmoothingParams")), SmoothingParamsColor);

	float CurveChannelDepthValue = (CurveSettings.ChannelDepth + FalloffSettings.ZOffset) * (CurveSettings.bUseCurveChannel ? 1.0f : 0.0f);
	BrushActorRenderContext.MID->SetScalarParameterValue(FName(TEXT("CurveChannelDepth")), CurveChannelDepthValue);
	DistanceFieldCacheMID->SetScalarParameterValue(FName(TEXT("CurveChannelDepth")), CurveChannelDepthValue);

	float ChannelEdgeOffsetValue = CurveSettings.ChannelEdgeOffset - (SplineMeshExtension / 2.0f);
	BrushActorRenderContext.MID->SetScalarParameterValue(FName(TEXT("ChannelEdgeOffset")), ChannelEdgeOffsetValue);
	DistanceFieldCacheMID->SetScalarParameterValue(FName(TEXT("ChannelEdgeOffset")), ChannelEdgeOffsetValue);

	BrushActorRenderContext.MID->SetScalarParameterValue(FName(TEXT("CurveRampWidth")), CurveSettings.CurveRampWidth);
	DistanceFieldCacheMID->SetScalarParameterValue(FName(TEXT("CurveRampWidth")), CurveSettings.CurveRampWidth);

	UCurveFloat* ElevationCurveAsset = GetElevationCurveAsset(CurveSettings);
	UTextureRenderTarget2D* CurveRTValue = BrushCurveRTCache.Find(ElevationCurveAsset)->CacheRenderTarget;
	check(CurveRTValue);
	BrushActorRenderContext.MID->SetTextureParameterValue(FName(TEXT("CurveRT")), CurveRTValue);
	DistanceFieldCacheMID->SetTextureParameterValue(FName(TEXT("CurveRT")), CurveRTValue);

	FLinearColor TerraceParamsColor(Terracing.TerraceAlpha, Terracing.TerraceSmoothness, Terracing.TerraceSpacing, Terracing.MaskLength);
	BrushActorRenderContext.MID->SetVectorParameterValue(FName(TEXT("TerraceParams")), TerraceParamsColor);

	BrushActorRenderContext.MID->SetScalarParameterValue(FName(TEXT("TerraceOffset")), Terracing.MaskStartOffset);
}

void AWaterBrushManager::FalloffAndBlendMode(const FBrushActorRenderContext& BrushActorRenderContext)
{
	check(::IsValid(BrushActorRenderContext.MID));
	check(::IsValid(WeightmapMID));

	const FWaterBodyHeightmapSettings& HeightmapSettings = BrushActorRenderContext.WaterBrushActor->GetWaterHeightmapSettings();
	const FWaterFalloffSettings& FalloffSettings = HeightmapSettings.FalloffSettings;

	float FalloffTangentValue = FMath::Clamp(FalloffSettings.FalloffAngle, 1.0f, 89.0f);
	FalloffTangentValue = FMath::Tan(FMath::DegreesToRadians(FalloffTangentValue));

	const float EdgeWidth = FMath::Max(FalloffSettings.FalloffWidth, 0.1f);

	BrushActorRenderContext.MID->SetScalarParameterValue(FName(TEXT("FalloffTangent")), FalloffTangentValue);
	BrushActorRenderContext.MID->SetScalarParameterValue(FName(TEXT("UseMeshDepth")), 1.0f);
	BrushActorRenderContext.MID->SetScalarParameterValue(FName(TEXT("CapShapeInterior")), 1.0f);
	BrushActorRenderContext.MID->SetScalarParameterValue(FName(TEXT("EdgeWidth")), EdgeWidth);

	float EdgeCenterOffsetValue = FalloffSettings.EdgeOffset - (SplineMeshExtension / 2.0f);
	BrushActorRenderContext.MID->SetScalarParameterValue(FName(TEXT("EdgeCenterOffset")), EdgeCenterOffsetValue);

	float ModeValue = 3.0f;
	switch (HeightmapSettings.BlendMode)
	{
	case EWaterBrushBlendType::AlphaBlend:
		ModeValue = 0.0f;
		break;
	case EWaterBrushBlendType::Min:
		ModeValue = 1.0f;
		break;
	case EWaterBrushBlendType::Max:
		ModeValue = 2.0f;
		break;
	default:
		ModeValue = 3.0f;
	}

	WeightmapMID->UMaterialInstanceDynamic::SetScalarParameterValue(FName(TEXT("BlendMode")), ModeValue);
	BrushActorRenderContext.MID->UMaterialInstanceDynamic::SetScalarParameterValue(FName(TEXT("BlendMode")), ModeValue);
}

void AWaterBrushManager::DisplacementSettings(const FBrushActorRenderContext& BrushActorRenderContext)
{
	check(::IsValid(BrushActorRenderContext.MID));

	const FWaterBodyHeightmapSettings& HeightmapSettings = BrushActorRenderContext.WaterBrushActor->GetWaterHeightmapSettings();
	const FWaterBrushEffectDisplacement& Displacement = HeightmapSettings.Effects.Displacement;

	float BrushTexHeightValue = DisableBrushTextureEffects ? 0.0f : Displacement.DisplacementHeight;
	BrushActorRenderContext.MID->SetScalarParameterValue(FName(TEXT("BrushTexHeight")), BrushTexHeightValue);
	BrushActorRenderContext.MID->SetScalarParameterValue(FName(TEXT("T")), Displacement.DisplacementTiling);
	BrushActorRenderContext.MID->SetTextureParameterValue(FName(TEXT("BrushRT")), Displacement.Texture.Get());
	BrushActorRenderContext.MID->SetScalarParameterValue(FName(TEXT("Displacement Midpoint")), Displacement.Midpoint);
	BrushActorRenderContext.MID->SetVectorParameterValue(FName(TEXT("DisplacementChannel")), Displacement.Channel);
}

void AWaterBrushManager::ApplyWeightmapSettings(const FBrushRenderContext& BrushRenderContext, const FBrushActorRenderContext& BrushActorRenderContext, const FWaterBodyWeightmapSettings& WMSettings)
{
	check(::IsValid(WeightmapMID));
	check(::IsValid(BrushActorRenderContext.CacheContainer));

	const float GradientWidth = FMath::Max(WMSettings.FalloffWidth, 0.1f);
	WeightmapMID->SetTextureParameterValue(FName(TEXT("CachedDistanceFieldHeight")), BrushActorRenderContext.CacheContainer->Cache.CacheRenderTarget);
	WeightmapMID->SetScalarParameterValue(FName(TEXT("GradientWidth")), GradientWidth);

	float EdgeOffsetValue = WMSettings.EdgeOffset - (SplineMeshExtension / 2.0f);
	WeightmapMID->SetScalarParameterValue(FName(TEXT("EdgeOffset")), EdgeOffsetValue);

	WeightmapMID->SetTextureParameterValue(FName(TEXT("BrushRT")), WMSettings.ModulationTexture.Get());
	WeightmapMID->SetScalarParameterValue(FName(TEXT("T")), WMSettings.TextureTiling);

	float WeightmapInfluenceValue = WMSettings.TextureInfluence * (float)!DisableBrushTextureEffects;
	WeightmapMID->SetScalarParameterValue(FName(TEXT("WeightmapInfluence")), WeightmapInfluenceValue);

	WeightmapMID->SetScalarParameterValue(FName(TEXT("Displacement Midpoint")), WMSettings.Midpoint);
	WeightmapMID->SetScalarParameterValue(FName(TEXT("Opacity")), WMSettings.FinalOpacity);

	WeightmapMID->SetTextureParameterValue(FName(TEXT("HeightRT")), WeightPingPongRead(BrushRenderContext));
}

void AWaterBrushManager::SetBrushMIDParams(const FBrushRenderContext& BrushRenderContext, FBrushActorRenderContext& BrushActorRenderContext)
{
	const FWaterBodyHeightmapSettings& HeightmapSettings = BrushActorRenderContext.WaterBrushActor->GetWaterHeightmapSettings();
	JumpFloodComponent2D->UseBlur = HeightmapSettings.Effects.Blurring.bBlurShape;
	JumpFloodComponent2D->BlurPasses = HeightmapSettings.Effects.Blurring.Radius;

	if (BrushActorRenderContext.TryGetActorAs<AWaterBodyIsland>() != nullptr)
	{
		BrushActorRenderContext.MID = IslandFalloffMID;
	}
	else
	{
		if (HeightmapSettings.FalloffSettings.FalloffMode == EWaterBrushFalloffMode::Angle)
		{
			BrushActorRenderContext.MID = BrushAngleFalloffMID;
		}
		else
		{
			BrushActorRenderContext.MID = BrushWidthFalloffMID;
		}
	}

	check(::IsValid(BrushActorRenderContext.MID));

	DistanceFieldCaching(BrushActorRenderContext);
	CurvesSmoothingAndTerracing(BrushActorRenderContext);

	if (BrushRenderContext.bHeightmapRender)
	{
		BrushActorRenderContext.MID->UMaterialInstanceDynamic::SetTextureParameterValue(FName(TEXT("HeightRT")), HeightPingPongRead(BrushRenderContext));
		BrushActorRenderContext.MID->UMaterialInstanceDynamic::SetTextureParameterValue(FName(TEXT("CombinedAlphaAndHeightRT")), VelocityPingPongRead(BrushRenderContext));

		FalloffAndBlendMode(BrushActorRenderContext);
		DisplacementSettings(BrushActorRenderContext);
	}
	else
	{
		const FWaterBodyWeightmapSettings* WMSettings = BrushActorRenderContext.WaterBrushActor->GetLayerWeightmapSettings().Find(BrushRenderContext.WeightmapLayerName);
		check(WMSettings != nullptr);
		ApplyWeightmapSettings(BrushRenderContext, BrushActorRenderContext, *WMSettings);
	}
}


void AWaterBrushManager::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName ChangedProperty = PropertyChangedEvent.GetPropertyName();
	if (ChangedProperty == GET_MEMBER_NAME_CHECKED(AWaterBrushManager, RenderRiverSplineDepthMaterial))
	{
		RiverSplineMIDs.Reset();
	}
}

bool AWaterBrushManager::AllocateRTs()
{
	bool bSuccess = true;
	HeightmapRTA = FWaterUtils::GetOrCreateTransientRenderTarget2D(HeightmapRTA, TEXT("HeightmapRTA"), LandscapeRTRes, RTF_RGBA8);
	HeightmapRTB = FWaterUtils::GetOrCreateTransientRenderTarget2D(HeightmapRTB, TEXT("HeightmapRTB"), LandscapeRTRes, RTF_RGBA8);
	if ((HeightmapRTA == nullptr) || (HeightmapRTB == nullptr))
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Heightmap Render Target for Water Brush. Aborting AllocateRTs.");
		bSuccess = false;
	}

	JumpFloodRTA = FWaterUtils::GetOrCreateTransientRenderTarget2D(JumpFloodRTA, TEXT("JumpFloodRTA"), LandscapeRTRes, RTF_RGBA32f);
	JumpFloodRTB = FWaterUtils::GetOrCreateTransientRenderTarget2D(JumpFloodRTB, TEXT("JumpFloodRTB"), LandscapeRTRes, RTF_RGBA32f);
	if ((JumpFloodRTA != nullptr) && (JumpFloodRTB != nullptr))
	{
		JumpFloodRTA->AddressX = TextureAddress::TA_Clamp;
		JumpFloodRTA->AddressY = TextureAddress::TA_Clamp;

		JumpFloodRTB->AddressX = TextureAddress::TA_Clamp;
		JumpFloodRTB->AddressY = TextureAddress::TA_Clamp;

		JumpFloodComponent2D->AssignRenderTargets(JumpFloodRTA, JumpFloodRTB);
	}
	else
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid JumpFlood Render Target for Water Brush. Aborting AllocateRTs.");
		bSuccess = false;
	}

	DepthAndShapeRT = FWaterUtils::GetOrCreateTransientRenderTarget2D(DepthAndShapeRT, TEXT("DepthAndShapeRT"), LandscapeRTRes, RTF_RG32f);
	if (DepthAndShapeRT != nullptr)
	{
		SceneCaptureComponent2D->TextureTarget = DepthAndShapeRT;
	}
	else
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid DepthAndShape Render Target for Water Brush. Aborting AllocateRTs.");
		bSuccess = false;
	}

	WaterDepthAndVelocityRT = FWaterUtils::GetOrCreateTransientRenderTarget2D(WaterDepthAndVelocityRT, TEXT("WaterDepthAndVelocityRT"), LandscapeRTRes, RTF_RGBA32f);
	if (WaterDepthAndVelocityRT == nullptr)
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid WaterDepthAndVelocity Render Target for Water Brush. Aborting AllocateRTs.");
		bSuccess = false;
	}

	WeightmapRTA = FWaterUtils::GetOrCreateTransientRenderTarget2D(WeightmapRTA, TEXT("WeightmapRTA"), LandscapeRTRes, RTF_R8);
	WeightmapRTB = FWaterUtils::GetOrCreateTransientRenderTarget2D(WeightmapRTB, TEXT("WeightmapRTB"), LandscapeRTRes, RTF_R8);
	if ((WeightmapRTA == nullptr) || (WeightmapRTB == nullptr))
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid Weightmap Render Target for Water Brush. Aborting AllocateRTs.");
		bSuccess = false;
	}

	CombinedVelocityAndHeightRTA = FWaterUtils::GetOrCreateTransientRenderTarget2D(CombinedVelocityAndHeightRTA, TEXT("CombinedVelocityAndHeightRTA"), LandscapeRTRes, RTF_RGBA16f);
	CombinedVelocityAndHeightRTB = FWaterUtils::GetOrCreateTransientRenderTarget2D(CombinedVelocityAndHeightRTB, TEXT("CombinedVelocityAndHeightRTB"), LandscapeRTRes, RTF_RGBA16f);
	if ((CombinedVelocityAndHeightRTA == nullptr) || (CombinedVelocityAndHeightRTB == nullptr))
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid CombinedVelocityAndHeight Render Target for Water Brush. Aborting AllocateRTs.");
		bSuccess = false;
	}

	return bSuccess;
}

void AWaterBrushManager::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);
	SetMPCParams();
}

void AWaterBrushManager::BeginPlay()
{
	Super::BeginPlay();
	SetMPCParams();
}

void AWaterBrushManager::ComputeWaterLandscapeInfo(FVector& OutRTWorldLocation, FVector& OutRTWorldSizeVector) const
{
	FVector LandscapeScale = LandscapeTransform.GetScale3D();
	OutRTWorldSizeVector = FVector(LandscapeRTRes) * LandscapeScale;
	OutRTWorldSizeVector.Z = 1.0f;
	OutRTWorldLocation = LandscapeTransform.GetLocation();
	OutRTWorldLocation -= FVector(LandscapeScale.X, LandscapeScale.Y, 0.0f) * 0.5f;
}

bool AWaterBrushManager::DeprecateWaterLandscapeInfo(FVector& OutRTWorldLocation, FVector& OutRTWorldSizeVector)
{
#if WITH_EDITOR
	if (ALandscape* Landscape = GetOwningLandscape())
	{
		FIntPoint LandscapeSize;
		if (Landscape->ComputeLandscapeLayerBrushInfo(LandscapeTransform, LandscapeSize, LandscapeRTRes))
		{
			ComputeWaterLandscapeInfo(OutRTWorldLocation, OutRTWorldSizeVector);
			return true;
		}
	}

	return false;
#endif // WITH_EDITOR
}

#if WITH_EDITOR

void AWaterBrushManager::ShowForceUpdateMapCheckError()
{
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("WaterBrush"), FText::FromString(GetActorNameOrLabel()));
	Arguments.Add(TEXT("Outer"), FText::FromString(GetPackage()->GetPathName()));
	
	FMessageLog("MapCheck").Warning()
		->AddToken(FUObjectToken::Create(this))
		->AddToken(FTextToken::Create(FText::Format(LOCTEXT("AWaterBrushManager_ShowForceUpdateMapCheckError_Message_ForceUpdateNeeded", "Water brush {WaterBrush} in package {Outer} is out of date and needs updating for water to render properly."), Arguments)))
		->AddToken(FMapErrorToken::Create(TEXT("AWaterBrushManager_ShowForceUpdateMapCheckError_MapError_ForceUpdateNeeded")))
		->AddToken(FActionToken::Create(LOCTEXT("AWaterBrushManager_ShowForceUpdateMapCheckError_ActionName_ForceUpdateNeeded", "Update water brush"), FText(),
			FOnActionTokenExecuted::CreateUObject(this, &AWaterBrushManager::ForceUpdate), true));
}

#endif // WITH_EDITOR

void AWaterBrushManager::SetMPCParams()
{
	UWorld* World = GetWorld();
	UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(World);
	if ((World != nullptr) && (WaterSubsystem != nullptr))
	{
		FVector RTWorldLocation, RTWorldSizeVector;
		ComputeWaterLandscapeInfo(RTWorldLocation, RTWorldSizeVector);

		if (GEditor == nullptr)
		{
			UE_LOGF(LogWaterEditor, Error, "GEditor is null for AWaterBrushManager::SetMPCParams(): %ls", *GetFullName());
			return;
		}

		UWaterEditorSubsystem* WaterEditorSubsystem = GEditor->GetEditorSubsystem<UWaterEditorSubsystem>();
		check(WaterEditorSubsystem);
		UMaterialParameterCollection* LandscapeCollection =  WaterEditorSubsystem->GetLandscapeMaterialParameterCollection();
		if (LandscapeCollection == nullptr)
        {
            UE_LOGF(LogWaterEditor, Error, "No Landscape MaterialParameterCollection Assigned");
			return;
        }
        
		UMaterialParameterCollectionInstance* LandscapeCollectionInstance = World->GetParameterCollectionInstance(CastChecked<UMaterialParameterCollection>(LandscapeCollection));
		check(LandscapeCollectionInstance != nullptr);

		if (!LandscapeCollectionInstance->SetScalarParameterValue(FName(TEXT("RTResX")), (float)LandscapeRTRes.X))
		{
			UE_LOGF(LogWaterEditor, Error, "Failed to set \"RTResX\" on Landscape MaterialParameterCollection");
		}

		if (!LandscapeCollectionInstance->SetScalarParameterValue(FName(TEXT("RTResY")), (float)LandscapeRTRes.Y))
		{
			UE_LOGF(LogWaterEditor, Error, "Failed to set \"RTResY\" on Landscape MaterialParameterCollection");
		}

		if (!LandscapeCollectionInstance->SetScalarParameterValue(FName(TEXT("LSQuadsX")), (float)LandscapeQuads.X))
		{
			UE_LOGF(LogWaterEditor, Error, "Failed to set \"LSQuadsX\" on Landscape MaterialParameterCollection");
		}
		if (!LandscapeCollectionInstance->SetScalarParameterValue(FName(TEXT("LSQuadsY")), (float)LandscapeQuads.Y))
		{
			UE_LOGF(LogWaterEditor, Error, "Failed to set \"LSQuadsY\" on Landscape MaterialParameterCollection");
		}

		if (!LandscapeCollectionInstance->SetScalarParameterValue(FName(TEXT("WorldSizeX")), WorldSize.X))
		{
			UE_LOGF(LogWaterEditor, Error, "Failed to set \"WorldSizeX\" on Landscape MaterialParameterCollection");
		}
		if (!LandscapeCollectionInstance->SetScalarParameterValue(FName(TEXT("WorldSizeY")), WorldSize.Y))
		{
			UE_LOGF(LogWaterEditor, Error, "Failed to set \"WorldSizeY\" on Landscape MaterialParameterCollection");
		}

		if (!LandscapeCollectionInstance->SetVectorParameterValue(FName(TEXT("LandscapeLocation")), FLinearColor(LandscapeTransform.GetLocation())))
		{
			UE_LOGF(LogWaterEditor, Error, "Failed to set \"LandscapeLocation\" on Landscape MaterialParameterCollection");
		}
		if (!LandscapeCollectionInstance->SetScalarParameterValue(FName(TEXT("LandscapeZLocation")), LandscapeTransform.GetLocation().Z))
		{
			UE_LOGF(LogWaterEditor, Error, "Failed to set \"LandscapeZLocation\" on Landscape MaterialParameterCollection");
		}
		// TODO [jonathan.bard] : find out what this 128.0f corresponds to and put in a constant : ZSCALE in LandscapeLayersPS.usf maybe ??
		if (!LandscapeCollectionInstance->SetScalarParameterValue(FName(TEXT("LandscapeZScale")), LandscapeTransform.GetScale3D().Z / 128.0f))
		{
			UE_LOGF(LogWaterEditor, Error, "Failed to set \"LandscapeZScale\" on Landscape MaterialParameterCollection");
		}
		if (!LandscapeCollectionInstance->SetVectorParameterValue(FName(TEXT("RTWorldSize")), FLinearColor(RTWorldSizeVector)))
		{
			UE_LOGF(LogWaterEditor, Error, "Failed to set \"RTWorldSize\" on Landscape MaterialParameterCollection");
		}
		if (!LandscapeCollectionInstance->SetVectorParameterValue(FName(TEXT("RTWorldLocation")), FLinearColor(RTWorldLocation)))
		{
			UE_LOGF(LogWaterEditor, Error, "Failed to set \"RTWorldLocation\" on Landscape MaterialParameterCollection");
		}
		if (!LandscapeCollectionInstance->SetScalarParameterValue(FName(TEXT("WaterClearHeight")), WaterClearHeight))
		{
			UE_LOGF(LogWaterEditor, Error, "Failed to set \"WaterClearHeight\" on Landscape MaterialParameterCollection");
		}
		
	}
}

void AWaterBrushManager::ApplyToCompositeWaterBodyTexture(FBrushRenderContext& BrushRenderContext, const FBrushActorRenderContext& BrushActorRenderContext)
{
	AWaterBody* WaterBody = BrushActorRenderContext.TryGetActorAs<AWaterBody>();
	if (BrushRenderContext.bHeightmapRender && (WaterBody != nullptr))
	{
		check(::IsValid(BrushActorRenderContext.CacheContainer));
		const FWaterBodyHeightmapSettings& HeightmapSettings = BrushActorRenderContext.WaterBrushActor->GetWaterHeightmapSettings();

		CompositeWaterBodyTextureMID->SetTextureParameterValue(FName(TEXT("CachedDistanceFieldHeight")), BrushActorRenderContext.CacheContainer->Cache.CacheRenderTarget);
		CompositeWaterBodyTextureMID->SetTextureParameterValue(FName(TEXT("CombinedVelocityAndHeight")), VelocityPingPongRead(BrushRenderContext));
		CompositeWaterBodyTextureMID->SetTextureParameterValue(FName(TEXT("LandscapeHeight")), HeightPingPongRead(BrushRenderContext));
		CompositeWaterBodyTextureMID->SetScalarParameterValue(FName(TEXT("ZOffset")), HeightmapSettings.FalloffSettings.ZOffset);
		CompositeWaterBodyTextureMID->SetScalarParameterValue(FName(TEXT("Shape Dilation")), WaterBody->GetWaterBodyComponent()->ShapeDilation);

		UE_LOGF(LogWaterEditor, Verbose, "Rendering Water Body Velocity/Height to Combined Texture: %ls", *UKismetSystemLibrary::GetDisplayName(VelocityPingPongWrite(BrushRenderContext)));

		UKismetRenderingLibrary::ClearRenderTarget2D(this, VelocityPingPongWrite(BrushRenderContext), FLinearColor::Black);
		UKismetRenderingLibrary::DrawMaterialToRenderTarget(this, VelocityPingPongWrite(BrushRenderContext), CompositeWaterBodyTextureMID);

		++BrushRenderContext.VelocityRTIndex;
	}
}

void AWaterBrushManager::RenderBrushActorContext(FBrushRenderContext& BrushRenderContext, FBrushActorRenderContext& BrushActorRenderContext)
{
	if (!BrushRenderContext.bHeightmapRender)
	{
		if (!(BrushActorRenderContext.WaterBrushActor->GetLayerWeightmapSettings().Find(BrushRenderContext.WeightmapLayerName)))
		{
			UE_LOGF(LogWaterEditor, Verbose, "Actor does NOT affect this layer, Skipping");
			return;
		}
	}

	UWaterBodyBrushCacheContainer* CacheContainer = nullptr;
	FWaterBodyBrushCache WaterBrushCache;
	GetWaterCacheKey(BrushActorRenderContext.GetActor(), /*out*/ CacheContainer, /*out*/ WaterBrushCache);
	BrushActorRenderContext.CacheContainer = CacheContainer;

	SetBrushMIDParams(BrushRenderContext, BrushActorRenderContext);

	UE_LOGF(LogWaterEditor, Verbose, "====================================");
	UE_LOGF(LogWaterEditor, Verbose, "Current Actor: %ls", *UKismetSystemLibrary::GetDisplayName(BrushActorRenderContext.WaterBrushActor.GetObject()));
	UE_LOGF(LogWaterEditor, Verbose, "Type: %ls", *BrushActorRenderContext.WaterBrushActor.GetObject()->GetClass()->GetName());
	UE_LOGF(LogWaterEditor, Verbose, "Cache is Valid: %ls", (BrushActorRenderContext.CacheContainer->Cache.CacheIsValid ? TEXT("true") : TEXT("false")));

	if (BrushActorRenderContext.CacheContainer->Cache.CacheIsValid)
	{
		if (bKillCache)
		{
			UE_LOGF(LogWaterEditor, Verbose, "Kill Cache Detected, running full render pass for Brush");
		}
	}
	else
	{
		UseDynamicPreviewRT = true;
	}

	AWaterBody* WaterBody = BrushActorRenderContext.TryGetActorAs<AWaterBody>();
	if (!BrushActorRenderContext.CacheContainer->Cache.CacheIsValid || bKillCache)
	{
		const FWaterBodyHeightmapSettings& HeightmapSettings = BrushActorRenderContext.WaterBrushActor->GetWaterHeightmapSettings();
		const FWaterBrushEffectCurlNoise& CurlNoise = HeightmapSettings.Effects.CurlNoise;
		FLinearColor CurlColor(CurlNoise.Curl1Tiling, CurlNoise.Curl1Amount, CurlNoise.Curl2Tiling, CurlNoise.Curl2Amount);
		if ((WaterBody != nullptr) && (WaterBody->GetWaterBodyType() == EWaterBodyType::River))
		{
			UE_LOGF(LogWaterEditor, Verbose, "River depth and verlocity render");
			CaptureRiverDepthAndVelocity(BrushActorRenderContext);
			UE_LOGF(LogWaterEditor, Verbose, "Jump flood");
			JumpFloodComponent2D->JumpFlood(DepthAndShapeRT, 50000.0f, CurlColor, true, 0.0f);
		}
		else
		{
			UE_LOGF(LogWaterEditor, Verbose, "Canvas shape render");
			DrawCanvasShape(BrushActorRenderContext);
			UE_LOGF(LogWaterEditor, Verbose, "Jump flood");
			JumpFloodComponent2D->JumpFlood(DepthAndShapeRT, 50000.0f, CurlColor, false, BrushActorRenderContext.GetActor()->GetActorLocation().Z);
		}

		UE_LOGF(LogWaterEditor, Verbose, "Distance Field generation");
		CacheBrushDistanceField(BrushActorRenderContext);
	}

	DrawBrushMaterial(BrushRenderContext, BrushActorRenderContext);
	// TODO [jonathan.bard] : this part is weird in the BP : 
	if (BrushRenderContext.bHeightmapRender)
	{
		// TODO [jonathan.bard] : along with MakePreviewRT?
	//++HeightOnlyIndex;
	}
	++BrushRenderContext.RTIndex;

	ApplyToCompositeWaterBodyTexture(BrushRenderContext, BrushActorRenderContext);
}

bool AWaterBrushManager::CreateMIDs()
{
	BrushAngleFalloffMID = FWaterUtils::GetOrCreateTransientMID(BrushAngleFalloffMID, TEXT("BrushAngleFalloffMID"), BrushAngleFalloffMaterial);
	IslandFalloffMID = FWaterUtils::GetOrCreateTransientMID(IslandFalloffMID, TEXT("IslandFalloffMID"), IslandFalloffMaterial);
	BrushWidthFalloffMID = FWaterUtils::GetOrCreateTransientMID(BrushWidthFalloffMID, TEXT("BrushWidthFalloffMID"), BrushWidthFalloffMaterial);
	WeightmapMID = FWaterUtils::GetOrCreateTransientMID(WeightmapMID, TEXT("WeightmapMID"), WeightmapMaterial);
	DistanceFieldCacheMID = FWaterUtils::GetOrCreateTransientMID(DistanceFieldCacheMID, TEXT("DistanceFieldCacheMID"), DistanceFieldCacheMaterial);
	CompositeWaterBodyTextureMID = FWaterUtils::GetOrCreateTransientMID(CompositeWaterBodyTextureMID, TEXT("CompositeWaterBodyTextureMID"), CompositeWaterBodyTextureMaterial);
	DrawCanvasMID = FWaterUtils::GetOrCreateTransientMID(DrawCanvasMID, TEXT("DrawCanvasMID"), DrawCanvasMaterial);

	if ((BrushAngleFalloffMID == nullptr)
		|| (IslandFalloffMID == nullptr)
		|| (BrushWidthFalloffMID == nullptr)
		|| (WeightmapMID == nullptr)
		|| (DistanceFieldCacheMID == nullptr)
		|| (CompositeWaterBodyTextureMID == nullptr)
		|| (DrawCanvasMID == nullptr))
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid water brush materials.");
		return false;
	}

	return JumpFloodComponent2D->CreateMIDs();
}

void AWaterBrushManager::SortWaterBodiesForBrushRender_Implementation(TArray<AWaterBody*>& InOutWaterBodies) const
{
	InOutWaterBodies.Sort([](const AWaterBody& LHS, const AWaterBody& RHS)
	{
		// Render the Ocean first, then the others, so as not to stomp the latter with the former
		static constexpr int32 WaterBrushPriorityByType[] =
		{
			2, // EWaterBodyType::River
			1, // EWaterBodyType::Lake
			0, // EWaterBodyType::Ocean
			3, // EWaterBodyType::Transition
		};
		static_assert(sizeof(WaterBrushPriorityByType) / sizeof(WaterBrushPriorityByType[0]) == (size_t)EWaterBodyType::Num, "Invalid priority count");

		int32 LHSPriority = WaterBrushPriorityByType[(int32)LHS.GetWaterBodyType()];
		int32 RHSPriority = WaterBrushPriorityByType[(int32)RHS.GetWaterBodyType()];
		if (LHSPriority == RHSPriority)
		{
			// Return a deterministic order based on their full names : 
			return LHS.GetFullName() < RHS.GetFullName();
		}
		return LHSPriority < RHSPriority;
	});
}

void AWaterBrushManager::SetupDefaultMaterials()
{
	const UWaterEditorSettings* WaterEditorSettings = GetDefault<UWaterEditorSettings>();
	check(WaterEditorSettings != nullptr);

	BrushAngleFalloffMaterial = WaterEditorSettings->GetDefaultBrushAngleFalloffMaterial();
	BrushWidthFalloffMaterial = WaterEditorSettings->GetDefaultBrushWidthFalloffMaterial();
	DistanceFieldCacheMaterial = WaterEditorSettings->GetDefaultCacheDistanceFieldCacheMaterial();
	RenderRiverSplineDepthMaterial = WaterEditorSettings->GetDefaultRenderRiverSplineDepthsMaterial();
	WeightmapMaterial = WaterEditorSettings->GetDefaultBrushWeightmapMaterial();
	DrawCanvasMaterial = WaterEditorSettings->GetDefaultDrawCanvasMaterial();
	CompositeWaterBodyTextureMaterial = WaterEditorSettings->GetDefaultCompositeWaterBodyTextureMaterial();
	IslandFalloffMaterial = WaterEditorSettings->GetDefaultBrushIslandFalloffMaterial();

	JumpStepMaterial = WaterEditorSettings->GetDefaultJumpFloodStepMaterial();
	BlurEdgesMaterial = WaterEditorSettings->GetDefaultBlurEdgesMaterial();
	FindEdgesMaterial = WaterEditorSettings->GetDefaultFindEdgesMaterial();
}

UTextureRenderTarget2D* AWaterBrushManager::RenderLayer_Native(const FLandscapeBrushParameters& InParameters)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AWaterBrushManager::RenderLayer_Native);

	LandscapeRTRef = InParameters.CombinedResult;

	FBrushRenderContext BrushRenderContext;
	BrushRenderContext.bHeightmapRender = InParameters.LayerType == ELandscapeToolTargetType::Heightmap;
	BrushRenderContext.WeightmapLayerName = InParameters.WeightmapLayerName;

	if (!BrushRenderSetup())
	{
		UE_LOGF(LogWaterEditor, Error, "Invalid setup for water brush. Aborting Render.");
		return nullptr;
	}

	if (BrushRenderContext.bHeightmapRender)
	{
		const FLinearColor ClearColor(0.000000, 0.000000, WaterClearHeight, 1.000000);
		UKismetRenderingLibrary::ClearRenderTarget2D(this, CombinedVelocityAndHeightRTA, ClearColor);
		UKismetRenderingLibrary::ClearRenderTarget2D(this, CombinedVelocityAndHeightRTB, ClearColor);
	}

	UE_LOGF(LogWaterEditor, Verbose, "====================================");

	if (BrushRenderContext.bHeightmapRender)
	{
		UE_LOGF(LogWaterEditor, Verbose, "BrushManager: Heightmap Render Pass");
	}
	else
	{
		UE_LOGF(LogWaterEditor, Verbose, "Brush Manager Render: Weightmap Render Pass: %ls", *BrushRenderContext.WeightmapLayerName.ToString());
	}

	TArray<AWaterBody*> WaterBodies;
	AWaterLandscapeBrush::GetWaterBodies(AWaterBody::StaticClass(), WaterBodies);

	SortWaterBodiesForBrushRender(WaterBodies);

	for (AWaterBody* WaterBody : WaterBodies)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RenderWaterBody);
		FBrushActorRenderContext BrushActorRenderContext(WaterBody);
		RenderBrushActorContext(BrushRenderContext, BrushActorRenderContext);
	}

	// Inner Loop for Islands
	TArray<AWaterBodyIsland*> WaterbodyIslands;
	GetWaterBodyIslands(AWaterBodyIsland::StaticClass(), WaterbodyIslands);

	for (AWaterBodyIsland* WaterBodyIsland : WaterbodyIslands)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RenderWaterIsland);
		FBrushActorRenderContext BrushActorRenderContext(WaterBodyIsland);
		RenderBrushActorContext(BrushRenderContext, BrushActorRenderContext);
	}

	// Render Completed
	UTextureRenderTarget2D* ReturnRT = (InParameters.LayerType == ELandscapeToolTargetType::Heightmap) ? HeightPingPongRead(BrushRenderContext) : WeightPingPongRead(BrushRenderContext);

	bKillCache = false;

	bNeedsForceUpdate = false;

	return ReturnRT;
}

void AWaterBrushManager::UpdateBrushCacheKeys()
{
	for (TWeakInterfacePtr<IWaterBrushActorInterface> BrushActor : GetActorsAffectingLandscape())
	{
		if (BrushActor.IsValid())
		{
			ETextureRenderTargetFormat Format = BrushActor->GetBrushRenderTargetFormat();

			AActor* Actor = CastChecked<AActor>(BrushActor.GetObject());
			UWaterBodyBrushCacheContainer* WaterBrushCacheContainer = Cast<UWaterBodyBrushCacheContainer>(GetActorCache(Actor, UWaterBodyBrushCacheContainer::StaticClass()));
			// If no cache entry or the cache entry is invalid, create one and associate it to the actor : 
			if (!::IsValid(WaterBrushCacheContainer))
			{
				WaterBrushCacheContainer = NewObject<UWaterBodyBrushCacheContainer>(this, NAME_None, RF_Transactional);
				check(!WaterBrushCacheContainer->Cache.CacheIsValid);
				SetActorCache(Actor, WaterBrushCacheContainer);
			}
			// Make sure there's an appropriate render target in that cache : 
			WaterBrushCacheContainer->Cache.CacheRenderTarget = FWaterUtils::GetOrCreateTransientRenderTarget2D(WaterBrushCacheContainer->Cache.CacheRenderTarget, FName(*FString::Printf(TEXT("BrushCacheRT_%s"), *Actor->GetActorNameOrLabel())), LandscapeRTRes, Format);
			check(WaterBrushCacheContainer->Cache.CacheRenderTarget != nullptr);
		}
	}
}


#if WITH_EDITOR

void AWaterBrushManager::CheckForErrors()
{
	Super::CheckForErrors();

	// If a force update action was requested but hasn't been performed yet, display the message again : 
	if (bNeedsForceUpdate)
	{
		ShowForceUpdateMapCheckError();
	}
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

