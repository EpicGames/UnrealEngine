// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakInterfacePtr.h"
#include "WaterLandscapeBrush.h"
#include "WaterBrushCacheContainer.h"
#include "WaterBrushManager.generated.h"

class UMaterialInterface;


class UTextureRenderTarget2D;
class USceneCaptureComponent2D;
class UStaticMeshComponent;
class UJumpFloodComponent2D;
class UMaterialInstanceDynamic;
class UTexture2D;
class IWaterBrushActorInterface;
class UCanvas;
class AWaterBodyIsland;
class UCurveFloat;
class UCurveBase;
class UMaterialParameterCollection;


UCLASS(config = Engine, Blueprintable, BlueprintType)
class AWaterBrushManager : public AWaterLandscapeBrush
{
public:
	GENERATED_BODY()
	
 	UPROPERTY()
	TObjectPtr<USceneCaptureComponent2D> SceneCaptureComponent2D = nullptr;

	UPROPERTY()
	TObjectPtr<UJumpFloodComponent2D> JumpFloodComponent2D = nullptr;

	// RTs
	UPROPERTY(VisibleAnywhere, Transient, meta = (Category = "Render Targets"))
	TObjectPtr<UTextureRenderTarget2D> HeightmapRTA = nullptr;

	UPROPERTY(VisibleAnywhere, Transient, meta=(Category="Render Targets"))
	TObjectPtr<UTextureRenderTarget2D> HeightmapRTB = nullptr;

	UPROPERTY(VisibleAnywhere, Transient, meta=(Category="Render Targets"))
	TObjectPtr<UTextureRenderTarget2D> JumpFloodRTA;

	UPROPERTY(VisibleAnywhere, Transient, meta=(Category="Render Targets"))
	TObjectPtr<UTextureRenderTarget2D> JumpFloodRTB = nullptr;

	UPROPERTY(VisibleAnywhere, Transient, meta=(Category="Render Targets"))
	TObjectPtr<UTextureRenderTarget2D> DepthAndShapeRT = nullptr;

	UPROPERTY(VisibleAnywhere, Transient, meta=(Category="Render Targets"))
	TObjectPtr<UTextureRenderTarget2D> WaterDepthAndVelocityRT = nullptr;

	UPROPERTY(VisibleAnywhere, Transient, meta=(Category="Render Targets"))
	TObjectPtr<UTextureRenderTarget2D> CombinedVelocityAndHeightRTA = nullptr;

	UPROPERTY(VisibleAnywhere, Transient, meta=(Category="Render Targets"))
	TObjectPtr<UTextureRenderTarget2D> CombinedVelocityAndHeightRTB = nullptr;

	UPROPERTY(VisibleAnywhere, Transient, meta=(Category="Render Targets"))
	TObjectPtr<UTextureRenderTarget2D> LandscapeRTRef = nullptr;

	UPROPERTY(VisibleAnywhere, Transient, meta=(Category="Render Targets"))
	TObjectPtr<UTextureRenderTarget2D> WeightmapRTA = nullptr;
	
	UPROPERTY(VisibleAnywhere, Transient, meta=(Category="Render Targets"))
	TObjectPtr<UTextureRenderTarget2D> WeightmapRTB = nullptr;
	// RTs End

	// Brush materials
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Brush Materials"))
	TObjectPtr<UMaterialInterface> BrushAngleFalloffMaterial = nullptr;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Brush Materials"))
	TObjectPtr<UMaterialInterface> BrushWidthFalloffMaterial = nullptr;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Brush Materials"))
	TObjectPtr<UMaterialInterface> DistanceFieldCacheMaterial = nullptr;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Brush Materials"))
	TObjectPtr<UMaterialInterface> RenderRiverSplineDepthMaterial = nullptr;

	// TODO [jonathan.bard] : rename to DebugDistanceFieldMaterial and make it work : 
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Brush Materials"))
	TObjectPtr<UMaterialInterface> DebugDistanceFieldMaterial = nullptr;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Brush Materials"))
	TObjectPtr<UMaterialInterface> WeightmapMaterial = nullptr;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Brush Materials"))
	TObjectPtr<UMaterialInterface> DrawCanvasMaterial = nullptr;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Brush Materials"))
	TObjectPtr<UMaterialInterface> CompositeWaterBodyTextureMaterial = nullptr;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Brush Materials"))
	TObjectPtr<UMaterialInterface> IslandFalloffMaterial = nullptr;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Brush Materials"))
	TObjectPtr<UMaterialInterface> JumpStepMaterial = nullptr;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Brush Materials"))
	TObjectPtr<UMaterialInterface> FindEdgesMaterial = nullptr;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, meta = (Category = "Brush Materials"))
	TObjectPtr<UMaterialInterface> BlurEdgesMaterial = nullptr;
	// Brush materials end

	// MIDs
	UPROPERTY(EditInstanceOnly, AdvancedDisplay, BlueprintReadWrite, Transient, meta=(Category="Debug MIDs"))
	TObjectPtr<UMaterialInstanceDynamic> BrushAngleFalloffMID = nullptr;

	UPROPERTY(EditInstanceOnly, AdvancedDisplay, BlueprintReadWrite, Transient, meta=(Category="Debug MIDs"))
	TObjectPtr<UMaterialInstanceDynamic> BrushWidthFalloffMID = nullptr;

	UPROPERTY(EditInstanceOnly, AdvancedDisplay, BlueprintReadWrite, Transient, meta=(Category="Debug MIDs"))
	TObjectPtr<UMaterialInstanceDynamic> DistanceFieldCacheMID = nullptr;

	UPROPERTY(EditInstanceOnly, AdvancedDisplay, BlueprintReadWrite, Transient, meta = (Category = "Debug MIDs"))
	TArray<TObjectPtr<UMaterialInstanceDynamic>> RiverSplineMIDs;

	UPROPERTY(EditInstanceOnly, AdvancedDisplay, BlueprintReadWrite, Transient, meta = (Category = "Debug MIDs"))
	TObjectPtr<UMaterialInstanceDynamic> DebugDistanceFieldMID = nullptr;

	UPROPERTY(EditInstanceOnly, AdvancedDisplay, BlueprintReadWrite, Transient, meta = (Category = "Debug MIDs"))
	TObjectPtr<UMaterialInstanceDynamic> WeightmapMID = nullptr;

	UPROPERTY(EditInstanceOnly, AdvancedDisplay, BlueprintReadWrite, Transient, meta = (Category = "Debug MIDs"))
	TObjectPtr<UMaterialInstanceDynamic> DrawCanvasMID = nullptr;

	UPROPERTY(EditInstanceOnly, AdvancedDisplay, BlueprintReadWrite, Transient, meta = (Category = "Debug MIDs"))
	TObjectPtr<UMaterialInstanceDynamic> CompositeWaterBodyTextureMID = nullptr;

	UPROPERTY(EditInstanceOnly, AdvancedDisplay, BlueprintReadWrite, Transient, meta = (Category = "Debug MIDs"))
	TObjectPtr<UMaterialInstanceDynamic> IslandFalloffMID = nullptr;
	// MIDs End

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Transient, meta=(Category="Debug"))
	TMap<TObjectPtr<UCurveFloat>,FWaterBodyBrushCache> BrushCurveRTCache;
	
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Transient, meta=(Category="Debug"))
	FVector WorldSize;
	
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Transient, meta=(Category="Debug"))
	FIntPoint LandscapeRTRes;
	
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Transient, meta=(Category="Debug"))
	FIntPoint LandscapeQuads;
	
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Transient, meta=(Category="Debug"))
	FTransform LandscapeTransform;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(Category="Debug"))
	bool ShowGradient = false;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(Category="Debug"))
	float DistanceDivisor = 0.1f;
			
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(Category="Debug"))
	bool ShowDistance = false;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(Category="Debug"))
	bool ShowGrid = false;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(Category="Settings"))
	float CanvasSegmentSize = 1024.0f;	
					
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(Category="Settings"))
	float WaterClearHeight = -16384.0f;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(Category="Settings"))
	float SplineMeshExtension = 0.0f;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(Category="Debug"))
	bool UseDynamicPreviewRT = false;
		
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(Category="Debug"))
	bool DisableBrushTextureEffects = false;

	UPROPERTY()
	bool bNeedsForceUpdate = false;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(all, "This material is now useless, the WaterVelocityTexture is now regenerated at runtime (WaterInfoTexture in AWaterZone).")
	UPROPERTY()
	TObjectPtr<UMaterialInterface> FinalizeVelocityHeightMaterial_DEPRECATED = nullptr;

	UE_DEPRECATED(all, "This material instance is now useless, the WaterVelocityTexture is now regenerated at runtime (WaterInfoTexture in AWaterZone).")
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> FinalizeVelocityHeightMID_DEPRECATED = nullptr;
#endif // WITH_EDITORONLY_DATA

	AWaterBrushManager(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
	virtual void OnConstruction(const FTransform& Transform) override;
	virtual void BeginPlay() override;
	virtual void PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph) override;
	virtual void BlueprintOnRenderTargetTexturesUpdated_Native(UTexture2D* VelocityTexture) override;
	virtual void BlueprintWaterBodyChanged_Native(AActor* Actor) override;
	virtual void Initialize_Native(FTransform const& InLandscapeTransform, FIntPoint const& InLandscapeSize, FIntPoint const& InLandscapeRenderTargetSize) override;
	virtual UTextureRenderTarget2D*  RenderLayer_Native(const FLandscapeBrushParameters& InParameters) override;
		
	virtual void GetRenderDependencies(TSet<UObject*>& OutDependencies) override;

	// Debug Buttons
	UFUNCTION(BlueprintCallable, meta = (CallInEditor = "true", Category = "Debug"))
	virtual void ForceUpdate();

	UFUNCTION(BlueprintCallable, meta = (CallInEditor = "true", Category = "Debug"))
	virtual void SingleBlurStep();

	UFUNCTION(BlueprintCallable, meta = (CallInEditor = "true", Category = "Debug"))
	virtual void FindEdges();

	UFUNCTION(BlueprintCallable, meta = (CallInEditor = "true", Category = "Debug"))
	virtual void SingleJumpStep();
	// End Debug Buttons
	
	UFUNCTION(BlueprintCallable, BlueprintPure, meta = (Category = "Debug"))
	virtual void GetWaterCacheKey(AActor* WaterBrush, /*out*/ UWaterBodyBrushCacheContainer*& ContainerObject, /*out*/ FWaterBodyBrushCache& Value);

	/** 
		Sorts the water bodies in the order they should be rendered when rendering the water brush
		@param InOutWaterBodies : list of water bodies that needs sorting
	*/
	UFUNCTION(BlueprintNativeEvent)
	void SortWaterBodiesForBrushRender(TArray<AWaterBody*>& InOutWaterBodies) const;

	UFUNCTION(BlueprintCallable, meta = (CallInEditor = "true", Category = "Debug"))
	void SetupDefaultMaterials();

private:
	/** Internal struct for passing information around when rendering a water brush actor. */
	struct FBrushActorRenderContext
	{
		FBrushActorRenderContext(const TWeakInterfacePtr<IWaterBrushActorInterface>& InWaterBrushActor)
			: WaterBrushActor(InWaterBrushActor)
		{}

		template <typename T>
		T* TryGetActorAs() const { return Cast<T>(WaterBrushActor.GetObject()); }

		template <typename T>
		T* GetActorAs() const { return CastChecked<T>(WaterBrushActor.GetObject()); }

		AActor* GetActor() const { return GetActorAs<AActor>(); }

		TWeakInterfacePtr<IWaterBrushActorInterface> WaterBrushActor;
		UWaterBodyBrushCacheContainer* CacheContainer = nullptr;
		UMaterialInstanceDynamic* MID = nullptr;
	};

	/** Internal struct for passing information around when rendering the whole brush. */
	struct FBrushRenderContext
	{
		bool bHeightmapRender = false; 
		FName WeightmapLayerName;
		int32 RTIndex = 0;
		int32 VelocityRTIndex = 0;
	};

private:
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool AllocateRTs();
	virtual void SetMPCParams();

	virtual bool SetupRiverSplineRenderMIDs(const FBrushActorRenderContext& BrushActorRenderContext, bool bRestoreMIDs, TArray<UMaterialInterface*>& InOutMIDs);
	virtual void CaptureMeshDepth(const TArrayView<UStaticMeshComponent*>& MeshComponents);
	virtual void CacheBrushDistanceField(const FBrushActorRenderContext& BrushActorRenderContext);
	virtual void DrawCanvasShape(const FBrushActorRenderContext& BrushActorRenderContext);
	virtual void DrawBrushMaterial(const FBrushRenderContext& BrushRenderContext, const FBrushActorRenderContext& BrushActorRenderContext);
	virtual void CaptureRiverDepthAndVelocity(const FBrushActorRenderContext& BrushActorRenderContext);
	virtual void UpdateCurves();
	virtual bool BrushRenderSetup();
	virtual void SetBrushMIDParams(const FBrushRenderContext& BrushRenderContext, FBrushActorRenderContext& BrushActorRenderContext);
	virtual void UpdateCurveCacheKeys();
	virtual void UpdateBrushCacheKeys();
	virtual void RenderBrushActorContext(FBrushRenderContext& BrushRenderContext, FBrushActorRenderContext& BrushActorRenderContext);
	virtual bool CreateMIDs();
	virtual void DistanceFieldCaching(const FBrushActorRenderContext& BrushActorRenderContext);
	virtual void CurvesSmoothingAndTerracing(const FBrushActorRenderContext& BrushActorRenderContext);
	virtual void FalloffAndBlendMode(const FBrushActorRenderContext& BrushActorRenderContext);
	virtual void DisplacementSettings(const FBrushActorRenderContext& BrushActorRenderContext);
	virtual void ApplyWeightmapSettings(const FBrushRenderContext& BrushRenderContext, const FBrushActorRenderContext& BrushActorRenderContext, const FWaterBodyWeightmapSettings& WMSettings);
	virtual void ApplyToCompositeWaterBodyTexture(FBrushRenderContext& BrushRenderContext, const FBrushActorRenderContext& BrushActorRenderContext);

#if WITH_EDITOR
	virtual void CheckForErrors() override;
#endif // WITH_EDITOR

	UCurveFloat* GetElevationCurveAsset(const FWaterCurveSettings& CurveSettings);
	void ClearCurveCache();
	void OnCurveUpdated(UCurveBase* Curve, EPropertyChangeType::Type ChangeType);
	void ComputeWaterLandscapeInfo(FVector& OutRTWorldLocation, FVector& OutRTWorldSizeVector) const;
	
	// HACK [jonathan.bard] : this is only needed for data deprecation, when LandscapeTransform and LandscapeRTRes were not serialized: 
	bool DeprecateWaterLandscapeInfo(FVector& OutRTWorldLocation, FVector& OutRTWorldSizeVector);

#if WITH_EDITOR
	void ShowForceUpdateMapCheckError();
#endif // WITH_EDITOR

	UTextureRenderTarget2D* VelocityPingPongRead(const FBrushRenderContext& BrushRenderContext) const;
	UTextureRenderTarget2D* VelocityPingPongWrite(const FBrushRenderContext& BrushRenderContext) const;

	UTextureRenderTarget2D* HeightPingPongRead(const FBrushRenderContext& BrushRenderContext) const;
	UTextureRenderTarget2D* HeightPingPongWrite(const FBrushRenderContext& BrushRenderContext) const;

	UTextureRenderTarget2D* WeightPingPongRead(const FBrushRenderContext& BrushRenderContext) const;
	UTextureRenderTarget2D* WeightPingPongWrite(const FBrushRenderContext& BrushRenderContext) const;

	static void AddDependencyIfValid(UObject* Dependency, TSet<UObject*>& OutDependencies);

private:
	bool bKillCache = false;
	// HACK [jonathan.bard] : shouldn't be needed anymore once deprecation is done : 
	FDelegateHandle OnWorldPostInitHandle;
	FDelegateHandle OnLevelAddedToWorldHandle;
};
