// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBrushManagerFactory.h"
#include "WaterBrushManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBrushManagerFactory)

#define LOCTEXT_NAMESPACE "WaterBrushManagerFactory"

// --------------------------------------------------
// AWaterBrushManager Factory
// --------------------------------------------------
UWaterBrushManagerFactory::UWaterBrushManagerFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("WaterBrushManagerFactoryDisplayName", "Water Brush Manager");
	NewActorClass = AWaterBrushManager::StaticClass();
}

void UWaterBrushManagerFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	AWaterBrushManager* WaterBrushManager = CastChecked<AWaterBrushManager>(NewActor);
	WaterBrushManager->SetupDefaultMaterials();
}

#undef LOCTEXT_NAMESPACE
