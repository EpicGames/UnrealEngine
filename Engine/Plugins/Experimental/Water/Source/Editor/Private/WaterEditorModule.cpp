// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterEditorModule.h"
#include "Engine/CollisionProfile.h"
#include "WaterUIStyle.h"
#include "PropertyEditorModule.h"
#include "WaterLandscapeBrush.h"
#include "EngineUtils.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Landscape.h"
#include "Logging/MessageLog.h"
#include "WaterZoneActor.h"
#include "WaterEditorSettings.h"
#include "Editor/UnrealEdEngine.h"
#include "WaterSplineComponentVisualizer.h"
#include "WaterSplineComponent.h"
#include "UnrealEdGlobals.h"
#include "WaterBodyActorFactory.h"
#include "WaterBodyIslandActorFactory.h"
#include "WaterZoneActorFactory.h"
#include "WaterBodyActorDetailCustomization.h"
#include "WaterBrushManagerFactory.h"
#include "AssetDefinition_WaterWaves.h"
#include "WaterBrushCacheContainer.h"
#include "WaterBodyBrushCacheContainerThumbnailRenderer.h"
#include "WaterWavesEditorToolkit.h"
#include "WaterRuntimeSettings.h"
#include "WaterBodyOceanComponent.h"

#define LOCTEXT_NAMESPACE "WaterEditor"

DEFINE_LOG_CATEGORY(LogWaterEditor);

EAssetTypeCategories::Type FWaterEditorModule::WaterAssetCategory;

namespace WaterEditorModule
{
	static TAutoConsoleVariable<float> CVarOverrideNewWaterZoneScale(TEXT("r.Water.WaterZoneActor.OverrideNewWaterZoneScale"), 0, TEXT("Multiply WaterZone actor extent beyond landscape by this amount. 0 means no override."));
	static TAutoConsoleVariable<float> CVarOverrideNewWaterZoneMinimumMargin(TEXT("r.Water.WaterZoneActor.OverrideNewWaterZoneMinimumMargin"), 0, TEXT("Enforce a minimum distance between landscape and the WaterZone actor extent. 0 means no override."));
}

void FWaterEditorModule::StartupModule()
{
	FWaterUIStyle::Initialize();

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	PropertyModule.RegisterCustomClassLayout(TEXT("WaterBody"), FOnGetDetailCustomizationInstance::CreateStatic(&FWaterBodyActorDetailCustomization::MakeInstance));

	GEngine->OnLevelActorAdded().AddRaw(this, &FWaterEditorModule::OnLevelActorAddedToWorld);

	RegisterComponentVisualizer(UWaterSplineComponent::StaticClass()->GetFName(), MakeShareable(new FWaterSplineComponentVisualizer));

	if (GEditor)
	{
		GEditor->ActorFactories.Add(NewObject<UWaterZoneActorFactory>());
		GEditor->ActorFactories.Add(NewObject<UWaterBodyIslandActorFactory>());
		GEditor->ActorFactories.Add(NewObject<UWaterBodyRiverActorFactory>());
		GEditor->ActorFactories.Add(NewObject<UWaterBodyLakeActorFactory>());
		GEditor->ActorFactories.Add(NewObject<UWaterBodyOceanActorFactory>());
		GEditor->ActorFactories.Add(NewObject<UWaterBodyCustomActorFactory>());
		GEditor->ActorFactories.Add(NewObject<UWaterBrushManagerFactory>());
	}

	UThumbnailManager::Get().RegisterCustomRenderer(UWaterBodyBrushCacheContainer::StaticClass(), UWaterBodyBrushCacheContainerThumbnailRenderer::StaticClass());

	OnLoadCollisionProfileConfigHandle = UCollisionProfile::Get()->OnLoadProfileConfig.AddLambda([this](UCollisionProfile* CollisionProfile)
		{
			check(UCollisionProfile::Get() == CollisionProfile);
			CheckForWaterCollisionProfile();
		});

	CheckForWaterCollisionProfile();
}

void FWaterEditorModule::ShutdownModule()
{
	if (UObjectInitialized())
	{
		UCollisionProfile::Get()->OnLoadProfileConfig.Remove(OnLoadCollisionProfileConfigHandle);

		UThumbnailManager::Get().UnregisterCustomRenderer(UWaterBodyBrushCacheContainer::StaticClass());
	}

	if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		PropertyModule.UnregisterCustomClassLayout(TEXT("WaterBody"));
	}

	if (GEngine)
	{
		GEngine->OnLevelActorAdded().RemoveAll(this);

		// Iterate over all class names we registered for
		for (FName ClassName : RegisteredComponentClassNames)
		{
			GUnrealEd->UnregisterComponentVisualizer(ClassName);
		}
	}

	FEditorDelegates::OnMapOpened.RemoveAll(this);

	FWaterUIStyle::Shutdown();
}

TSharedRef<FWaterWavesEditorToolkit> FWaterEditorModule::CreateWaterWaveAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UObject* WavesAsset)
{
	TSharedRef<FWaterWavesEditorToolkit> NewWaterWaveAssetEditor(new FWaterWavesEditorToolkit());
	NewWaterWaveAssetEditor->InitWaterWavesEditor(Mode, InitToolkitHost, WavesAsset);
	return NewWaterWaveAssetEditor;
}

void FWaterEditorModule::RegisterComponentVisualizer(FName ComponentClassName, TSharedPtr<FComponentVisualizer> Visualizer)
{
	if (GUnrealEd != NULL)
	{
		GUnrealEd->RegisterComponentVisualizer(ComponentClassName, Visualizer);
	}

	RegisteredComponentClassNames.Add(ComponentClassName);

	if (Visualizer.IsValid())
	{
		Visualizer->OnRegister();
	}
}

void FWaterEditorModule::OnLevelActorAddedToWorld(AActor* Actor)
{
	UWorld* ActorWorld = Actor->GetWorld();
	IWaterBrushActorInterface* WaterBrushActor = Cast<IWaterBrushActorInterface>(Actor);
	AWaterBody* WaterBodyActor = Cast<AWaterBody>(Actor);

	if (!Actor->bIsEditorPreviewActor && !Actor->HasAnyFlags(RF_Transient)
		&& ActorWorld && ActorWorld->IsEditorWorld()
		&& ((WaterBrushActor != nullptr) || (WaterBodyActor != nullptr)))
	{
		// Search for all overlapping landscapes 
		// If we cannot find a suitable landscape via this method, default to using the first landscape in the world.

		FBox WaterZoneBounds(ForceInit);
		TArray<ALandscape*> FoundLandscapes;
		bool bFoundIntersectingLandscape = false;

		const bool bNonColliding = true;
		const bool bIncludeChildActors = false;
		const FBox ActorBounds = Actor->GetComponentsBoundingBox(bNonColliding, bIncludeChildActors);

		ALandscape* FallbackLandscape = nullptr;
		for (ALandscape* Landscape : TActorRange<ALandscape>(ActorWorld))
		{
			check(Landscape);

			// This function is called while copy-pasting landscapes, before the landscape properties have been imported
			// in which case the Guid will be invalid, and trying to GetCompleteBounds will assert.
			// Note that skipping this logic means that we won't automatically create Water layers / Water brushes,
			// and any automatically created Water Zone may not be initialized to the full bounds of the landscapes
			if (Landscape->GetLandscapeGuid().IsValid())
			{
				if (FallbackLandscape == nullptr)
				{
					FallbackLandscape = Landscape;
				}

				const FBox LandscapeBounds = Landscape->GetCompleteBounds();
				if (LandscapeBounds.Intersect(ActorBounds))
				{
					FoundLandscapes.Add(Landscape);
					// Make sure the water zone's bounds is large enough to fit all landscapes that intersect with this water body :
					WaterZoneBounds += LandscapeBounds;
					bFoundIntersectingLandscape = true;
				}
			}
		}

		// If no intersecting landscape was found, use the first valid one as a fallback
		if (!bFoundIntersectingLandscape && FallbackLandscape != nullptr)
		{
			FoundLandscapes.Add(FallbackLandscape);
			const FBox LandscapeBounds = FallbackLandscape->GetCompleteBounds();
			WaterZoneBounds += LandscapeBounds;
		}

		const FBox LandscapeBounds = WaterZoneBounds;
		const UWaterEditorSettings* WaterEditorSettings = GetDefault<UWaterEditorSettings>();
		check(WaterEditorSettings != nullptr);
		// Automatically setup landscape-affecting features (water brush) if needed : 
		if ((WaterBrushActor != nullptr) && WaterBrushActor->AffectsLandscape() && !FoundLandscapes.IsEmpty())
		{
			TSubclassOf<AWaterLandscapeBrush> WaterBrushClass = WaterEditorSettings->GetWaterManagerClass();
			if (UClass* WaterBrushClassPtr = WaterBrushClass.Get())
			{
				if (!bFoundIntersectingLandscape)
				{
					UE_LOGF(LogWaterEditor, Warning, "Could not find a suitable landscape to which to assign the water brush! Defaulting to the first landscape.");
				}

				// Spawn a Water brush for every landscape this actor overlaps with.
				for (ALandscape* FoundLandscape : FoundLandscapes)
				{
					check(IsValid(FoundLandscape));

					bool bHasWaterManager = false;
					FoundLandscape->ForEachLayerConst([&bHasWaterManager](const FLandscapeLayer& Layer)
					{
						for (const FLandscapeLayerBrush& Brush : Layer.Brushes)
						{
							bHasWaterManager |= Cast<AWaterLandscapeBrush>(Brush.GetBrush()) != nullptr;
							if (bHasWaterManager)
							{
								// Stop iterating the moment we've found the brush manager : 
								return false;
							}
						}

						return true;
					});

					if (!bHasWaterManager)
					{
						UActorFactory* WaterBrushActorFactory = GEditor->FindActorFactoryForActorClass(WaterBrushClassPtr);

						// The manager is added from the BP brush tool only and is being spawned on a valid edit layer
						FString BrushActorString = FString::Format(TEXT("{0}_{1}"), { FoundLandscape->GetActorLabel(), WaterBrushClassPtr->GetName() });
						FName BrushActorName = MakeUniqueObjectName(FoundLandscape->GetOuter(), WaterBrushClassPtr, FName(BrushActorString));
						FActorSpawnParameters SpawnParams;
						SpawnParams.Name = BrushActorName;
						SpawnParams.bAllowDuringConstructionScript = true; // This can be called by construction script if the actor being added to the world is part of a blueprint, for example : 
						AWaterLandscapeBrush* NewBrush = (WaterBrushActorFactory != nullptr)
							? Cast<AWaterLandscapeBrush>(WaterBrushActorFactory->CreateActor(WaterBrushClassPtr, FoundLandscape->GetLevel(), FTransform(WaterZoneBounds.GetCenter()), SpawnParams))
							: ActorWorld->SpawnActor<AWaterLandscapeBrush>(WaterBrushClassPtr, SpawnParams);

						if (NewBrush)
						{
							if (!WaterBrushActorFactory)
							{
								UE_LOGF(LogWaterEditor, Warning, "WaterManager Actor Factory could not be found! The newly spawned %ls may have incorrect defaults!", *NewBrush->GetActorLabel());
							}

							bHasWaterManager = true;
							NewBrush->SetActorLabel(BrushActorString);
							NewBrush->SetTargetLandscape(FoundLandscape);
						}
					}
				}
			}
			else
			{
				UE_LOGF(LogWaterEditor, Warning, "Could not find Water Manager class %ls to spawn", *WaterEditorSettings->GetWaterManagerClassPath().GetAssetPathString());
			}
		}

		if (WaterBodyActor != nullptr && (WaterBodyActor->CanEverAffectWaterInfo() || WaterBodyActor->CanEverAffectWaterMesh()))
		{
			const bool bHasZoneActor = !!TActorIterator<AWaterZone>(ActorWorld);
			// Setup the water zone actor for this water body : 
			if (!bHasZoneActor)
			{
				TSubclassOf<AWaterZone> WaterZoneClass = WaterEditorSettings->GetWaterZoneClass();
				if (UClass* WaterZoneClassPtr = WaterZoneClass.Get())
				{
					UActorFactory* WaterZoneActorFactory = GEditor->FindActorFactoryForActorClass(WaterZoneClassPtr);

					FActorSpawnParameters SpawnParams;
					SpawnParams.OverrideLevel = ActorWorld->PersistentLevel;
					SpawnParams.bAllowDuringConstructionScript = true; // This can be called by construction script if the actor being added to the world is part of a blueprint, for example : 

					AWaterZone* WaterZoneActor = (WaterZoneActorFactory != nullptr)
						? Cast<AWaterZone>(WaterZoneActorFactory->CreateActor(WaterZoneClassPtr, Actor->GetLevel(), FTransform(WaterZoneBounds.GetCenter()), SpawnParams))
						: ActorWorld->SpawnActor<AWaterZone>(WaterZoneClassPtr, SpawnParams);

					if (WaterZoneActor)
					{
						if (!WaterZoneActorFactory)
						{
							UE_LOGF(LogWaterEditor, Warning, "WaterZone Actor Factory could not be found! The newly spawned %ls may have incorrect defaults!", *WaterZoneActor->GetActorLabel());
						}

						// TODO [jonathan.bard] : when we can tag static meshes as "water ground", add these to the bounds
						// Set a more sensible default location and extent so that the zone fully encapsulates the landscape if one exists.
						if (WaterZoneBounds.IsValid)
						{
							WaterZoneActor->SetActorLocation(WaterZoneBounds.GetCenter());

							// FBox::GetExtent returns the radius, SetZoneExtent expects diameter.
							FVector2D NewExtent = 2 * FVector2D(WaterZoneBounds.GetExtent());

							float ZoneExtentScale = WaterEditorModule::CVarOverrideNewWaterZoneScale.GetValueOnGameThread();
							const float MinimumMargin = WaterEditorModule::CVarOverrideNewWaterZoneMinimumMargin.GetValueOnGameThread();

							if (ZoneExtentScale == 0)
							{
								ZoneExtentScale = GetDefault<UWaterEditorSettings>()->WaterZoneActorDefaults.NewWaterZoneScale;
							}

							if (ZoneExtentScale != 0)
							{
								NewExtent = FMath::Abs(ZoneExtentScale) * NewExtent;
							}

							if ((MinimumMargin > 0) && (LandscapeBounds.IsValid))
							{
								const FVector2D LandscapeBoundsDiameter = 2.f * FVector2D(LandscapeBounds.GetExtent());
								const FVector2D MinimumWaterZoneExtent = LandscapeBoundsDiameter + 2.f * MinimumMargin;

								NewExtent.X = (NewExtent.X < MinimumWaterZoneExtent.X) ? MinimumWaterZoneExtent.X : NewExtent.X;
								NewExtent.Y = (NewExtent.Y < MinimumWaterZoneExtent.Y) ? MinimumWaterZoneExtent.Y : NewExtent.Y;
							}

							WaterZoneActor->SetZoneExtent(NewExtent);
						}
					}
				}
				else
				{
					UE_LOGF(LogWaterEditor, Warning, "Could not find Water Zone class %ls to spawn", *WaterEditorSettings->GetWaterZoneClassPath().GetAssetPathString());
				}

			}

			// If the actor is an ocean, we can help the user by initializing the ocean extent to fully fill the zone to which it belongs:
			check(WaterBodyActor->GetWaterBodyComponent());
			if (UWaterBodyOceanComponent* OceanComponent = Cast<UWaterBodyOceanComponent>(WaterBodyActor->GetWaterBodyComponent()))
			{
				OceanComponent->FillWaterZoneWithOcean();
			}
		}
	}
}

void FWaterEditorModule::CheckForWaterCollisionProfile()
{
	// Make sure WaterCollisionProfileName is added to Engine's collision profiles
	const FName WaterCollisionProfileName = GetDefault<UWaterRuntimeSettings>()->GetDefaultWaterCollisionProfileName();
	FCollisionResponseTemplate WaterBodyCollisionProfile;
	if (!UCollisionProfile::Get()->GetProfileTemplate(WaterCollisionProfileName, WaterBodyCollisionProfile))
	{
		// Only error when we have a UE project loaded, as otherwise the error would be unactionable since base UE doesn't provide a collision profile
		if (FApp::HasProjectName())
		{
			FMessageLog("LoadErrors").Error()
				->AddToken(FTextToken::Create(LOCTEXT("MissingWaterCollisionProfile", "Collision Profile settings do not include an entry for the Water Body Collision profile, which is required for water collision to function.")))
				->AddToken(FActionToken::Create(LOCTEXT("AddWaterCollisionProfile", "Add entry to DefaultEngine.ini?"), FText(),
					FOnActionTokenExecuted::CreateRaw(this, &FWaterEditorModule::AddWaterCollisionProfile), true));
		}
	}
}

void FWaterEditorModule::AddWaterCollisionProfile()
{
	// Make sure WaterCollisionProfileName is added to Engine's collision profiles
	const FName WaterCollisionProfileName = GetDefault<UWaterRuntimeSettings>()->GetDefaultWaterCollisionProfileName();
	FCollisionResponseTemplate WaterBodyCollisionProfile;
	if (FApp::HasProjectName() && !UCollisionProfile::Get()->GetProfileTemplate(WaterCollisionProfileName, WaterBodyCollisionProfile))
	{
		WaterBodyCollisionProfile.Name = WaterCollisionProfileName;
		WaterBodyCollisionProfile.CollisionEnabled = ECollisionEnabled::QueryOnly;
		WaterBodyCollisionProfile.ObjectType = ECollisionChannel::ECC_WorldStatic;
		WaterBodyCollisionProfile.bCanModify = false;
		WaterBodyCollisionProfile.ResponseToChannels = FCollisionResponseContainer::GetDefaultResponseContainer();
		WaterBodyCollisionProfile.ResponseToChannels.Camera = ECR_Ignore;
		WaterBodyCollisionProfile.ResponseToChannels.Visibility = ECR_Ignore;
		WaterBodyCollisionProfile.ResponseToChannels.WorldDynamic = ECR_Overlap;
		WaterBodyCollisionProfile.ResponseToChannels.Pawn = ECR_Overlap;
		WaterBodyCollisionProfile.ResponseToChannels.PhysicsBody = ECR_Overlap;
		WaterBodyCollisionProfile.ResponseToChannels.Destructible = ECR_Overlap;
		WaterBodyCollisionProfile.ResponseToChannels.Vehicle = ECR_Overlap;
#if WITH_EDITORONLY_DATA
		WaterBodyCollisionProfile.HelpMessage = TEXT("Default Water Collision Profile (Created by Water Plugin)");
#endif
		FCollisionProfilePrivateAccessor::AddProfileTemplate(WaterBodyCollisionProfile);
	}
}


IMPLEMENT_MODULE(FWaterEditorModule, WaterEditor);

#undef LOCTEXT_NAMESPACE
