// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterEditorSettings.h"
#include "Engine/StaticMesh.h"
#include "Engine/TextureDefines.h"
#include "Materials/MaterialInterface.h"
#include "WaterLandscapeBrush.h"
#include "WaterZoneActor.h"
#include "WaterWaves.h"
#include "UObject/ConstructorHelpers.h"
#include "Curves/CurveFloat.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterEditorSettings)

FWaterBrushActorDefaults::FWaterBrushActorDefaults()
{
	// Default values for water bodies : 
	static ConstructorHelpers::FObjectFinder<UCurveFloat> WaterElevationCurveAssetRef(TEXT("/Water/Curves/FloatCurve.FloatCurve"));
	CurveSettings.ElevationCurveAsset = WaterElevationCurveAssetRef.Object;
	CurveSettings.bUseCurveChannel = true;
	CurveSettings.ChannelDepth = 256.0f;
	CurveSettings.CurveRampWidth = 1024.0f;
}

FWaterZoneActorDefaults::FWaterZoneActorDefaults()
	: FarDistanceMaterial(FSoftObjectPath(TEXT("/Water/Materials/WaterSurface/Water_FarMesh.Water_FarMesh")))
{}

UMaterialInterface* FWaterZoneActorDefaults::GetFarDistanceMaterial() const
{
	return FarDistanceMaterial.LoadSynchronous();
}

FWaterBodyDefaults::FWaterBodyDefaults()
	: WaterMaterial(FSoftObjectPath(TEXT("/Water/Materials/WaterSurface/Water_Material.Water_Material")))
	, WaterHLODMaterial(FSoftObjectPath(TEXT("/Water/Materials/HLOD/HLODWater.HLODWater")))
	, UnderwaterPostProcessMaterial(FSoftObjectPath(TEXT("/Water/Materials/PostProcessing/M_UnderWater_PostProcess_Volume.M_UnderWater_PostProcess_Volume")))
{}

UMaterialInterface* FWaterBodyDefaults::GetWaterMaterial() const
{
	return WaterMaterial.LoadSynchronous();
}

UMaterialInterface* FWaterBodyDefaults::GetWaterStaticMeshMaterial() const
{
	return WaterStaticMeshMaterial.LoadSynchronous();
}

UMaterialInterface* FWaterBodyDefaults::GetWaterHLODMaterial() const
{
	return WaterHLODMaterial.LoadSynchronous();
}

UMaterialInterface* FWaterBodyDefaults::GetUnderwaterPostProcessMaterial() const
{
	return UnderwaterPostProcessMaterial.LoadSynchronous();
}

FWaterBodyRiverDefaults::FWaterBodyRiverDefaults()
	: FWaterBodyDefaults()
	, RiverToOceanTransitionMaterial(FSoftObjectPath(TEXT("/Water/Materials/WaterSurface/Transitions/Water_Material_River_To_Ocean_Transition.Water_Material_River_To_Ocean_Transition")))
	, RiverToLakeTransitionMaterial(FSoftObjectPath(TEXT("/Water/Materials/WaterSurface/Transitions/Water_Material_River_To_Lake_Transition.Water_Material_River_To_Lake_Transition")))
{
	BrushDefaults.CurveSettings.CurveRampWidth = 512.0f;
	BrushDefaults.HeightmapSettings.FalloffSettings.FalloffMode = EWaterBrushFalloffMode::Width;

	BrushDefaults.HeightmapSettings.FalloffSettings.EdgeOffset = 256.0f;
	BrushDefaults.HeightmapSettings.FalloffSettings.ZOffset = 16.0f;
	
	WaterMaterial = FSoftObjectPath(TEXT("/Water/Materials/WaterSurface/Water_Material_River.Water_Material_River"));
	WaterStaticMeshMaterial = FSoftObjectPath(TEXT("/Water/Materials/WaterSurface/LODs/Water_Material_River_LOD.Water_Material_River_LOD"));
}

UMaterialInterface* FWaterBodyRiverDefaults::GetRiverToOceanTransitionMaterial() const
{
	return RiverToOceanTransitionMaterial.LoadSynchronous();
}

UMaterialInterface* FWaterBodyRiverDefaults::GetRiverToLakeTransitionMaterial() const
{
	return RiverToLakeTransitionMaterial.LoadSynchronous();
}

FWaterBodyLakeDefaults::FWaterBodyLakeDefaults()
	: FWaterBodyDefaults()
{
	BrushDefaults.CurveSettings.ChannelEdgeOffset = 0.0f;
	BrushDefaults.CurveSettings.ChannelDepth = 500.0f;
	BrushDefaults.CurveSettings.CurveRampWidth = 2000.0f;

	BrushDefaults.HeightmapSettings.FalloffSettings.EdgeOffset = 500.0f;
	BrushDefaults.HeightmapSettings.FalloffSettings.ZOffset = 0.0f;

	WaterMaterial = FSoftObjectPath(TEXT("/Water/Materials/WaterSurface/Water_Material_Lake.Water_Material_Lake"));
	WaterStaticMeshMaterial = FSoftObjectPath(TEXT("/Water/Materials/WaterSurface/LODs/Water_Material_Lake_LOD.Water_Material_Lake_LOD"));
}

FWaterBodyOceanDefaults::FWaterBodyOceanDefaults()
	: FWaterBodyDefaults()
{
	BrushDefaults.CurveSettings.ChannelEdgeOffset = -1000.0f;
	BrushDefaults.CurveSettings.ChannelDepth = 2000.0f;
	BrushDefaults.CurveSettings.CurveRampWidth = 8000.0f;

	BrushDefaults.HeightmapSettings.FalloffSettings.EdgeOffset = 1000.0f;
	BrushDefaults.HeightmapSettings.FalloffSettings.ZOffset = 32.0f;

	WaterMaterial = FSoftObjectPath(TEXT("/Water/Materials/WaterSurface/Water_Material_Ocean.Water_Material_Ocean"));
	WaterStaticMeshMaterial = FSoftObjectPath(TEXT("/Water/Materials/WaterSurface/LODs/Water_Material_Ocean_LOD.Water_Material_Ocean_LOD"));
}

FWaterBodyCustomDefaults::FWaterBodyCustomDefaults()
	: FWaterBodyDefaults()
	, WaterMesh(FSoftObjectPath(TEXT("/Water/Meshes/S_WaterPlane_256.S_WaterPlane_256")))
{
	WaterMaterial = FSoftObjectPath(TEXT("/Water/Materials/WaterSurface/Water_Material_CustomMesh.Water_Material_CustomMesh"));
	UnderwaterPostProcessMaterial = FSoftObjectPath(TEXT("/Water/Materials/PostProcessing/MI_UnderWater_PostProcess_Volume_Custom.MI_UnderWater_PostProcess_Volume_Custom"));
}

UStaticMesh* FWaterBodyCustomDefaults::GetWaterMesh() const
{
	return WaterMesh.LoadSynchronous();
}

FWaterBodyIslandDefaults::FWaterBodyIslandDefaults()
{
	static ConstructorHelpers::FObjectFinder<UCurveFloat> IslandElevationCurveAssetRef(TEXT("/Water/Curves/FloatCurve_WaterBodyIsland.FloatCurve_WaterBodyIsland"));
	BrushDefaults.CurveSettings.ElevationCurveAsset = IslandElevationCurveAssetRef.Object;
	BrushDefaults.CurveSettings.bUseCurveChannel = true;
	BrushDefaults.CurveSettings.ChannelDepth = 0.0f;
	BrushDefaults.CurveSettings.CurveRampWidth = 1024.0f;

	BrushDefaults.HeightmapSettings.BlendMode = EWaterBrushBlendType::AlphaBlend;
	BrushDefaults.HeightmapSettings.FalloffSettings.FalloffMode = EWaterBrushFalloffMode::Angle;
	BrushDefaults.HeightmapSettings.FalloffSettings.FalloffAngle = 45.0f;
	BrushDefaults.HeightmapSettings.FalloffSettings.FalloffWidth = 1024.0f;
	BrushDefaults.HeightmapSettings.FalloffSettings.EdgeOffset = 0;
	BrushDefaults.HeightmapSettings.FalloffSettings.ZOffset = 0;
}

UWaterEditorSettings::UWaterEditorSettings()
	: TextureGroupForGeneratedTextures(TEXTUREGROUP_World)
	, MaxWaterVelocityAndHeightTextureSize(2048)
	, LandscapeMaterialParameterCollection(FSoftObjectPath(TEXT("/Landmass/Landscape/BlueprintBrushes/MPC/MPC_Landscape.MPC_Landscape")))
	, WaterZoneClassPath(TEXT("/Script/Water.WaterZone"))
	, WaterManagerClassPath(TEXT("/Script/WaterEditor.WaterBrushManager"))
	, DefaultBrushAngleFalloffMaterial(FSoftObjectPath(TEXT("/Water/Materials/Brushes/MeshBrush_Angle.MeshBrush_Angle")))
	, DefaultBrushIslandFalloffMaterial(FSoftObjectPath(TEXT("/Water/Materials/Brushes/MeshBrush_Island.MeshBrush_Island")))
	, DefaultBrushWidthFalloffMaterial(FSoftObjectPath(TEXT("/Water/Materials/Brushes/MeshBrush_Width.MeshBrush_Width")))
	, DefaultBrushWeightmapMaterial(FSoftObjectPath(TEXT("/Water/Materials/Brushes/MeshBrush_Weightmap.MeshBrush_Weightmap")))
	, DefaultCacheDistanceFieldCacheMaterial(FSoftObjectPath(TEXT("/Landmass/Landscape/BlueprintBrushes/Materials/CacheDistanceField.CacheDistanceField")))
	, DefaultCompositeWaterBodyTextureMaterial(FSoftObjectPath(TEXT("/Water/Materials/Brushes/CompositeWaterBodyTexture.CompositeWaterBodyTexture")))
	, DefaultJumpFloodStepMaterial(FSoftObjectPath(TEXT("/Landmass/DistanceFields/Materials/JumpFloodStep.JumpFloodStep")))
	, DefaultBlurEdgesMaterial(FSoftObjectPath(TEXT("/Landmass/DistanceFields/Materials/BlurEdges.BlurEdges")))
	, DefaultFindEdgesMaterial(FSoftObjectPath(TEXT("/Landmass/DistanceFields/Materials/DetectEdges.DetectEdges")))
	, DefaultDrawCanvasMaterial(FSoftObjectPath(TEXT("/Water/Materials/Brushes/CanvasDrawing.CanvasDrawing")))
	, DefaultRenderRiverSplineDepthsMaterial(FSoftObjectPath(TEXT("/Landmass/Landscape/BlueprintBrushes/Materials/RenderSplineDepths.RenderSplineDepths")))
{

	// Ideally, this should be done in FWaterBodyLakeDefaults but we need CreateEditorOnlyDefaultSubobject, which is only available in a UObject : 
	if (UWaterWavesAssetReference* WaterWavesRef = CreateEditorOnlyDefaultSubobject<UWaterWavesAssetReference>(TEXT("DefaultLakeWaterWaves"), /* bTransient = */false))
	{
		static ConstructorHelpers::FObjectFinder<UWaterWavesAsset> WaterWavesAssetRef(TEXT("/Water/Waves/GerstnerWaves_Lake.GerstnerWaves_Lake"));
		WaterWavesRef->SetWaterWavesAsset(WaterWavesAssetRef.Object);
		WaterBodyLakeDefaults.WaterWaves = WaterWavesRef;
	}

	// Ideally, this should be done in FWaterBodyLakeDefaults but we need CreateEditorOnlyDefaultSubobject, which is only available in a UObject : 
	if (UWaterWavesAssetReference* WaterWavesRef = CreateEditorOnlyDefaultSubobject<UWaterWavesAssetReference>(TEXT("DefaultOceanWaterWaves"), /* bTransient = */false))
	{
		static ConstructorHelpers::FObjectFinder<UWaterWavesAsset> WaterWavesAssetRef(TEXT("/Water/Waves/GerstnerWaves_Ocean.GerstnerWaves_Ocean"));
		WaterWavesRef->SetWaterWavesAsset(WaterWavesAssetRef.Object);
		WaterBodyOceanDefaults.WaterWaves = WaterWavesRef;
	}
}

TSubclassOf<AWaterZone> UWaterEditorSettings::GetWaterZoneClass() const
{
	return WaterZoneClassPath.TryLoadClass<AWaterZone>();
}

TSubclassOf<AWaterLandscapeBrush> UWaterEditorSettings::GetWaterManagerClass() const
{
	return WaterManagerClassPath.TryLoadClass<AWaterLandscapeBrush>();
}

UMaterialInterface* UWaterEditorSettings::GetDefaultBrushAngleFalloffMaterial() const
{
	return DefaultBrushAngleFalloffMaterial.LoadSynchronous();
}

UMaterialInterface* UWaterEditorSettings::GetDefaultBrushIslandFalloffMaterial() const
{
	return DefaultBrushIslandFalloffMaterial.LoadSynchronous();
}

UMaterialInterface* UWaterEditorSettings::GetDefaultBrushWidthFalloffMaterial() const
{
	return DefaultBrushWidthFalloffMaterial.LoadSynchronous();
}

UMaterialInterface* UWaterEditorSettings::GetDefaultBrushWeightmapMaterial() const
{
	return DefaultBrushWeightmapMaterial.LoadSynchronous();
}

UMaterialInterface* UWaterEditorSettings::GetDefaultCacheDistanceFieldCacheMaterial() const
{
	return DefaultCacheDistanceFieldCacheMaterial.LoadSynchronous();
}

UMaterialInterface* UWaterEditorSettings::GetDefaultCompositeWaterBodyTextureMaterial() const
{
	return DefaultCompositeWaterBodyTextureMaterial.LoadSynchronous();
}

UMaterialInterface* UWaterEditorSettings::GetDefaultJumpFloodStepMaterial() const
{
	return DefaultJumpFloodStepMaterial.LoadSynchronous();
}

UMaterialInterface* UWaterEditorSettings::GetDefaultBlurEdgesMaterial() const
{
	return DefaultBlurEdgesMaterial.LoadSynchronous();
}

UMaterialInterface* UWaterEditorSettings::GetDefaultFindEdgesMaterial() const
{
	return DefaultFindEdgesMaterial.LoadSynchronous();
}

UMaterialInterface* UWaterEditorSettings::GetDefaultDrawCanvasMaterial() const
{
	return DefaultDrawCanvasMaterial.LoadSynchronous();
}

UMaterialInterface* UWaterEditorSettings::GetDefaultRenderRiverSplineDepthsMaterial() const
{
	return DefaultRenderRiverSplineDepthsMaterial.LoadSynchronous();
}

