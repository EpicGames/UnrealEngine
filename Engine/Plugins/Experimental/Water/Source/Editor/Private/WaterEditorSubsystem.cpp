// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterEditorSubsystem.h"

#include "ActionableMessageSubsystem.h"
#include "Engine/Texture2D.h"
#include "UObject/ConstructorHelpers.h"
#include "WaterEditorSettings.h"
#include "Materials/MaterialParameterCollection.h"
#include "WaterModule.h"
#include "Modules/ModuleManager.h"
#include "Editor.h"
#include "LevelEditor.h"
#include "WaterSubsystem.h"
#include "Algo/Count.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterEditorSubsystem)

#define LOCTEXT_NAMESPACE "WaterEditorSubsystem"

namespace UE::Water
{
	static const FName WaterBodiesNeedSaveKey(TEXT("WaterBodiesNeedResave_MessageKey"));

	static void MarkOutdatedPackagesDirty()
	{
		const IWaterModuleInterface& WaterModule = FModuleManager::GetModuleChecked<IWaterModuleInterface>("Water");
		if (IWaterEditorServices* WaterEditorServices = WaterModule.GetWaterEditorServices())
		{
			WaterEditorServices->DirtyAllModifiedPackages();
		}
	}
}

UWaterEditorSubsystem::UWaterEditorSubsystem()
{
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinderOptional<UTexture2D> DefaultWaterActorSprite;
		ConstructorHelpers::FObjectFinderOptional<UTexture2D> ErrorSprite;

		FConstructorStatics()
			: DefaultWaterActorSprite(TEXT("/Water/Icons/WaterSprite"))
			, ErrorSprite(TEXT("/Water/Icons/WaterErrorSprite"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	DefaultWaterActorSprite = ConstructorStatics.DefaultWaterActorSprite.Get();
	ErrorSprite = ConstructorStatics.ErrorSprite.Get();
}

void UWaterEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	LandscapeMaterialParameterCollection = GetDefault<UWaterEditorSettings>()->LandscapeMaterialParameterCollection.LoadSynchronous();

	IWaterModuleInterface& WaterModule = FModuleManager::GetModuleChecked<IWaterModuleInterface>("Water");
	WaterModule.SetWaterEditorServices(this);

	UPackage::PackageDirtyStateChangedEvent.AddUObject(this, &UWaterEditorSubsystem::OnPackageDirtied);
	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditor.OnMapChanged().AddUObject(this, &UWaterEditorSubsystem::OnMapChanged);
}

void UWaterEditorSubsystem::Deinitialize()
{
	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditor.OnMapChanged().RemoveAll(this);
	UPackage::PackageDirtyStateChangedEvent.RemoveAll(this);

	IWaterModuleInterface& WaterModule = FModuleManager::GetModuleChecked<IWaterModuleInterface>("Water");
	if (IWaterEditorServices* WaterEditorServices = WaterModule.GetWaterEditorServices())
	{
		if (WaterEditorServices == this)
		{
			WaterModule.SetWaterEditorServices(nullptr);
		}
	}

	Super::Deinitialize();
}

void UWaterEditorSubsystem::RegisterWaterActorSprite(UClass* InClass, UTexture2D* Texture)
{
	WaterActorSprites.Add(InClass, Texture);
}

UTexture2D* UWaterEditorSubsystem::GetWaterActorSprite(UClass* InClass) const
{
	UClass const* Class = InClass;
	typename decltype(WaterActorSprites)::ValueType const* SpritePtr = nullptr;

	// Traverse the class hierarchy and find the first available sprite
	while (Class != nullptr && SpritePtr == nullptr)
	{
		SpritePtr = WaterActorSprites.Find(Class);
		Class = Class->GetSuperClass();
	}

	if (SpritePtr != nullptr)
	{
		return *SpritePtr;
	}

	return DefaultWaterActorSprite;
}

UWorld* UWaterEditorSubsystem::GetEditorWorld() const
{
	return GEditor ? GEditor->GetEditorWorldContext(false).World() : nullptr;
}

void UWaterEditorSubsystem::UpdateModifiedPackagesMessage()
{
	if (UWorld* EditorWorld = GetEditorWorld())
	{
		if (UActionableMessageSubsystem* ActionableMessageSubsystem = EditorWorld->GetSubsystem<UActionableMessageSubsystem>())
		{
			if (PackagesNeedingDirtying.Num() > 0)
			{
				FActionableMessage Message;
				Message.Message = LOCTEXT("WaterBodiesNeedSave_Message", "Some water bodies are not up to date");
				Message.Tooltip = LOCTEXT("WaterBodiesNeedSave_Tooltip", "Some water bodies have some derived data that is being rebuilt on load. This is causing editor loading to slow down and can be resolved by simply resaving the water bodies.");
				Message.ActionMessage = LOCTEXT("WaterBodiesNeedSave_ActionMsg", "Mark Dirty");
				Message.ActionCallback = UE::Water::MarkOutdatedPackagesDirty;
				ActionableMessageSubsystem->SetActionableMessage(UE::Water::WaterBodiesNeedSaveKey, Message);
			}
			else
			{
				ActionableMessageSubsystem->ClearActionableMessage(UE::Water::WaterBodiesNeedSaveKey);
			}
		}
	}
}

void UWaterEditorSubsystem::OnPackageDirtied(UPackage* Package)
{
	if (!bSuppressOnDirtyEvents)
	{
		if (Package && Package->IsDirty())
		{
			// Only update the modified packages messsage if we actually made a change:
			if (PackagesNeedingDirtying.Remove(Package) > 0)
			{
				UpdateModifiedPackagesMessage();
			}
		}
	}
}

void UWaterEditorSubsystem::OnMapChanged(UWorld* InWorld, EMapChangeType InChangeType)
{
	// Stop tracking packages for maps that are no longer loaded.
	if (InChangeType == EMapChangeType::TearDownWorld)
	{
		ClearModifiedPackages();
	}
}

bool UWaterEditorSubsystem::TryMarkPackageAsModified(UPackage* InPackage)
{
	// Don't need to mark it as dirty while transacting since it will be marked dirty later anyways:
	if (GIsTransacting)
	{
		return false;
	}

	// Don't need to do anything if the package is already dirty:
	if (InPackage->IsDirty())
	{
		return false;
	}

	// We don't care about temp packages:
	if (FPackageName::IsTempPackage(InPackage->GetName()))
	{
		return false;
	}

	PackagesNeedingDirtying.Add(InPackage);

	UpdateModifiedPackagesMessage();

	return true;
}

bool UWaterEditorSubsystem::HasAnyModifiedPackages() const
{
	return IntCastChecked<int32>(Algo::CountIf(PackagesNeedingDirtying, [](const TWeakObjectPtr<UPackage>& InWeakPackagePtr) { return InWeakPackagePtr.IsValid(); })) > 0;
}

void UWaterEditorSubsystem::ForEachModifiedPackage(const TFunctionRef<bool(UPackage*)>& Func) const
{
	for (TWeakObjectPtr<UPackage> WeakPackage : PackagesNeedingDirtying)
	{
		if (WeakPackage.IsValid())
		{
			if (!Func(WeakPackage.Get()))
			{
				break;
			}
		}
	}
}

void UWaterEditorSubsystem::ClearModifiedPackages()
{
	PackagesNeedingDirtying.Empty();
	UpdateModifiedPackagesMessage();
}

void UWaterEditorSubsystem::DirtyAllModifiedPackages()
{
	TGuardValue<bool> SuppressOnDirtyEvents(bSuppressOnDirtyEvents, true);

	ForEachModifiedPackage([](UPackage* Package)
	{
		if (!Package->IsDirty())
		{
			Package->SetDirtyFlag(true);
		}
		return true;
	});

	ClearModifiedPackages();
}

bool UWaterEditorSubsystem::GetShouldUpdateWaterMeshDuringInteractiveChanges() const
{
	return GetDefault<UWaterEditorSettings>()->GetShouldUpdateWaterMeshDuringInteractiveChanges();
}

#undef LOCTEXT_NAMESPACE

