// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterLandscapeBrush.h"
#include "Landscape.h"
#include "Engine/Engine.h"
#include "UObject/WeakInterfacePtr.h"
#include "WaterBodyActor.h"
#include "WaterBodyIslandActor.h"
#include "WaterEditorServices.h"
#include "WaterSubsystem.h"
#include "Engine/TextureRenderTarget2D.h"
#include "EngineUtils.h"
#include "Algo/AnyOf.h"
#include "Misc/MapErrors.h"
#include "Misc/UObjectToken.h"
#include "Logging/MessageLog.h"
#include "WaterIconHelper.h"
#include "Components/BillboardComponent.h"
#include "Modules/ModuleManager.h"
#include "WaterModule.h"
#include "LandscapeSettings.h"
#include "LandscapeModule.h"
#include "LandscapeEditorServices.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterLandscapeBrush)

#define LOCTEXT_NAMESPACE "WaterLandscapeBrush"

AWaterLandscapeBrush::AWaterLandscapeBrush(const FObjectInitializer& ObjectInitializer)
{
	SetCanAffectHeightmap(true);

	ActorIcon = FWaterIconHelper::EnsureSpriteComponentCreated(this, TEXT("/Water/Icons/WaterLandscapeBrushSprite"));
}

void AWaterLandscapeBrush::AddActorInternal(AActor* Actor, const UWorld* ThisWorld, UObject* InCache, bool bTriggerEvent, bool bModify)
{
	if (IsActorAffectingLandscape(Actor) &&
		!Actor->HasAnyFlags(RF_Transient | RF_ClassDefaultObject | RF_ArchetypeObject) &&
		IsValidChecked(Actor) &&
		!Actor->IsUnreachable() &&
		Actor->GetLevel() != nullptr &&
		!Actor->GetLevel()->bIsBeingRemoved &&
		ThisWorld == Actor->GetWorld() &&
		!ActorsAffectingLandscape.Contains(Cast<IWaterBrushActorInterface>(Actor)))
	{
		if (bModify)
		{
			const bool bMarkPackageDirty = false;
			Modify(bMarkPackageDirty);
		}

		IWaterBrushActorInterface* WaterBrushActor = CastChecked<IWaterBrushActorInterface>(Actor);
		ActorsAffectingLandscape.Add(TWeakInterfacePtr<IWaterBrushActorInterface>(WaterBrushActor));

		if (InCache)
		{
			Cache.Add(TSoftObjectPtr<AActor>(Actor), InCache);
		}

		if (bTriggerEvent)
		{
			UpdateAffectedWeightmaps();
			OnActorsAffectingLandscapeChanged();
		}
	}
}

void AWaterLandscapeBrush::RemoveActorInternal(AActor* Actor)
{
	IWaterBrushActorInterface* WaterBrushActor = CastChecked<IWaterBrushActorInterface>(Actor);

	const bool bMarkPackageDirty = false;
	Modify(bMarkPackageDirty);
	int32 Index = ActorsAffectingLandscape.IndexOfByKey(TWeakInterfacePtr<IWaterBrushActorInterface>(WaterBrushActor));
	if (Index != INDEX_NONE)
	{
		ActorsAffectingLandscape.RemoveAt(Index);
		Cache.Remove(TSoftObjectPtr<AActor>(Actor));

		OnActorsAffectingLandscapeChanged();

		UpdateAffectedWeightmaps();
	}
}

void AWaterLandscapeBrush::BlueprintWaterBodiesChanged_Implementation()
{
	BlueprintWaterBodiesChanged_Native();
}

void AWaterLandscapeBrush::UpdateAffectedWeightmaps()
{
	AffectedWeightmapLayers.Empty();
	for (const TWeakInterfacePtr<IWaterBrushActorInterface>& WaterBrushActor : ActorsAffectingLandscape)
	{
		if (WaterBrushActor.IsValid())
		{
			for (const TPair<FName, FWaterBodyWeightmapSettings>& Pair : WaterBrushActor->GetLayerWeightmapSettings())
			{
				AffectedWeightmapLayers.AddUnique(Pair.Key);
			}
		}
	}
}

template<class T>
class FGetActorsOfType
{
public:
	void operator()(const AWaterLandscapeBrush* Brush, TSubclassOf<T> ActorClass, TArray<T*>& OutActors)
	{
		OutActors.Empty();
		for (const TWeakInterfacePtr<IWaterBrushActorInterface>& WaterBrushActor : Brush->ActorsAffectingLandscape)
		{
			T* Actor = Cast<T>(WaterBrushActor.GetObject());
			if (Actor && Actor->IsA(ActorClass))
			{
				OutActors.Add(Actor);
			}
		}
	}
};

void AWaterLandscapeBrush::UpdateActors(bool bInTriggerEvents)
{
	if (IsTemplate())
	{
		return;
	}

	check(!IsUnreachable());

	const bool bMarkPackageDirty = false;
	Modify(bMarkPackageDirty);

	ClearActors();

	// Backup Cache
	TMap<TSoftObjectPtr<AActor>, TObjectPtr<UObject>> PreviousCache;
	Swap(Cache, PreviousCache);

	if (UWorld* World = GetWorld())
	{
		for (FActorIterator It(World); It; ++It)
		{
			AActor* Actor = *It;
			if (IWaterBrushActorInterface* WaterBrushActor = Cast<IWaterBrushActorInterface>(Actor))
			{
				const TObjectPtr<UObject>* FoundCache = PreviousCache.Find(TSoftObjectPtr<AActor>(Actor));
				const bool bTriggerEvent = false;
				const bool bModify = false;
				AddActorInternal(Actor, World, FoundCache != nullptr ? *FoundCache : nullptr, bTriggerEvent, bModify);
			}
		}
	}

	UpdateAffectedWeightmaps();

	if (bInTriggerEvents)
	{
		OnActorsAffectingLandscapeChanged();
	}
}

void AWaterLandscapeBrush::OnWaterBrushActorChanged(const IWaterBrushActorInterface::FWaterBrushActorChangedEventParams& InParams)
{
	// Make sure we don't process this callback when being torn down
	if (IsUnreachable())
	{
		return;
	}

	AActor* Actor = CastChecked<AActor>(InParams.WaterBrushActor);
	const bool bAffectsLandscape = InParams.WaterBrushActor->AffectsLandscape();
	const bool bAffectsWaterMesh = InParams.WaterBrushActor->AffectsWaterMesh();

	const bool bActorAlreadyAffectingLandscape = ActorsAffectingLandscape.Contains(InParams.WaterBrushActor);
	// if the actor went from affecting landscape to non-affecting landscape (and vice versa), update the brush
	bool bForceUpdateBrush = false;
	
	if (bAffectsLandscape != bActorAlreadyAffectingLandscape)
	{
		if (bAffectsLandscape)
		{
			AddActorInternal(Actor, GetWorld(), nullptr, /*bTriggerEvent = */true, /*bModify =*/true);
		}
		else
		{
			RemoveActorInternal(Actor);
		}

		bForceUpdateBrush = true;
	}

	if (InParams.bWeightmapSettingsChanged)
	{
		UpdateAffectedWeightmaps();
	}

	BlueprintWaterBodyChanged(Actor);

	const ULandscapeSettings* LandscapeSettings = GetDefault<ULandscapeSettings>();
	check(LandscapeSettings != nullptr);

	const bool bAllowLandscapeUpdate = (InParams.PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive) || LandscapeSettings->GetShouldUpdateEditLayersDuringInteractiveChanges();
	if (bForceUpdateBrush || (bAffectsLandscape && bAllowLandscapeUpdate))
	{
		RequestLandscapeUpdate(/* bInUserTriggered = */ InParams.bUserTriggered);
	}
}

void AWaterLandscapeBrush::OnActorsAffectingLandscapeChanged()
{
	BlueprintWaterBodiesChanged();
	RequestLandscapeUpdate();
	if (UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(GetWorld()))
	{
		WaterSubsystem->MarkAllWaterZonesForRebuild(EWaterZoneRebuildFlags::All, /* DebugRequestingObject = */ this);
	}
}

bool AWaterLandscapeBrush::IsActorAffectingLandscape(AActor* Actor) const
{
	IWaterBrushActorInterface* WaterBrushActor = Cast<IWaterBrushActorInterface>(Actor);
	return ((WaterBrushActor != nullptr) && WaterBrushActor->AffectsLandscape());
}

void AWaterLandscapeBrush::RegisterDelegates()
{
	if (!IsTemplate())
	{
		OnWorldPostInitHandle = FWorldDelegates::OnPostWorldInitialization.AddLambda([this](UWorld* World, const UWorld::InitializationValues IVS)
		{
			if (World == GetWorld())
			{
				const bool bTriggerEvents = false;
				UpdateActors(bTriggerEvents);
			}
		});

		OnLevelAddedToWorldHandle = FWorldDelegates::LevelAddedToWorld.AddLambda([this](ULevel* Level, UWorld* World)
		{
			if ((World == GetWorld())
				&& (World->IsEditorWorld()
				&& (Level != nullptr)
				&& !IsUnreachable() // It seems like we can receive this callback while being torn down
				&& Algo::AnyOf(Level->Actors, [this](AActor* Actor) { return IsActorAffectingLandscape(Actor); })))
			{
				UpdateActors(!UE::GetIsEditorLoadingPackage());
			}
		});

		OnLevelRemovedFromWorldHandle = FWorldDelegates::LevelRemovedFromWorld.AddLambda([this](ULevel* Level, UWorld* World)
		{
			if ((World == GetWorld())
				&& (World->IsEditorWorld()
				&& (Level != nullptr)
				&& !IsUnreachable() // It seems like we can receive this callback while being torn down
				&& Algo::AnyOf(Level->Actors, [this](AActor* Actor) { return IsActorAffectingLandscape(Actor); })))
			{
				UpdateActors(!UE::GetIsEditorLoadingPackage());
			}
		});

		OnLevelActorAddedHandle = GEngine->OnLevelActorAdded().AddUObject(this, &AWaterLandscapeBrush::OnLevelActorAdded);
		OnLevelActorDeletedHandle = GEngine->OnLevelActorDeleted().AddUObject(this, &AWaterLandscapeBrush::OnLevelActorRemoved);

		IWaterBrushActorInterface::GetOnWaterBrushActorChangedEvent().AddUObject(this, &AWaterLandscapeBrush::OnWaterBrushActorChanged);
	}
}

void AWaterLandscapeBrush::PostActorCreated()
{
	Super::PostActorCreated();

	RegisterDelegates();

	constexpr bool bInTriggerEvents = true;
	UpdateActors(bInTriggerEvents);
}

void AWaterLandscapeBrush::PostLoad()
{
	Super::PostLoad();

	RegisterDelegates();

	// If we are loading do not trigger events
	constexpr bool bInTriggerEvents = false;
	UpdateActors(bInTriggerEvents);
}

void AWaterLandscapeBrush::OnLevelActorAdded(AActor* InActor)
{
	if ((InActor->GetWorld() == GetWorld())
		&& !IsUnreachable()) // It seems like we can receive this callback while being torn down
	{
		AddActorInternal(InActor, GetWorld(), /*InCache = */nullptr, /*bTriggerEvent = */true, /*bModify = */true);
	}
}

void AWaterLandscapeBrush::OnLevelActorRemoved(AActor* InActor)
{
	if ((InActor->GetWorld() == GetWorld())
		&& !IsUnreachable()) // It seems like we can receive this callback while being torn down
	{
		if (IsActorAffectingLandscape(InActor))
		{
			RemoveActorInternal(InActor);
		}
	}
}

void AWaterLandscapeBrush::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);

	UpdateActorIcon();
}

void AWaterLandscapeBrush::ClearActors()
{
	ActorsAffectingLandscape.Empty();
}

void AWaterLandscapeBrush::BeginDestroy()
{
	Super::BeginDestroy();

	if (!IsTemplate())
	{
		ClearActors();

		FWorldDelegates::OnPostWorldInitialization.Remove(OnWorldPostInitHandle);
		OnWorldPostInitHandle.Reset();

		FWorldDelegates::LevelAddedToWorld.Remove(OnLevelAddedToWorldHandle);
		OnLevelAddedToWorldHandle.Reset();

		FWorldDelegates::LevelRemovedFromWorld.Remove(OnLevelRemovedFromWorldHandle);
		OnLevelRemovedFromWorldHandle.Reset();

		GEngine->OnLevelActorAdded().Remove(OnLevelActorAddedHandle);
		OnLevelActorAddedHandle.Reset();

		GEngine->OnLevelActorDeleted().Remove(OnLevelActorDeletedHandle);
		OnLevelActorDeletedHandle.Reset();

		IWaterBrushActorInterface::GetOnWaterBrushActorChangedEvent().RemoveAll(this);
	}
}

void AWaterLandscapeBrush::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

#if WITH_EDITOR
	if (!IsTemplate())
	{
		if (!OnLoadedActorRemovedFromLevelEventHandle.IsValid())
		{
			check(!OnLoadedActorRemovedFromLevelEventHandle.IsValid());

			// In world partition, actors don't belong to levels and the on loaded/removed callbacks are different : 
			// Since these are world events, we register/unregister to it in AWaterLandscapeBrush::RegisterAllComponents() / AWaterLandscapeBrush::UnregisterAllComponents() to make sure that the world is in a valid state when it's called :
			UWorld* World = GetWorld();
			checkf((World != nullptr) && (World->PersistentLevel != nullptr), TEXT("This function should only be called when the world and level are accessible"));
			OnLoadedActorAddedToLevelEventHandle = World->PersistentLevel->OnLoadedActorAddedToLevelEvent.AddLambda([this](AActor& InActor) { OnLevelActorAdded(&InActor); });
			OnLoadedActorRemovedFromLevelEventHandle = World->PersistentLevel->OnLoadedActorRemovedFromLevelEvent.AddLambda([this](AActor& InActor) { OnLevelActorRemoved(&InActor); });
		}

		// It's possible that actors registered to us via OnLoadedActorAddedToLevelEvent were already loaded by the time PostRegisterAllComponents runs, so we need to update our list of actors now : 
		UpdateActors();
	}
#endif // WITH_EDITOR
}

void AWaterLandscapeBrush::UnregisterAllComponents(bool bForReregister)
{
	Super::UnregisterAllComponents(bForReregister);

#if WITH_EDITOR
	if (!IsTemplate())
	{
		if (OnLoadedActorAddedToLevelEventHandle.IsValid())
		{
			check(OnLoadedActorRemovedFromLevelEventHandle.IsValid());

			// UnregisterAllComponents can be called when the world if getting GCed. By this time, the world won't be able to broadcast these events so it's fine if we don't unregister from them :
		if (UWorld* World = GetWorld())
		{
				// Since these are world events, we register/unregister to it in AWaterLandscapeBrush::RegisterAllComponents() / AWaterLandscapeBrush::UnregisterAllComponents() to make sure that the world is in a valid state when it's called :
				checkf(World->PersistentLevel != nullptr, TEXT("This function should only be called when the world and level are accessible"));
				World->PersistentLevel->OnLoadedActorAddedToLevelEvent.Remove(OnLoadedActorAddedToLevelEventHandle);
				World->PersistentLevel->OnLoadedActorRemovedFromLevelEvent.Remove(OnLoadedActorRemovedFromLevelEventHandle);
			}
		OnLoadedActorAddedToLevelEventHandle.Reset();
		OnLoadedActorRemovedFromLevelEventHandle.Reset();
		}
	}
#endif // WITH_EDITOR
}

void AWaterLandscapeBrush::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	AWaterLandscapeBrush* This = CastChecked<AWaterLandscapeBrush>(InThis);
	Super::AddReferencedObjects(This, Collector);

	// TODO [jonathan.bard] : remove : probably not necessary since it's now a uproperty :
	for (TPair<TSoftObjectPtr<AActor>, TObjectPtr<UObject>>& Pair : This->Cache)
	{
		Collector.AddReferencedObject(Pair.Value);
	}
}

#if WITH_EDITOR
TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> AWaterLandscapeBrush::GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const
{
	using namespace UE::Landscape::EditLayers; 

	TArray<FEditLayerRenderItem> RenderItems = Super::GetRenderItems(InMergeContext);
	check(RenderItems.Num() == 1);

	// For now, this brush requires the entire landscape to be loaded to work deterministically, so we force the input area to be "infinite" : 
	RenderItems[0].SetInputWorldArea(FInputWorldArea::CreateInfinite());

	return RenderItems;
}
#endif // WITH_EDITOR

void AWaterLandscapeBrush::GetWaterBodies(TSubclassOf<AWaterBody> WaterBodyClass, TArray<AWaterBody*>& OutWaterBodies) const
{
	FGetActorsOfType<AWaterBody>()(this, WaterBodyClass, OutWaterBodies);
}

void AWaterLandscapeBrush::GetWaterBodyIslands(TSubclassOf<AWaterBodyIsland> WaterBodyIslandClass, TArray<AWaterBodyIsland*>& OutWaterBodyIslands) const
{
	FGetActorsOfType<AWaterBodyIsland>()(this, WaterBodyIslandClass, OutWaterBodyIslands);
}

void AWaterLandscapeBrush::GetActorsAffectingLandscape(TArray<TScriptInterface<IWaterBrushActorInterface>>& OutWaterBrushActors) const
{
	OutWaterBrushActors.Reserve(ActorsAffectingLandscape.Num());
	Algo::TransformIf(ActorsAffectingLandscape, OutWaterBrushActors,
		[](const TWeakInterfacePtr<IWaterBrushActorInterface>& WeakPtr) { return WeakPtr.IsValid(); },
		[](const TWeakInterfacePtr<IWaterBrushActorInterface>& WeakPtr) { return WeakPtr.ToScriptInterface(); });
}

void AWaterLandscapeBrush::BlueprintWaterBodyChanged_Implementation(AActor* Actor)
{
	BlueprintWaterBodyChanged_Native(Actor);
}

void AWaterLandscapeBrush::SetActorCache(AActor* InActor, UObject* InCache)
{
	if (!InCache)
	{
		return;
	}

	TObjectPtr<UObject>& Value = Cache.FindOrAdd(TSoftObjectPtr<AActor>(InActor));
	Value = InCache;
}


UObject* AWaterLandscapeBrush::GetActorCache(AActor* InActor, TSubclassOf<UObject> CacheClass) const
{
	TObjectPtr<UObject> const* ValuePtr = Cache.Find(TSoftObjectPtr<AActor>(InActor));
	if (ValuePtr && (*ValuePtr) && (*ValuePtr)->IsA(*CacheClass))
	{
		return *ValuePtr;
	}
	return nullptr;
}

void AWaterLandscapeBrush::ClearActorCache(AActor* InActor)
{
	Cache.Remove(TSoftObjectPtr<AActor>(InActor));
}

void AWaterLandscapeBrush::BlueprintGetRenderTargets_Implementation(UTextureRenderTarget2D* InHeightRenderTarget, UTextureRenderTarget2D*& OutVelocityRenderTarget)
{
	// Deprecated
}

void AWaterLandscapeBrush::SetTargetLandscape(ALandscape* InTargetLandscape)
{
	if (OwningLandscape != InTargetLandscape)
	{
		if (OwningLandscape)
		{
			OwningLandscape->RemoveBrush(this);
		}

		if (InTargetLandscape && !InTargetLandscape->IsTemplate())
		{
			static const FName WaterLayerName = FName("Water");
			
			ILandscapeModule& LandscapeModule = FModuleManager::GetModuleChecked<ILandscapeModule>("Landscape");
			int32 WaterLayerIndex = LandscapeModule.GetLandscapeEditorServices()->GetOrCreateEditLayer(WaterLayerName, InTargetLandscape);
			
			InTargetLandscape->AddBrushToLayer(WaterLayerIndex, this);
		}
	}

#if WITH_EDITOR
	UpdateActorIcon();
#endif // WITH_EDITOR
}

void AWaterLandscapeBrush::OnEditLayersMerged(const FOnLandscapeEditLayersMergedParams& InParams)
{
	if (!InParams.bIsHeightmapMerge)
	{
		return;
	}

	// #todo_water [roey]: This needs to be changed when the WaterZone can maintain it's own list of "ground actors" so that we don't needlessly update all water zones.
	if (UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(GetWorld()))
	{
		WaterSubsystem->MarkAllWaterZonesForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture, /* DebugRequestingObject = */ this);
	}
}

void AWaterLandscapeBrush::SetOwningLandscape(ALandscape* InOwningLandscape)
{
	if (OwningLandscape != nullptr)
	{
		OwningLandscape->OnEditLayersMerged().RemoveAll(this);
	}

	Super::SetOwningLandscape(InOwningLandscape);

	if (OwningLandscape != nullptr)
	{
		OwningLandscape->OnEditLayersMerged().AddUObject(this, &AWaterLandscapeBrush::OnEditLayersMerged);
	}
}

void AWaterLandscapeBrush::GetRenderDependencies(TSet<UObject*>& OutDependencies)
{
	Super::GetRenderDependencies(OutDependencies);

	for (const TWeakInterfacePtr<IWaterBrushActorInterface>& WaterBrushActor : ActorsAffectingLandscape)
	{
		if (WaterBrushActor.IsValid())
		{
			WaterBrushActor->GetBrushRenderDependencies(OutDependencies);
		}
	}
}

void AWaterLandscapeBrush::ForceUpdate()
{
	OnActorsAffectingLandscapeChanged();
}


void AWaterLandscapeBrush::BlueprintOnRenderTargetTexturesUpdated_Implementation(UTexture2D* VelocityTexture)
{
	// Deprecated
}

#if WITH_EDITOR

AWaterLandscapeBrush::EWaterBrushStatus AWaterLandscapeBrush::CheckWaterBrushStatus()
{
	if (GetWorld() && !IsTemplate())
	{
		ALandscape* Landscape = GetOwningLandscape();
		if (Landscape == nullptr)
		{
			return EWaterBrushStatus::MissingLandscapeWithEditLayers;
		}

		if (Landscape->GetBrushLayer(this) == INDEX_NONE)
		{
			return EWaterBrushStatus::MissingFromLandscapeEditLayers;
		}
	}

	return EWaterBrushStatus::Valid;
}

void AWaterLandscapeBrush::CheckForErrors()
{
	Super::CheckForErrors();

	switch (CheckWaterBrushStatus())
	{
	case EWaterBrushStatus::MissingLandscapeWithEditLayers:
		FMessageLog("MapCheck").Error()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_NoLandscape", "The water brush requires an edit layer Landscape.")))
			->AddToken(FMapErrorToken::Create(TEXT("WaterBrushNoLandscape")));
	case EWaterBrushStatus::MissingFromLandscapeEditLayers:
		FMessageLog("MapCheck").Error()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_MissingFromLandscapeEditLayers", "The water brush is missing from the owning landscape edit layers.")))
			->AddToken(FMapErrorToken::Create(TEXT("WaterBrushMissingFromLandscapeEditLayers")));
		break;
	}
}

void AWaterLandscapeBrush::UpdateActorIcon()
{
	if (ActorIcon && !bIsEditorPreviewActor)
	{
		UTexture2D* IconTexture = ActorIcon->Sprite;
		IWaterModuleInterface& WaterModule = FModuleManager::GetModuleChecked<IWaterModuleInterface>("Water");
		if (const IWaterEditorServices* WaterEditorServices = WaterModule.GetWaterEditorServices())
		{
			if (CheckWaterBrushStatus() != EWaterBrushStatus::Valid)
			{
				IconTexture = WaterEditorServices->GetErrorSprite();
			}
			else
			{
				IconTexture = WaterEditorServices->GetWaterActorSprite(GetClass());
			}
		}

		FWaterIconHelper::UpdateSpriteComponent(this, IconTexture);
	}
}

bool AWaterLandscapeBrush::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	// Weightmap layers are automatically populated by the list of IWaterBrushActorInterface affecting this brush :
	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(AWaterLandscapeBrush, AffectedWeightmapLayers))
	{
		return false;
	}

	return true;
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

