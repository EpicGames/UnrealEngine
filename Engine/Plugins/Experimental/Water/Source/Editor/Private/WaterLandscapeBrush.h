// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LandscapeBlueprintBrush.h"
#include "WaterBrushActorInterface.h"
#include "WaterLandscapeBrush.generated.h"

class ALandscape;
class UBillboardComponent;
class UTexture2D;
template <class T> struct TWeakInterfacePtr;
template <typename InterfaceType> class TScriptInterface;

class AWaterBody;
class AWaterBodyIsland;

UCLASS(Blueprintable, Abstract, hidecategories = (Collision, Replication, Input, LOD, Actor, Cooking, Rendering))
class AWaterLandscapeBrush : public ALandscapeBlueprintBrush
{
	GENERATED_BODY()

public:
	AWaterLandscapeBrush(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual void OnConstruction(const FTransform& Transform) override;
	virtual void BeginDestroy() override;
	virtual void PostRegisterAllComponents() override;
	virtual void UnregisterAllComponents(bool bForReregister) override;
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

#if WITH_EDITOR
	//~ Begin ILandscapeEditLayerRenderer implementation
	virtual TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override;
	//~ End ILandscapeEditLayerRenderer implementation
#endif // WITH_EDITOR

	UFUNCTION(BlueprintCallable, Category = "Water", meta = (DeterminesOutputType = "WaterBodyClass", DynamicOutputParam = "OutWaterBodies"))
	void GetWaterBodies(TSubclassOf<AWaterBody> WaterBodyClass, TArray<AWaterBody*>& OutWaterBodies) const;

	UFUNCTION(BlueprintCallable, Category = "Water", meta = (DeterminesOutputType = "WaterBodyIslandClass", DynamicOutputParam = "OutWaterBodyIslands"))
	void GetWaterBodyIslands(TSubclassOf<AWaterBodyIsland> WaterBodyIslandClass, TArray<AWaterBodyIsland*>& OutWaterBodyIslands) const;

	UFUNCTION(BlueprintCallable, Category = "Water")
	void GetActorsAffectingLandscape(TArray<TScriptInterface<IWaterBrushActorInterface>>& OutWaterBrushActors) const;

	const TArray<TWeakInterfacePtr<IWaterBrushActorInterface>>& GetActorsAffectingLandscape() const { return ActorsAffectingLandscape; }

	UFUNCTION(BlueprintNativeEvent, meta = (CallInEditor = "true", Category = "Debug"))
	void BlueprintWaterBodiesChanged();
	virtual void BlueprintWaterBodiesChanged_Native() {}

	UFUNCTION(BlueprintNativeEvent, meta = (CallInEditor = "true", Category = "Debug"))
	void BlueprintWaterBodyChanged(AActor* Actor);
	virtual void BlueprintWaterBodyChanged_Native(AActor* Actor) {}

	UE_DEPRECATED(all, "Use SetActorCache instead")
	UFUNCTION(BlueprintCallable, Category = "Cache", meta = (DeprecatedFunction, DeprecationMessage="Use SetActorCache instead"))
	void SetWaterBodyCache(AWaterBody* WaterBody, UObject* InCache) {}
	
	UE_DEPRECATED(all, "Use GetActorCache instead")
	UFUNCTION(BlueprintCallable, Category = "Cache", meta = (DeterminesOutputType = "CacheClass", DeprecatedFunction, DeprecationMessage = "Use GetActorCache instead"))
	UObject* GetWaterBodyCache(AWaterBody* WaterBody, TSubclassOf<UObject> CacheClass) const { return nullptr; }

	UE_DEPRECATED(all, "Use ClearActorCache instead")
	UFUNCTION(BlueprintCallable, Category = "Cache", meta = (DeprecatedFunction, DeprecationMessage = "Use ClearActorCache instead"))
	void ClearWaterBodyCache(AWaterBody* WaterBody) {}

	UFUNCTION(BlueprintCallable, Category = "Cache")
	void SetActorCache(AActor* InActor, UObject* InCache);

	UFUNCTION(BlueprintCallable, Category = "Cache", meta = (DeterminesOutputType = "CacheClass"))
	UObject* GetActorCache(AActor* InActor, TSubclassOf<UObject> CacheClass) const;

	UFUNCTION(BlueprintCallable, Category = "Cache")
	void ClearActorCache(AActor* InActor);

	UFUNCTION(BlueprintNativeEvent, meta = (DeprecatedFunction, DeprecationMessage = "This event isn't called anymore, the WaterVelocityTexture is now regenerated at runtime (WaterInfoTexture in AWaterZone)."))
	void BlueprintGetRenderTargets(UTextureRenderTarget2D* InHeightRenderTarget, UTextureRenderTarget2D*& OutVelocityRenderTarget);
	UE_DEPRECATED(all, "This function isn't called anymore, the WaterVelocityTexture is now regenerated at runtime (WaterInfoTexture in AWaterZone)")
	virtual void BlueprintGetRenderTargets_Native(UTextureRenderTarget2D* InHeightRenderTarget, UTextureRenderTarget2D*& OutVelocityRenderTarget) {}

	UFUNCTION(BlueprintNativeEvent, meta = (DeprecatedFunction, DeprecationMessage = "This event isn't called anymore, the WaterVelocityTexture is now regenerated at runtime (WaterInfoTexture in AWaterZone)."))
	void BlueprintOnRenderTargetTexturesUpdated(UTexture2D* VelocityTexture);
	UE_DEPRECATED(all, "This function isn't called anymore, the WaterVelocityTexture is now regenerated at runtime (WaterInfoTexture in AWaterZone)")
	virtual void BlueprintOnRenderTargetTexturesUpdated_Native(UTexture2D* VelocityTexture) {}

	UE_DEPRECATED(all, "This function is now useless, the WaterVelocityTexture is now regenerated at runtime (WaterInfoTexture in AWaterZone).")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now useless, the WaterVelocityTexture is now regenerated at runtime (WaterInfoTexture in AWaterZone)."))
	void ForceWaterTextureUpdate() {}

	void SetTargetLandscape(ALandscape* InTargetLandscape);
		
	virtual void SetOwningLandscape(ALandscape* InOwningLandscape) override;

	virtual void GetRenderDependencies(TSet<UObject *>& OutDependencies) override;

	virtual void PostLoad() override;
	virtual void PostActorCreated() override;

	void ForceUpdate();

#if WITH_EDITOR
	enum class EWaterBrushStatus : uint8
	{
		Valid,
		MissingLandscapeWithEditLayers,
		MissingFromLandscapeEditLayers
	};

	EWaterBrushStatus CheckWaterBrushStatus();

	virtual void CheckForErrors() override;
	
	void UpdateActorIcon();

	virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif // WITH_EDITOR

private:
	template<class T>
	friend class FGetActorsOfType;

	void AddActorInternal(AActor* Actor, const UWorld* ThisWorld, UObject* InCache, bool bTriggerEvent, bool bModify);
	void RemoveActorInternal(AActor* Actor);
	void UpdateActors(bool bTriggerEvents = true);
	void UpdateAffectedWeightmaps();
	void ClearActors();
	bool IsActorAffectingLandscape(AActor* Actor) const;
	void RegisterDelegates();
	void OnEditLayersMerged(const struct FOnLandscapeEditLayersMergedParams& InParams);
	void OnWaterBrushActorChanged(const IWaterBrushActorInterface::FWaterBrushActorChangedEventParams& InParams);
	void OnActorsAffectingLandscapeChanged();
	void OnLevelActorAdded(AActor* InActor);
	void OnLevelActorRemoved(AActor* InActor);

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<UBillboardComponent> ActorIcon;
#endif // WITH_EDITORONLY_DATA

private:
	TArray<TWeakInterfacePtr<IWaterBrushActorInterface>> ActorsAffectingLandscape;
	FDelegateHandle OnWorldPostInitHandle;
	FDelegateHandle OnLevelAddedToWorldHandle;
	FDelegateHandle OnLevelRemovedFromWorldHandle;
	FDelegateHandle OnLevelActorAddedHandle;
	FDelegateHandle OnLevelActorDeletedHandle;
	FDelegateHandle OnActorMovedHandle;
#if WITH_EDITOR
	FDelegateHandle OnLoadedActorAddedToLevelEventHandle;
	FDelegateHandle OnLoadedActorRemovedFromLevelEventHandle;
#endif // WITH_EDITOR

	UPROPERTY(Transient, DuplicateTransient, VisibleAnywhere, AdvancedDisplay, meta = (Category = "Debug"))
	TMap<TSoftObjectPtr<AActor>, TObjectPtr<UObject>> Cache;
};
