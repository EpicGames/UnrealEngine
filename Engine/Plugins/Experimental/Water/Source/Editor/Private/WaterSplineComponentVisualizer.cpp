// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterSplineComponentVisualizer.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "EditorViewportClient.h"
#include "ScopedTransaction.h"
#include "WaterEditorSettings.h"
#include "WaterSplineComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterSplineComponentVisualizer)

IMPLEMENT_HIT_PROXY(HWaterSplineVisProxy, HComponentVisProxy);
IMPLEMENT_HIT_PROXY(HWaterSplineKeyProxy, HWaterSplineVisProxy);
IMPLEMENT_HIT_PROXY(HWaterSplineWaterVelocityProxy, HWaterSplineKeyProxy);
IMPLEMENT_HIT_PROXY(HWaterSplineRiverWidthProxy, HWaterSplineKeyProxy);
IMPLEMENT_HIT_PROXY(HWaterSplineDepthProxy, HWaterSplineKeyProxy);

#define LOCTEXT_NAMESPACE "WaterSplineComponentVisualizer"
DEFINE_LOG_CATEGORY_STATIC(LogWaterSplineComponentVisualizer, Log, All)

#define VISUALIZE_SPLINE_UPVECTORS 0

/** Define commands for the spline component visualizer */
class FWaterSplineComponentVisualizerCommands : public TCommands<FWaterSplineComponentVisualizerCommands>
{
public:
	FWaterSplineComponentVisualizerCommands() : TCommands <FWaterSplineComponentVisualizerCommands>
		(
			"WaterSplineComponentVisualizer",	// Context name for fast lookup
			LOCTEXT("WaterSplineComponentVisualizer", "WaterSpline Component Visualizer"),	// Localized context name for displaying
			NAME_None,	// Parent
			FAppStyle::GetAppStyleSetName()
			)
	{
	}

	virtual void RegisterCommands() override
	{
		UI_COMMAND(VisualizeWaterVelocity, "Visualize Water Velocity", "Whether the visualization should show water velocity on this spline.", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(VisualizeRiverWidth, "Visualize River Width", "Whether the visualization should show river width on this spline.", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(VisualizeDepth, "Visualize Depth", "Whether the visualization should show water depth on this spline.", EUserInterfaceActionType::ToggleButton, FInputChord());
	}

public:

	/** Whether this spline should visualize water velocity */
	TSharedPtr<FUICommandInfo> VisualizeWaterVelocity;

	/** Whether this spline should visualize river width */
	TSharedPtr<FUICommandInfo> VisualizeRiverWidth;

	/** Whether this spline should visualize depth */
	TSharedPtr<FUICommandInfo> VisualizeDepth;
};

FWaterSplineComponentVisualizer::FWaterSplineComponentVisualizer()
	: FSplineComponentVisualizer()
{
	FWaterSplineComponentVisualizerCommands::Register();

	WaterSplineComponentVisualizerActions = MakeShareable(new FUICommandList);

	SelectionState = NewObject<UWaterSplineComponentVisualizerSelectionState>(GetTransientPackage(), TEXT("WaterSelectionState"), RF_Transactional);
}

void FWaterSplineComponentVisualizer::OnRegister()
{
	FSplineComponentVisualizer::OnRegister();

	const auto& Commands = FWaterSplineComponentVisualizerCommands::Get();

	SplineComponentVisualizerActions->MapAction(
		Commands.VisualizeWaterVelocity,
		FExecuteAction::CreateSP(this, &FWaterSplineComponentVisualizer::OnSetVisualizeWaterVelocity),
		FCanExecuteAction::CreateSP(this, &FWaterSplineComponentVisualizer::CanSetVisualizeWaterVelocity),
		FIsActionChecked::CreateSP(this, &FWaterSplineComponentVisualizer::IsVisualizingWaterVelocity));

	SplineComponentVisualizerActions->MapAction(
		Commands.VisualizeRiverWidth,
		FExecuteAction::CreateSP(this, &FWaterSplineComponentVisualizer::OnSetVisualizeRiverWidth),
		FCanExecuteAction::CreateSP(this, &FWaterSplineComponentVisualizer::CanSetVisualizeRiverWidth),
		FIsActionChecked::CreateSP(this, &FWaterSplineComponentVisualizer::IsVisualizingRiverWidth));

	SplineComponentVisualizerActions->MapAction(
		Commands.VisualizeDepth,
		FExecuteAction::CreateSP(this, &FWaterSplineComponentVisualizer::OnSetVisualizeDepth),
		FCanExecuteAction::CreateSP(this, &FWaterSplineComponentVisualizer::CanSetVisualizeDepth),
		FIsActionChecked::CreateSP(this, &FWaterSplineComponentVisualizer::IsVisualizingDepth));
}

FWaterSplineComponentVisualizer::~FWaterSplineComponentVisualizer()
{
	FWaterSplineComponentVisualizerCommands::Unregister();
}

void FWaterSplineComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	FSplineComponentVisualizer::DrawVisualization(Component, View, PDI);

	const UWaterSplineComponentVisualizerSelectionState* WaterSelectionState = CastChecked<const UWaterSplineComponentVisualizerSelectionState>(SelectionState);
	check(WaterSelectionState);

	if (const UWaterSplineComponent* WaterSplineComp = Cast<const UWaterSplineComponent>(Component))
	{
		if (WaterSplineComp == GetEditedWaterSplineComponent())
		{
			if (const UWaterSplineMetadata* Metadata = Cast<const UWaterSplineMetadata>(WaterSplineComp->GetSplinePointsMetadata()))
			{
				if (Metadata->bShouldVisualizeWaterVelocity || Metadata->bShouldVisualizeRiverWidth || Metadata->bShouldVisualizeDepth)
				{
					static const float GrabHandleSize = 10.0f;
					static const float ArrowOffset = 300.0f;
					static const float ArrowHeadForward = -100.0f;
					static const float ArrowHeadSide = 50.0f;

					const bool bIsSplineEditable = !WaterSplineComp->bModifiedByConstructionScript; // bSplineHasBeenEdited || SplineInfo == Archetype->SplineCurves.Position || SplineComp->bInputSplinePointsToConstructionScript;

					const FColor ReadOnlyColor = FColor(255, 0, 255, 255);
					const FColor NormalColor = bIsSplineEditable ? FColor(WaterSplineComp->EditorUnselectedSplineSegmentColor.ToFColor(true)) : ReadOnlyColor;
					const FColor SelectedColor = bIsSplineEditable ? FColor(WaterSplineComp->EditorSelectedSplineSegmentColor.ToFColor(true)) : ReadOnlyColor;

					const FInterpCurveVector& SplinePosition = WaterSplineComp->GetSplinePointsPosition();

					for (int32 KeyIdx = 0; KeyIdx < SplinePosition.Points.Num(); KeyIdx++)
					{
						const FTransform SplineLocalToWorld = WaterSplineComp->GetComponentTransform();
						const FVector KeyPos = SplinePosition.Points[KeyIdx].OutVal;
						const FVector RightVector = WaterSplineComp->GetRightVectorAtSplinePoint(KeyIdx, ESplineCoordinateSpace::Local);
						const FVector ForwardVector = SplinePosition.Points[KeyIdx].LeaveTangent.GetSafeNormal();
						const FVector WorldKeyPos = SplineLocalToWorld.TransformPosition(KeyPos);

						bool bKeyIsSelected = WaterSelectionState->GetSelectedKeys().Contains(KeyIdx);

						if (Metadata->bShouldVisualizeWaterVelocity)
						{
							const FColor WaterVelocityColor = bKeyIsSelected && WaterSelectionState->GetWaterVelocityIsSelected() ? SelectedColor : NormalColor;

							const float HandleLength = Metadata->WaterVelocityScalar.Points[KeyIdx].OutVal * GetDefault<UWaterEditorSettings>()->VisualizeWaterVelocityScale;
							const float Dir = HandleLength < 0.0f ? -1.0f : 1.0f;

							const float HalfRiverWidth = Metadata->RiverWidth.Points[KeyIdx].OutVal * 0.5f;
							int NumArrowsPerSide = HalfRiverWidth < ArrowOffset ? 0 : 1;

							FVector EndPos = KeyPos + ForwardVector * HandleLength;
							FVector WorldMidPos = SplineLocalToWorld.TransformPosition(KeyPos + ForwardVector * HandleLength * 0.5f);
							FVector WorldEndPos = SplineLocalToWorld.TransformPosition(EndPos);
							FVector WorldArrowLeftPos = SplineLocalToWorld.TransformPosition(EndPos + ForwardVector * ArrowHeadForward * Dir + RightVector * ArrowHeadSide);
							FVector WorldArrowRightPos = SplineLocalToWorld.TransformPosition(EndPos + ForwardVector * ArrowHeadForward * Dir - RightVector * ArrowHeadSide);

							auto DrawArrowFunc = [&](const FVector& WorldOffset)
							{
								// Draw arrow base closest to key pos but don't include in the hit proxy
								PDI->SetHitProxy(new HWaterSplineWaterVelocityProxy(WaterSplineComp, KeyIdx));
								PDI->DrawLine(WorldMidPos + WorldOffset, WorldEndPos + WorldOffset, WaterVelocityColor, SDPG_Foreground, 0.0f);
								PDI->DrawLine(WorldEndPos + WorldOffset, WorldArrowLeftPos + WorldOffset, WaterVelocityColor, SDPG_Foreground, 0.0f);
								PDI->DrawLine(WorldEndPos + WorldOffset, WorldArrowRightPos + WorldOffset, WaterVelocityColor, SDPG_Foreground, 0.0f);

								PDI->SetHitProxy(NULL);
								PDI->DrawLine(WorldMidPos + WorldOffset, WorldKeyPos + WorldOffset, WaterVelocityColor, SDPG_Foreground, 0.0f);
							};

							// Draw arrows
							for (int i = 0; i <= NumArrowsPerSide; i++)
							{
								FVector ShiftArrow = SplineLocalToWorld.TransformVector(RightVector * (i * ArrowOffset));
								DrawArrowFunc(ShiftArrow);

								if (i > 0)
								{
									DrawArrowFunc(ShiftArrow * -1.0f);
								}
							}
						}

						auto DrawHandleFunc = [&](HWaterSplineVisProxy* VisProxy, const FVector& LocalAxis, float HandleLength, float OffsetLength, EHandleType HandleType, bool bHandleIsSelected, bool bPositiveHandleIsSelected)
						{
							FColor HandleColorPos = bKeyIsSelected && bHandleIsSelected && bPositiveHandleIsSelected ? SelectedColor : NormalColor;
							FColor HandleColorNeg = bKeyIsSelected && bHandleIsSelected && !bPositiveHandleIsSelected ? SelectedColor : NormalColor;
							FVector WorldHandleVector = SplineLocalToWorld.TransformVector(LocalAxis * HandleLength);
							FVector WorldOffsetVector = OffsetLength > 0.0f ? SplineLocalToWorld.TransformVector(LocalAxis * OffsetLength) : FVector::ZeroVector;

							PDI->SetHitProxy(VisProxy);
							if (HandleType == EHandleType::Both || HandleType == EHandleType::PositiveAxis)
							{
								PDI->DrawPoint(WorldKeyPos + WorldOffsetVector + WorldHandleVector, HandleColorPos, GrabHandleSize, SDPG_Foreground);
							}
							if (HandleType == EHandleType::Both || HandleType == EHandleType::NegativeAxis)
							{
								PDI->DrawPoint(WorldKeyPos - WorldOffsetVector - WorldHandleVector, HandleColorNeg, GrabHandleSize, SDPG_Foreground);
							}

							PDI->SetHitProxy(NULL);
							if (HandleType == EHandleType::Both || HandleType == EHandleType::PositiveAxis)
							{
								PDI->DrawLine(WorldKeyPos + WorldOffsetVector, WorldKeyPos + WorldOffsetVector + WorldHandleVector, HandleColorPos, SDPG_Foreground, 0.0f);
							}
							if (HandleType == EHandleType::Both || HandleType == EHandleType::NegativeAxis)
							{
								PDI->DrawLine(WorldKeyPos - WorldOffsetVector, WorldKeyPos - WorldOffsetVector - WorldHandleVector, HandleColorNeg, SDPG_Foreground, 0.0f);
							}
						};

						if (Metadata->bShouldVisualizeRiverWidth)
						{
							HWaterSplineRiverWidthProxy* RiverWidthProxy = new HWaterSplineRiverWidthProxy(WaterSplineComp, KeyIdx);
							DrawHandleFunc(RiverWidthProxy, RightVector, Metadata->RiverWidth.Points[KeyIdx].OutVal * 0.5f, 0.0f, EHandleType::Both, 
								WaterSelectionState->GetRiverWidthIsSelected(), WaterSelectionState->GetRiverWidthSelectedPosHandle());
						}

						if (Metadata->bShouldVisualizeDepth)
						{
							const FVector UpVector = WaterSplineComp->GetUpVectorAtSplinePoint(KeyIdx, ESplineCoordinateSpace::Local);
							const bool bDepthIsSelected = WaterSelectionState->GetDepthIsSelected();
							HWaterSplineDepthProxy* DepthProxy = new HWaterSplineDepthProxy(WaterSplineComp, KeyIdx);
							DrawHandleFunc(DepthProxy, UpVector, Metadata->Depth.Points[KeyIdx].OutVal, 0.0f, EHandleType::NegativeAxis, bDepthIsSelected, false);
						}
					}

					PDI->SetHitProxy(NULL);
				}
			}
		}
	}
}

void FWaterSplineComponentVisualizer::ClearSelectionState()
{
	if (UWaterSplineComponentVisualizerSelectionState* WaterSelectionState = CastChecked<UWaterSplineComponentVisualizerSelectionState>(SelectionState))
	{
		WaterSelectionState->SetWaterVelocityIsSelected(false);
		WaterSelectionState->SetDepthIsSelected(false);
		WaterSelectionState->SetRiverWidthIsSelected(false);
	}
}

void FWaterSplineComponentVisualizer::ChangeSelectionState(int32 Index, bool bIsCtrlHeld)
{
	ClearSelectionState();
	FSplineComponentVisualizer::ChangeSelectionState(Index, bIsCtrlHeld);
}

const UWaterSplineComponent* FWaterSplineComponentVisualizer::UpdateSelectedWaterSplineComponent(HComponentVisProxy* VisProxy)
{
	if (const USplineComponent* SplineComp = FSplineComponentVisualizer::UpdateSelectedSplineComponent(VisProxy))
	{
		return CastChecked<UWaterSplineComponent>(SplineComp);
	}
	return nullptr;
}

void FWaterSplineComponentVisualizer::UpdateSelectionState(const int32 InKeyIndex)
{
	ClearSelectionState();

	// Modify the selection state, unless key is already selected
	const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();
	if (SelectedKeys.Num() > 1 || !SelectedKeys.Contains(InKeyIndex))
	{
		ChangeSelectionState(InKeyIndex, false);
	}
}

bool FWaterSplineComponentVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click)
{
	ResetTempModes();

	UWaterSplineComponentVisualizerSelectionState* WaterSelectionState = CastChecked<UWaterSplineComponentVisualizerSelectionState>(SelectionState);
	check(WaterSelectionState);

	if (VisProxy && VisProxy->Component.IsValid())
	{
		if (VisProxy->IsA(HWaterSplineKeyProxy::StaticGetType()))
		{
			HWaterSplineKeyProxy* KeyProxy = (HWaterSplineKeyProxy*)VisProxy;

			if (VisProxy->IsA(HWaterSplineWaterVelocityProxy::StaticGetType()))
			{
				const FScopedTransaction Transaction(LOCTEXT("SelectWaterSplineVelocity", "Select Water Spline Velocity"));

				WaterSelectionState->Modify();
				UpdateSelectionState(KeyProxy->KeyIndex);

				int32 LastKeyIndexSelected = WaterSelectionState->GetLastKeyIndexSelected();
				if (LastKeyIndexSelected == INDEX_NONE)
				{
					WaterSelectionState->SetSplinePropertyPath(FComponentPropertyPath());
					return false;
				}

				if (const UWaterSplineComponent* WaterSplineComp = UpdateSelectedWaterSplineComponent(VisProxy))
				{
					if (const UWaterSplineMetadata* Metadata = Cast<const UWaterSplineMetadata>(WaterSplineComp->GetSplinePointsMetadata()))
					{
						check(LastKeyIndexSelected < Metadata->WaterVelocityScalar.Points.Num());
						const float HandleLength = Metadata->WaterVelocityScalar.Points[LastKeyIndexSelected].OutVal * GetDefault<UWaterEditorSettings>()->VisualizeWaterVelocityScale;
						const FVector ForwardVector = WaterSplineComp->GetSplinePointsPosition().Points[LastKeyIndexSelected].LeaveTangent.GetSafeNormal();
						VisProxyHandleWaterClick(WaterSplineComp, Click, HandleLength, EHandleType::PositiveAxis, ForwardVector, FVector::UpVector);
						WaterSelectionState->SetWaterVelocityIsSelected(true);
						return true;
					}
					return false;
				}
			}
			else if (VisProxy->IsA(HWaterSplineRiverWidthProxy::StaticGetType()))
			{
				const FScopedTransaction Transaction(LOCTEXT("SelectWaterSplineRiverWidth", "Select Water Spline River Width"));

				WaterSelectionState->Modify();
				UpdateSelectionState(KeyProxy->KeyIndex);

				int32 LastKeyIndexSelected = SelectionState->GetLastKeyIndexSelected();
				if (LastKeyIndexSelected == INDEX_NONE)
				{
					WaterSelectionState->SetSplinePropertyPath(FComponentPropertyPath());
					return false;
				}

				if (const UWaterSplineComponent* WaterSplineComp = UpdateSelectedWaterSplineComponent(VisProxy))
				{
					if (const UWaterSplineMetadata* Metadata = Cast<const UWaterSplineMetadata>(WaterSplineComp->GetSplinePointsMetadata()))
					{
						check(LastKeyIndexSelected < Metadata->RiverWidth.Points.Num());
						const float HandleLength = Metadata->RiverWidth.Points[LastKeyIndexSelected].OutVal * 0.5f;
						const FVector RightVector = WaterSplineComp->GetRightVectorAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::Local);
						bool bRiverWidthSelectedPosHandle = VisProxyHandleWaterClick(WaterSplineComp, Click, HandleLength, EHandleType::Both, RightVector, FVector::UpVector);
						WaterSelectionState->SetRiverWidthSelectedPosHandle(bRiverWidthSelectedPosHandle);
						WaterSelectionState->SetRiverWidthIsSelected(true);
						return true;
					}

					return false;
				}
			}
			else if (VisProxy->IsA(HWaterSplineDepthProxy::StaticGetType()))
			{
				const FScopedTransaction Transaction(LOCTEXT("SelectWaterSplineDepth", "Select Water Spline Depth"));

				WaterSelectionState->Modify();
				UpdateSelectionState(KeyProxy->KeyIndex);

				int32 LastKeyIndexSelected = SelectionState->GetLastKeyIndexSelected();
				if (LastKeyIndexSelected == INDEX_NONE)
				{
					WaterSelectionState->SetSplinePropertyPath(FComponentPropertyPath());
					return false;
				}

				if (const UWaterSplineComponent* WaterSplineComp = UpdateSelectedWaterSplineComponent(VisProxy))
				{
					if (const UWaterSplineMetadata* Metadata = Cast<const UWaterSplineMetadata>(WaterSplineComp->GetSplinePointsMetadata()))
					{
						check(LastKeyIndexSelected < Metadata->Depth.Points.Num());
						const float HandleLength = Metadata->Depth.Points[LastKeyIndexSelected].OutVal;
						const FVector LocalAxis = WaterSplineComp->GetUpVectorAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::Local);
						VisProxyHandleWaterClick(WaterSplineComp, Click, HandleLength, EHandleType::NegativeAxis, LocalAxis, FVector::RightVector);
						WaterSelectionState->SetDepthIsSelected(true);
						return true;
					}

					return false;
				}
			}
		}
	}

	return (FSplineComponentVisualizer::VisProxyHandleClick(InViewportClient, VisProxy, Click));
}

bool FWaterSplineComponentVisualizer::VisProxyHandleWaterClick(const UWaterSplineComponent* WaterSplineComp, const FViewportClick& Click, float HandleLength, EHandleType HandleType,
	const FVector& LocalAxis, const FVector& LocalRotAxis)
{
	check(WaterSplineComp);
	check(SelectionState);

	const FInterpCurveVector& SplinePosition = WaterSplineComp->GetSplinePointsPosition();
	const int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(WaterSplineComp->GetNumberOfSplinePoints());

	const FTransform SplineLocalToWorld = WaterSplineComp->GetComponentTransform();
	const FVector LocalHandleVector = LocalAxis * HandleLength;
	const FVector WorldHandleVector = SplineLocalToWorld.TransformVector(LocalHandleVector);
	const FVector WorldKeyPos = SplineLocalToWorld.TransformPosition(SplinePosition.Points[LastKeyIndexSelected].OutVal);
	const FVector WorldHandlePos1 = WorldKeyPos + WorldHandleVector;
	const FVector WorldHandlePos2 = WorldKeyPos - WorldHandleVector;

	if (HandleType == EHandleType::Both)
	{
		FVector HandleClosest;
		FVector RayClosest;
		FMath::SegmentDistToSegmentSafe(WorldHandlePos1, WorldHandlePos2, Click.GetOrigin(), Click.GetOrigin() + Click.GetDirection() * 50000.0f, HandleClosest, RayClosest);

		if (FVector::Distance(WorldHandlePos1, HandleClosest) < FVector::Distance(WorldHandlePos2, HandleClosest))
		{
			HandleType = EHandleType::PositiveAxis;
		}
		else
		{
			HandleType = EHandleType::NegativeAxis;
		}
	}

	FQuat CachedRotation;
	if (HandleType == EHandleType::PositiveAxis)
	{
		CachedRotation = WaterSplineComp->GetQuaternionAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::World);
	}
	else
	{
		const FVector WorldRotAxis = WaterSplineComp->GetTransformAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::World, false).TransformVector(LocalRotAxis);
		CachedRotation = FQuat(WorldRotAxis, PI) * WaterSplineComp->GetQuaternionAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::World);
	}
	SelectionState->Modify();
	SelectionState->SetCachedRotation(CachedRotation);

	return (HandleType == EHandleType::PositiveAxis);
}

UWaterSplineComponent* FWaterSplineComponentVisualizer::GetEditedWaterSplineComponent() const
{
	check(SelectionState != nullptr);
	return Cast<UWaterSplineComponent>(SelectionState->GetSplinePropertyPath().GetComponent());
}

bool FWaterSplineComponentVisualizer::GetWidgetLocation(const FEditorViewportClient* ViewportClient, FVector& OutLocation) const
{
	UWaterSplineComponentVisualizerSelectionState* WaterSelectionState = CastChecked<UWaterSplineComponentVisualizerSelectionState>(SelectionState);
	check(WaterSelectionState);

	if (const UWaterSplineComponent* WaterSplineComp = GetEditedWaterSplineComponent())
	{
		if (const UWaterSplineMetadata* Metadata = Cast<UWaterSplineMetadata>(WaterSplineComp->GetSplinePointsMetadata()))
		{
			bool bWaterVelocityIsSelected = WaterSelectionState->GetWaterVelocityIsSelected();
			bool bDepthIsSelected = WaterSelectionState->GetDepthIsSelected();
			bool bRiverWidthIsSelected = WaterSelectionState->GetRiverWidthIsSelected();

			if (bWaterVelocityIsSelected || bDepthIsSelected || bRiverWidthIsSelected)
			{
				int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(WaterSplineComp->GetNumberOfSplinePoints());

				const FInterpCurveVector& SplinePosition = WaterSplineComp->GetSplinePointsPosition();
				FVector KeyPos = SplinePosition.Points[LastKeyIndexSelected].OutVal;

				if (bWaterVelocityIsSelected)
				{
					const FVector ForwardVector = SplinePosition.Points[LastKeyIndexSelected].LeaveTangent.GetSafeNormal();
					float Length = Metadata->WaterVelocityScalar.Points[LastKeyIndexSelected].OutVal * GetDefault<UWaterEditorSettings>()->VisualizeWaterVelocityScale;
					OutLocation = WaterSplineComp->GetComponentTransform().TransformPosition(KeyPos + ForwardVector * Length);
				}
				else if (bRiverWidthIsSelected)
				{
					const FVector RightVector = WaterSplineComp->GetRightVectorAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::Local);
					float Length = Metadata->RiverWidth.Points[LastKeyIndexSelected].OutVal * (WaterSelectionState->GetRiverWidthSelectedPosHandle() ? 0.5f : -0.5f);
					OutLocation = WaterSplineComp->GetComponentTransform().TransformPosition(KeyPos + RightVector * Length);
				}
				else if (bDepthIsSelected)
				{
					const FVector UpVector = WaterSplineComp->GetUpVectorAtSplinePoint(LastKeyIndexSelected, ESplineCoordinateSpace::Local);
					float Length = Metadata->Depth.Points[LastKeyIndexSelected].OutVal;
					OutLocation = WaterSplineComp->GetComponentTransform().TransformPosition(KeyPos - UpVector * Length);
				}
				return true;
			}
		}
	}

	return FSplineComponentVisualizer::GetWidgetLocation(ViewportClient, OutLocation);
}

float FWaterSplineComponentVisualizer::ComputeDelta(UWaterSplineComponent* WaterSplineComp, const FVector& InDeltaTranslate, float InCurrentHandleLength, const FVector& InAxis,
	float InScale, bool bClampToZero)
{
	check(WaterSplineComp);
	check(SelectionState != nullptr);

	const int32 NumPoints = WaterSplineComp->GetNumberOfSplinePoints();
	int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(NumPoints);

	const FTransform SplinePointToComponent = WaterSplineComp->GetTransformAtSplineInputKey(LastKeyIndexSelected, ESplineCoordinateSpace::Local, true);
	const FTransform SplinePointToWorld = WaterSplineComp->GetTransformAtSplineInputKey(LastKeyIndexSelected, ESplineCoordinateSpace::World, true);

	// convert delta translate to spline pos space
	FVector DeltaTranslateSP = SplinePointToWorld.InverseTransformVector(InDeltaTranslate);
	DeltaTranslateSP = DeltaTranslateSP * InAxis;
	FVector DeltaTranslateLocal = SplinePointToComponent.TransformVector(DeltaTranslateSP);

	float DeltaSizeSP = DeltaTranslateSP.Size();
	float DeltaDir = (FVector::DotProduct(DeltaTranslateSP, InAxis) / DeltaSizeSP < 0.0f) ? -1.0f : 1.0f;
	float DeltaResult = DeltaTranslateLocal.Size() * DeltaDir / InScale;
	if (bClampToZero && InCurrentHandleLength + DeltaResult < 0)
	{
		DeltaResult = -1.0f * InCurrentHandleLength;
	}

	return DeltaResult;
}

bool FWaterSplineComponentVisualizer::HandleInputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& DeltaTranslate, FRotator& DeltaRotate, FVector& DeltaScale)
{
	UWaterSplineComponentVisualizerSelectionState* WaterSelectionState = CastChecked<UWaterSplineComponentVisualizerSelectionState>(SelectionState);
	check(WaterSelectionState);

	const bool bWaterVelocityIsSelected = WaterSelectionState->GetWaterVelocityIsSelected();
	const bool bDepthIsSelected = WaterSelectionState->GetDepthIsSelected();
	const bool bRiverWidthIsSelected = WaterSelectionState->GetRiverWidthIsSelected();

	if (bWaterVelocityIsSelected || bDepthIsSelected || bRiverWidthIsSelected)
	{
		if (UWaterSplineComponent* WaterSplineComp = GetEditedWaterSplineComponent())
		{
			if (UWaterSplineMetadata* Metadata = Cast<UWaterSplineMetadata>(WaterSplineComp->GetSplinePointsMetadata()))
			{
				if (!DeltaTranslate.IsZero())
				{
					int32 LastKeyIndexSelected = SelectionState->GetVerifiedLastKeyIndexSelected(WaterSplineComp->GetNumberOfSplinePoints());
					const TSet<int32>& SelectedKeys = SelectionState->GetSelectedKeys();

					float DeltaOffset = 0.0f;

					if (bWaterVelocityIsSelected)
					{
						DeltaOffset = ComputeDelta(WaterSplineComp, DeltaTranslate, Metadata->WaterVelocityScalar.Points[LastKeyIndexSelected].OutVal, FVector::ForwardVector, GetDefault<UWaterEditorSettings>()->VisualizeWaterVelocityScale, false);
						if (FMath::Abs(DeltaOffset) > SMALL_NUMBER)
						{
							const FScopedTransaction Transaction(LOCTEXT("SetSplinePointWaterVelocity", "Set spline point water velocity"));
							Metadata->Modify();
							for (int32 KeyIndex : SelectedKeys)
							{
								check(KeyIndex != INDEX_NONE);
								check(KeyIndex >= 0);
								check(KeyIndex < WaterSplineComp->GetNumberOfSplinePoints());
								Metadata->WaterVelocityScalar.Points[KeyIndex].OutVal += DeltaOffset;
							}
						}
					}
					else if (bRiverWidthIsSelected)
					{
						float ScaleValue = (WaterSelectionState->GetRiverWidthSelectedPosHandle() ? 0.5f : -0.5f);
						DeltaOffset = ComputeDelta(WaterSplineComp, DeltaTranslate, Metadata->RiverWidth.Points[LastKeyIndexSelected].OutVal, FVector::RightVector, ScaleValue, true);
						if (FMath::Abs(DeltaOffset) > SMALL_NUMBER)
						{
							const FScopedTransaction Transaction(LOCTEXT("SetSplinePointRiverWidth", "Set spline point river width"));
							Metadata->Modify();
							for (int32 KeyIndex : SelectedKeys)
							{
								check(KeyIndex != INDEX_NONE);
								check(KeyIndex >= 0);
								check(KeyIndex < WaterSplineComp->GetNumberOfSplinePoints());
								Metadata->RiverWidth.Points[KeyIndex].OutVal += DeltaOffset;
							}
						}
					}
					else if (bDepthIsSelected)
					{
						DeltaOffset = ComputeDelta(WaterSplineComp, DeltaTranslate, Metadata->Depth.Points[LastKeyIndexSelected].OutVal, FVector::UpVector, -1.0f, true);
						if (FMath::Abs(DeltaOffset) > SMALL_NUMBER)
						{
							const FScopedTransaction Transaction(LOCTEXT("SetSplinePointWaterDepth", "Set spline point water depth"));
							Metadata->Modify();
							for (int32 KeyIndex : SelectedKeys)
							{
								check(KeyIndex != INDEX_NONE);
								check(KeyIndex >= 0);
								check(KeyIndex < WaterSplineComp->GetNumberOfSplinePoints());
								Metadata->Depth.Points[KeyIndex].OutVal += DeltaOffset;
							}
						}
					}

					WaterSplineComp->UpdateSpline();
					WaterSplineComp->bSplineHasBeenEdited = true;

					// Transform the spline keys using an EPropertyChangeType::Interactive change. Later on, at the end of mouse tracking, a non-interactive change will be notified via void TrackingStopped :
					NotifyPropertiesModified(WaterSplineComp, SplineProperties, EPropertyChangeType::Interactive);

					GEditor->RedrawLevelEditingViewports(true);

					return true;
				}
			}
		}
		return false;
	}

	return FSplineComponentVisualizer::HandleInputDelta(ViewportClient, Viewport, DeltaTranslate, DeltaRotate, DeltaScale);
}

void FWaterSplineComponentVisualizer::OnSetVisualizeWaterVelocity()
{
	check(GetEditedWaterSplineComponent());
	if (UWaterSplineMetadata* Metadata = GetEditedWaterSplineMetaData())
	{
		Metadata->bShouldVisualizeWaterVelocity = !Metadata->bShouldVisualizeWaterVelocity;
		GEditor->RedrawLevelEditingViewports(true);
	}
}

UWaterSplineMetadata* FWaterSplineComponentVisualizer::GetEditedWaterSplineMetaData() const
{
	UWaterSplineComponent* WaterSplineComp = GetEditedWaterSplineComponent();
	return WaterSplineComp ? Cast<UWaterSplineMetadata>(WaterSplineComp->GetSplinePointsMetadata()) : nullptr;
}

bool FWaterSplineComponentVisualizer::CanSetVisualizeWaterVelocity() const
{
	const UWaterSplineMetadata* Metadata = GetEditedWaterSplineMetaData();
	return Metadata && Metadata->CanEditVelocity();
}

bool FWaterSplineComponentVisualizer::IsVisualizingWaterVelocity() const
{
	const UWaterSplineMetadata* Metadata = GetEditedWaterSplineMetaData();
	return Metadata && Metadata->bShouldVisualizeWaterVelocity;
}

void FWaterSplineComponentVisualizer::OnSetVisualizeRiverWidth()
{
	check(GetEditedWaterSplineComponent());
	if (UWaterSplineMetadata* Metadata = GetEditedWaterSplineMetaData())
	{
		Metadata->bShouldVisualizeRiverWidth = !Metadata->bShouldVisualizeRiverWidth;
		GEditor->RedrawLevelEditingViewports(true);
	}
}

bool FWaterSplineComponentVisualizer::CanSetVisualizeRiverWidth() const
{
	const UWaterSplineMetadata* Metadata = GetEditedWaterSplineMetaData();
	return Metadata && Metadata->CanEditRiverWidth();
}

bool FWaterSplineComponentVisualizer::IsVisualizingRiverWidth() const
{
	const UWaterSplineMetadata* Metadata = GetEditedWaterSplineMetaData();
	return Metadata && Metadata->bShouldVisualizeRiverWidth;
}

void FWaterSplineComponentVisualizer::OnSetVisualizeDepth()
{
	check(GetEditedWaterSplineComponent());
	if (UWaterSplineMetadata* Metadata = GetEditedWaterSplineMetaData())
	{
		Metadata->bShouldVisualizeDepth = !Metadata->bShouldVisualizeDepth;
		GEditor->RedrawLevelEditingViewports(true);
	}
}

bool FWaterSplineComponentVisualizer::CanSetVisualizeDepth() const
{
	const UWaterSplineMetadata* Metadata = GetEditedWaterSplineMetaData();
	return Metadata && Metadata->CanEditDepth();
}

bool FWaterSplineComponentVisualizer::IsVisualizingDepth() const
{
	const UWaterSplineMetadata* Metadata = GetEditedWaterSplineMetaData();
	return Metadata && Metadata->bShouldVisualizeDepth;
}

void FWaterSplineComponentVisualizer::GenerateContextMenuSections(FMenuBuilder& InMenuBuilder) const
{
	FSplineComponentVisualizer::GenerateContextMenuSections(InMenuBuilder);

	ensure(GetEditedWaterSplineComponent());
	if (const UWaterSplineMetadata* Metadata = GetEditedWaterSplineMetaData())
	{
		InMenuBuilder.BeginSection("Water Visualization", LOCTEXT("Water Visualization", "Water Visualization"));
		{
			InMenuBuilder.AddMenuEntry(FWaterSplineComponentVisualizerCommands::Get().VisualizeWaterVelocity);
			InMenuBuilder.AddMenuEntry(FWaterSplineComponentVisualizerCommands::Get().VisualizeRiverWidth);
			InMenuBuilder.AddMenuEntry(FWaterSplineComponentVisualizerCommands::Get().VisualizeDepth);
		}
		InMenuBuilder.EndSection();
	}
}

#undef LOCTEXT_NAMESPACE

