// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SplineComponentVisualizer.h"
#include "WaterSplineComponentVisualizer.generated.h"

#define UE_API WATEREDITOR_API

class UWaterSplineComponent;

class AActor;
class FEditorViewportClient;
class FMenuBuilder;
class FPrimitiveDrawInterface;
class FSceneView;
class FUICommandList;
class FViewport;
class SWidget;
class USplineComponent;
struct FViewportClick;
struct FConvexVolume;

/** Selection state data that will be captured by scoped transactions.*/
UCLASS(MinimalAPI, Transient)
class UWaterSplineComponentVisualizerSelectionState : public USplineComponentVisualizerSelectionState
{
	GENERATED_BODY()

public:

	bool GetWaterVelocityIsSelected() const { return bWaterVelocityIsSelected; }
	void SetWaterVelocityIsSelected(bool InWaterVelocityIsSelected) { bWaterVelocityIsSelected = InWaterVelocityIsSelected; }

	bool GetDepthIsSelected() const { return bDepthIsSelected; }
	void SetDepthIsSelected(bool InDepthIsSelected) { bDepthIsSelected = InDepthIsSelected; }

	bool GetRiverWidthIsSelected() const { return bRiverWidthIsSelected; }
	void SetRiverWidthIsSelected(bool InRiverWidthIsSelected) { bRiverWidthIsSelected = InRiverWidthIsSelected; }

	bool GetRiverWidthSelectedPosHandle() const { return bRiverWidthSelectedPosHandle; }
	void SetRiverWidthSelectedPosHandle(bool InRiverWidthSelectedPosHandle) { bRiverWidthSelectedPosHandle = InRiverWidthSelectedPosHandle; }

protected:

	/** Whether water velocity handle is selected */
	UPROPERTY()
	bool bWaterVelocityIsSelected = false;

	/** Whether water depth handle is selected */
	UPROPERTY()
	bool bDepthIsSelected = false;

	/** Whether water river width handle is selected */
	UPROPERTY()
	bool bRiverWidthIsSelected = false;

	/** When river width is selected, true if the handle on the positive right vector side was selected */
	UPROPERTY()
	bool bRiverWidthSelectedPosHandle = false;
};


/** Base class for clickable water spline editing proxies */
struct HWaterSplineVisProxy : public HComponentVisProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HWaterSplineVisProxy(const UActorComponent* InComponent)
	: HComponentVisProxy(InComponent, HPP_Wireframe)
	{}
};

/** Base class for clickable water spline editing proxies associated with a spline key */
struct HWaterSplineKeyProxy : public HWaterSplineVisProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HWaterSplineKeyProxy(const UActorComponent* InComponent, int32 InKeyIndex)
		: HWaterSplineVisProxy(InComponent)
		, KeyIndex(InKeyIndex)
	{}

	int32 KeyIndex;
};

/** Proxy for a water velocity handle */
struct HWaterSplineWaterVelocityProxy : public HWaterSplineKeyProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HWaterSplineWaterVelocityProxy(const UActorComponent* InComponent, int32 InKeyIndex)
		: HWaterSplineKeyProxy(InComponent, InKeyIndex)
	{}
};

/** Proxy for a river width handle */
struct HWaterSplineRiverWidthProxy : public HWaterSplineKeyProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HWaterSplineRiverWidthProxy(const UActorComponent* InComponent, int32 InKeyIndex)
		: HWaterSplineKeyProxy(InComponent, InKeyIndex)
	{}
};

/** Proxy for a water depth handle */
struct HWaterSplineDepthProxy : public HWaterSplineKeyProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HWaterSplineDepthProxy(const UActorComponent* InComponent, int32 InKeyIndex)
		: HWaterSplineKeyProxy(InComponent, InKeyIndex)
	{}
};

/** Proxy for a water shoreline audio intensity handle */
struct HWaterSplineAudioIntensityProxy : public HWaterSplineKeyProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HWaterSplineAudioIntensityProxy(const UActorComponent* InComponent, int32 InKeyIndex)
		: HWaterSplineKeyProxy(InComponent, InKeyIndex)
	{}

	int32 KeyIndex;
};

/** SplineComponent visualizer/edit functionality */
class FWaterSplineComponentVisualizer : public FSplineComponentVisualizer
{
public:
	FWaterSplineComponentVisualizer();
	virtual ~FWaterSplineComponentVisualizer();

	//~ Begin FComponentVisualizer Interface
	virtual void OnRegister() override;
	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click) override;

	virtual bool GetWidgetLocation(const FEditorViewportClient* ViewportClient, FVector& OutLocation) const override;
	virtual bool HandleInputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& DeltaTranslate, FRotator& DeltaRotate, FVector& DeltaScale) override;

	/** Add menu sections to the context menu */
	virtual void GenerateContextMenuSections(FMenuBuilder& InMenuBuilder) const;

	/** Get the water spline component we are currently editing */
	UWaterSplineComponent* GetEditedWaterSplineComponent() const;

protected:

	/** Update the key selection state of the visualizer */
	virtual void ClearSelectionState();

	/** Update the key selection state of the visualizer */
	virtual void ChangeSelectionState(int32 Index, bool bIsCtrlHeld);

	/** Updates the component and selected properties if the component has changed */
	const UWaterSplineComponent* UpdateSelectedWaterSplineComponent(HComponentVisProxy* VisProxy);

	/** Updates selected key prox index. */
	void UpdateSelectionState(const int32 KeyIndex);

	/** Draw vis proxy handles */
	enum class EHandleType
	{
		PositiveAxis,
		NegativeAxis,
		Both
	};

	/** Handle water spline metadata vis proxy clicked, returns whether it is the positive or negative end */
	bool VisProxyHandleWaterClick(const UWaterSplineComponent* WaterSplineComp, const FViewportClick& Click, float HandleLength, EHandleType HandleType, const FVector& LocalAxis, const FVector& LocalRotAxis);

	/** Compute the actual delta based on input delta */
	float ComputeDelta(UWaterSplineComponent* WaterSplineComp, const FVector& InDeltaTranslate, float InCurrentHandleLength, const FVector& InAxis, float InScale, bool bClampToZero);

	void OnSetVisualizeWaterVelocity();
	bool CanSetVisualizeWaterVelocity() const;
	bool IsVisualizingWaterVelocity() const;

	void OnSetVisualizeRiverWidth();
	bool CanSetVisualizeRiverWidth() const;
	bool IsVisualizingRiverWidth() const;

	void OnSetVisualizeDepth();
	bool CanSetVisualizeDepth() const;
	bool IsVisualizingDepth() const;

	class UWaterSplineMetadata* GetEditedWaterSplineMetaData() const;

	/** Action command list */
	TSharedPtr<FUICommandList> WaterSplineComponentVisualizerActions;
};

#undef UE_API
