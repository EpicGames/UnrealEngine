// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterSplineMetadataDetails.h"
#include "WaterSplineMetadata.h"
#include "IDetailGroup.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "ComponentVisualizer.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterSplineMetadataDetails)

#define LOCTEXT_NAMESPACE "FWaterSplineMetadataDetails"

UWaterSplineMetadataDetailsFactory::UWaterSplineMetadataDetailsFactory(const FObjectInitializer& ObjInitializer)
	: Super(ObjInitializer)
{

}

UClass* UWaterSplineMetadataDetailsFactory::GetMetadataClass() const
{
	return UWaterSplineMetadata::StaticClass();
}

TSharedPtr<ISplineMetadataDetails> UWaterSplineMetadataDetailsFactory::Create()
{
	return MakeShared<FWaterSplineMetadataDetails>();
}

FText FWaterSplineMetadataDetails::GetDisplayName() const
{
	return LOCTEXT("WaterSplineMetadataDetails", "Water");
}

template<class T>
bool UpdateMultipleValue(TOptional<T>& CurrentValue, T InValue)
{
	if (!CurrentValue.IsSet())
	{
		CurrentValue = InValue;
	}
	else if (CurrentValue.IsSet() && CurrentValue.GetValue() != InValue)
	{
		CurrentValue.Reset();
		return false;
	}

	return true;
}

void FWaterSplineMetadataDetails::Update(USplineComponent* InSplineComponent, const TSet<int32>& InSelectedKeys)
{
	SplineComp = InSplineComponent;
	SelectedKeys = InSelectedKeys;
	DepthValue.Reset();
	RiverWidthValue.Reset();
	VelocityValue.Reset();
	AudioIntensityValue.Reset();

	if (InSplineComponent)
	{
		bool bUpdateDepth = true;
		bool bUpdateRiverWidth = true;
		bool bUpdateVelocity = true;
		bool bUpdateAudioIntensity = true;

		UWaterSplineMetadata* Metadata = Cast<UWaterSplineMetadata>(InSplineComponent->GetSplinePointsMetadata());
		if (Metadata)
		{
			for (int32 Index : InSelectedKeys)
			{
				if (bUpdateDepth)
				{
					bUpdateDepth = UpdateMultipleValue(DepthValue, Metadata->Depth.Points[Index].OutVal);
				}

				if (bUpdateRiverWidth)
				{
					bUpdateRiverWidth = UpdateMultipleValue(RiverWidthValue, Metadata->RiverWidth.Points[Index].OutVal);
				}

				if (bUpdateVelocity)
				{
					bUpdateVelocity = UpdateMultipleValue(VelocityValue, Metadata->WaterVelocityScalar.Points[Index].OutVal);
				}

				if (bUpdateAudioIntensity)
				{
					bUpdateAudioIntensity = UpdateMultipleValue(AudioIntensityValue, Metadata->AudioIntensity.Points[Index].OutVal);
				}
			}
		}
	}
}

namespace WaterSplineMetadataDetailsLocals 
{
	static bool IsTextCommitValueSet(ETextCommit::Type CommitInfo)
	{
		return CommitInfo == ETextCommit::OnUserMovedFocus || CommitInfo == ETextCommit::OnEnter;
	}
}

template<class T>
void SetValues(FWaterSplineMetadataDetails& Details, TArray<FInterpCurvePoint<T>>& Points, const T& NewValue, ETextCommit::Type CommitInfo, const FText& TransactionText)
{
	// Scope the transaction to only include the value change and none of the derived data changes that might arise from NotifyPropertyModified
	{
		const FScopedTransaction Transaction(TransactionText);
		Details.SplineComp->GetSplinePointsMetadata()->Modify();
		for (int32 Index : Details.SelectedKeys)
		{
			Points[Index].OutVal = NewValue;
		}
	}

	Details.SplineComp->UpdateSpline();
	Details.SplineComp->bSplineHasBeenEdited = true;
	EPropertyChangeType::Type PropertyChangeType = WaterSplineMetadataDetailsLocals::IsTextCommitValueSet(CommitInfo) ? EPropertyChangeType::ValueSet : EPropertyChangeType::Interactive;
	
	static TArray<FProperty*> SplineProperties;
	if (SplineProperties.IsEmpty())
	{
		for (const FName& Property : USplineComponent::GetSplinePropertyNames())
		{
			SplineProperties.Add(FindFProperty<FProperty>(USplineComponent::StaticClass(), Property));
		}
	}

	FComponentVisualizer::NotifyPropertiesModified(Details.SplineComp, SplineProperties, PropertyChangeType);
	Details.Update(Details.SplineComp, Details.SelectedKeys);

	GEditor->RedrawLevelEditingViewports(true);
}

template <typename T>
bool CheckIfDifferent(const FWaterSplineMetadataDetails& Details, const TArray<FInterpCurvePoint<T>>& Points, const T& NewValue)
{
	bool bIsModified = false;
	for (int32 Index : Details.SelectedKeys)
	{
		bIsModified |= Points[Index].OutVal != NewValue;
	}

	return bIsModified;
}

void FWaterSplineMetadataDetails::OnBeginSliderMovement()
{
	EditSliderValueTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("EditWaterProperty", "Edit Water Property"));
}

void FWaterSplineMetadataDetails::OnEndSliderMovement(float NewValue)
{
	EditSliderValueTransaction.Reset();
}

UWaterSplineMetadata* FWaterSplineMetadataDetails::GetMetadata() const
{
	UWaterSplineMetadata* Metadata = SplineComp ? Cast<UWaterSplineMetadata>(SplineComp->GetSplinePointsMetadata()) : nullptr;
	return Metadata;
}

void FWaterSplineMetadataDetails::OnSetDepth(float NewValue, ETextCommit::Type CommitInfo)
{
	if (UWaterSplineMetadata* Metadata = GetMetadata())
	{
		if (CheckIfDifferent(*this, Metadata->Depth.Points, NewValue) || WaterSplineMetadataDetailsLocals::IsTextCommitValueSet(CommitInfo))
		{
			SetValues(*this, Metadata->Depth.Points, NewValue, CommitInfo, LOCTEXT("SetSplineDepth", "Set spline point water depth"));
		}
	}
}

void FWaterSplineMetadataDetails::OnSetRiverWidth(float NewValue, ETextCommit::Type CommitInfo)
{
	if (UWaterSplineMetadata* Metadata = GetMetadata())
	{
		if (CheckIfDifferent(*this, Metadata->RiverWidth.Points, NewValue) || WaterSplineMetadataDetailsLocals::IsTextCommitValueSet(CommitInfo))
		{
			SetValues(*this, Metadata->RiverWidth.Points, NewValue, CommitInfo, LOCTEXT("SetSplineWaterWidth", "Set spline point river width"));
		}
	}
}

void FWaterSplineMetadataDetails::OnSetVelocity(float NewValue, ETextCommit::Type CommitInfo)
{
	if (UWaterSplineMetadata* Metadata = GetMetadata())
	{
		if (CheckIfDifferent(*this, Metadata->WaterVelocityScalar.Points, NewValue) || WaterSplineMetadataDetailsLocals::IsTextCommitValueSet(CommitInfo))
		{
			SetValues(*this, Metadata->WaterVelocityScalar.Points, NewValue, CommitInfo, LOCTEXT("SetSplineWaterVelocity", "Set spline point water velocity"));
		}
	}
}

void FWaterSplineMetadataDetails::OnSetAudioIntensity(float NewValue, ETextCommit::Type CommitInfo)
{
	if (UWaterSplineMetadata* Metadata = GetMetadata())
	{
		if (CheckIfDifferent(*this, Metadata->AudioIntensity.Points, NewValue) || WaterSplineMetadataDetailsLocals::IsTextCommitValueSet(CommitInfo))
		{
			SetValues(*this, Metadata->AudioIntensity.Points, NewValue, CommitInfo, LOCTEXT("SetSpline point audio intensity", "Set spline point audio intensity"));
		}
	}
}

EVisibility FWaterSplineMetadataDetails::IsDepthVisible() const
{
	UWaterSplineMetadata* Metadata = GetMetadata();
	return (Metadata && Metadata->CanEditDepth()) ? IsEnabled() : EVisibility::Collapsed;
}

EVisibility FWaterSplineMetadataDetails::IsRiverWidthVisible() const
{
	UWaterSplineMetadata* Metadata = GetMetadata();
	return (Metadata && Metadata->CanEditRiverWidth()) ? IsEnabled() : EVisibility::Collapsed;
}

EVisibility FWaterSplineMetadataDetails::IsVelocityVisible() const
{
	UWaterSplineMetadata* Metadata = GetMetadata();
	return (Metadata && Metadata->CanEditVelocity()) ? IsEnabled() : EVisibility::Collapsed;
}

void FWaterSplineMetadataDetails::GenerateChildContent(IDetailGroup& DetailGroup)
{
	DetailGroup.AddWidgetRow()
		.RowTag("Depth")
		.FilterString(LOCTEXT("Depth", "Depth"))
		.Visibility(TAttribute<EVisibility>(this, &FWaterSplineMetadataDetails::IsDepthVisible))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Depth", "Depth"))
			.ToolTipText(LOCTEXT("RiverDepth_Tooltip", "The depth of the river at this spline point. Affects the terrain carving and collision"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(125.0f)
		[
			SNew(SNumericEntryBox<float>)
			.Value(this, &FWaterSplineMetadataDetails::GetDepth)
			.AllowSpin(true)
			.MinValue(0.0f)
			// Because we have no upper limit in MaxSliderValue, we need to "unspecify" the max value here, otherwise the spinner has a limited range,
			//  with TNumericLimits<NumericType>::Max() as the MaxValue and the spinning increment is huge
			.MaxValue(TOptional<float>()) 
			.MinSliderValue(0.0f)
			.MaxSliderValue(TOptional<float>()) // No upper limit
			.OnBeginSliderMovement(this, &FWaterSplineMetadataDetails::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &FWaterSplineMetadataDetails::OnEndSliderMovement)
			.UndeterminedString(LOCTEXT("Multiple", "Multiple"))
			.OnValueCommitted(this, &FWaterSplineMetadataDetails::OnSetDepth)
			.OnValueChanged(this, &FWaterSplineMetadataDetails::OnSetDepth, ETextCommit::Default)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	DetailGroup.AddWidgetRow()
		.RowTag("RiverWidth")
		.FilterString(LOCTEXT("RiverWidth", "River Width"))
		.Visibility(TAttribute<EVisibility>(this, &FWaterSplineMetadataDetails::IsRiverWidthVisible))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("RiverWidth", "River Width"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ToolTipText(LOCTEXT("RiverWidth_Tooltip", "The width of the river at this spline point. Affects the visuals, terrain carving, and collision."))
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(125.0f)
		[
			SNew(SNumericEntryBox<float>)
			.Value(this, &FWaterSplineMetadataDetails::GetRiverWidth)
			.AllowSpin(true)
			.OnBeginSliderMovement(this, &FWaterSplineMetadataDetails::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &FWaterSplineMetadataDetails::OnEndSliderMovement)
			.MinValue(0.0f)
			// Because we have no upper limit in MaxSliderValue, we need to "unspecify" the max value here, otherwise the spinner has a limited range,
			//  with TNumericLimits<NumericType>::Max() as the MaxValue and the spinning increment is huge
			.MaxValue(TOptional<float>()) 
			.MinSliderValue(0.0f)
			.MaxSliderValue(TOptional<float>()) // No upper limit
			.UndeterminedString(LOCTEXT("Multiple", "Multiple"))
			.OnValueCommitted(this, &FWaterSplineMetadataDetails::OnSetRiverWidth)
			.OnValueChanged(this, &FWaterSplineMetadataDetails::OnSetRiverWidth, ETextCommit::Default)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	DetailGroup.AddWidgetRow()
		.RowTag("Velocity")
		.FilterString(LOCTEXT("Velocity", "Velocity"))
		.Visibility(TAttribute<EVisibility>(this, &FWaterSplineMetadataDetails::IsEnabled))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Velocity", "Velocity"))
			.ToolTipText(LOCTEXT("RiverVelocity_Tooltip", "The river flow velocity at this spline point. Affects the visuals of the flowing water."))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(125.0f)
		[
			SNew(SNumericEntryBox<float>)
			.Value(this, &FWaterSplineMetadataDetails::GetVelocity)
			.AllowSpin(true)
			.OnBeginSliderMovement(this, &FWaterSplineMetadataDetails::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &FWaterSplineMetadataDetails::OnEndSliderMovement)
			.MinValue(TOptional<float>())
			.MaxValue(TOptional<float>())
			.MinSliderValue(TOptional<float>()) // No lower limit
			.MaxSliderValue(TOptional<float>()) // No upper limit
			.UndeterminedString(LOCTEXT("Multiple", "Multiple"))
			.OnValueCommitted(this, &FWaterSplineMetadataDetails::OnSetVelocity)
			.OnValueChanged(this, &FWaterSplineMetadataDetails::OnSetVelocity, ETextCommit::Default)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	DetailGroup.AddWidgetRow()
		.RowTag("AudioIntensity")
		.FilterString(LOCTEXT("AudioIntensity", "Audio Intensity"))
		.Visibility(TAttribute<EVisibility>(this, &FWaterSplineMetadataDetails::IsEnabled))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AudioIntensity", "Audio Intensity"))
			.ToolTipText(LOCTEXT("SplineAudioIntensity_Tooltip", "The audio intensity of the water at this spline point."))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(125.0f)
		[
			SNew(SNumericEntryBox<float>)
			.Value(this, &FWaterSplineMetadataDetails::GetAudioIntensity)
			.AllowSpin(true)
			.OnBeginSliderMovement(this, &FWaterSplineMetadataDetails::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &FWaterSplineMetadataDetails::OnEndSliderMovement)
			.MinValue(0.0f)
			// Because we have no upper limit in MaxSliderValue, we need to "unspecify" the max value here, otherwise the spinner has a limited range,
			//  with TNumericLimits<NumericType>::Max() as the MaxValue and the spinning increment is huge
			.MaxValue(TOptional<float>())
			.MinSliderValue(0.0f)
			.MaxSliderValue(TOptional<float>()) // No upper limit
			.UndeterminedString(LOCTEXT("Multiple", "Multiple"))
			.OnValueCommitted(this, &FWaterSplineMetadataDetails::OnSetAudioIntensity)
			.OnValueChanged(this, &FWaterSplineMetadataDetails::OnSetAudioIntensity, ETextCommit::Default)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
}

#undef LOCTEXT_NAMESPACE
