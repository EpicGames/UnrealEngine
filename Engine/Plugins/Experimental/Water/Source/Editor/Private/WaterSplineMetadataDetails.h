// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SplineMetadataDetailsFactory.h"
#include "Layout/Visibility.h"
#include "ScopedTransaction.h"
#include "WaterSplineMetadataDetails.generated.h"

namespace ETextCommit { enum Type : int; }

class USplineComponent;
class IDetailGroup;

UCLASS()
class UWaterSplineMetadataDetailsFactory : public USplineMetadataDetailsFactoryBase
{
	GENERATED_UCLASS_BODY()

public:
	virtual ~UWaterSplineMetadataDetailsFactory() = default;
	virtual TSharedPtr<ISplineMetadataDetails> Create() override;
	virtual UClass* GetMetadataClass() const override;
};

class FWaterSplineMetadataDetails : public ISplineMetadataDetails, public TSharedFromThis<FWaterSplineMetadataDetails>
{
public:
	virtual ~FWaterSplineMetadataDetails() = default;
	virtual FName GetName() const override { return FName(TEXT("WaterSplineMetadataDetails")); }
	virtual FText GetDisplayName() const override;
	virtual void Update(USplineComponent* InSplineComponent, const TSet<int32>& InSelectedKeys) override;
	virtual void GenerateChildContent(IDetailGroup& InGroup) override;

	TOptional<float> DepthValue;
	TOptional<float> RiverWidthValue;
	TOptional<float> VelocityValue;
	TOptional<float> AudioIntensityValue;

	USplineComponent* SplineComp = nullptr;
	TSet<int32> SelectedKeys;

private:
	void OnBeginSliderMovement();
	void OnEndSliderMovement(float NewValue);

	EVisibility IsEnabled() const { return (SelectedKeys.Num() > 0) ? EVisibility::Visible : EVisibility::Collapsed; }
	bool IsOnePointSelected() const { return SelectedKeys.Num() == 1; }

	EVisibility IsDepthVisible() const;
	TOptional<float> GetDepth() const { return DepthValue; }
	void OnSetDepth(float NewValue, ETextCommit::Type CommitInfo);

	EVisibility IsRiverWidthVisible() const;
	TOptional<float> GetRiverWidth() const { return RiverWidthValue; }
	void OnSetRiverWidth(float NewValue, ETextCommit::Type CommitInfo);

	EVisibility IsVelocityVisible() const;
	TOptional<float> GetVelocity() const { return VelocityValue; }
	void OnSetVelocity(float NewValue, ETextCommit::Type CommitInfo);

	TOptional<float> GetAudioIntensity() const { return AudioIntensityValue; }
	void OnSetAudioIntensity(float NewValue, ETextCommit::Type CommitInfo);

	class UWaterSplineMetadata* GetMetadata() const;
private:
	TUniquePtr<FScopedTransaction> EditSliderValueTransaction;
};

