// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterUIStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"

#define RootToContentDir StyleSet->RootToContentDir

TSharedPtr< FSlateStyleSet > FWaterUIStyle::StyleSet = nullptr;
TSharedPtr< class ISlateStyle > FWaterUIStyle::Get() { return StyleSet; }

FName FWaterUIStyle::GetStyleSetName()
{
	static FName WaterStyleName(TEXT("WaterUIStyle"));
	return WaterStyleName;
}

void FWaterUIStyle::Initialize()
{
	// Const icon sizes
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet(GetStyleSetName()));

	StyleSet->SetContentRoot(IPluginManager::Get().FindPlugin(TEXT("Water"))->GetContentDir());
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// WaterBodyRiver
	StyleSet->Set("ClassIcon.WaterBodyRiver", new IMAGE_BRUSH_SVG("Icons/WaterBodyRiver", Icon16x16));
	StyleSet->Set("ClassThumbnail.WaterBodyRiver", new IMAGE_BRUSH_SVG("Icons/WaterBodyRiver_64", Icon64x64));
	// WaterBodyOcean
	StyleSet->Set("ClassIcon.WaterBodyOcean", new IMAGE_BRUSH_SVG("Icons/WaterBodyOcean", Icon16x16));
	StyleSet->Set("ClassThumbnail.WaterBodyOcean", new IMAGE_BRUSH_SVG("Icons/WaterBodyOcean_64", Icon64x64));
	// WaterBodyLake
	StyleSet->Set("ClassIcon.WaterBodyLake", new IMAGE_BRUSH_SVG("Icons/WaterBodyLake", Icon16x16));
	StyleSet->Set("ClassThumbnail.WaterBodyLake", new IMAGE_BRUSH_SVG("Icons/WaterBodyLake_64", Icon64x64));
	// WaterBodyCustom
	StyleSet->Set("ClassIcon.WaterBodyCustom", new IMAGE_BRUSH_SVG("Icons/WaterBodyCustom", Icon16x16));
	StyleSet->Set("ClassThumbnail.WaterBodyCustom", new IMAGE_BRUSH_SVG("Icons/WaterBodyCustom_64", Icon64x64));
	// WaterBodyIsland
	StyleSet->Set("ClassIcon.WaterBodyIsland", new IMAGE_BRUSH_SVG("Icons/WaterBodyIsland", Icon16x16));
	StyleSet->Set("ClassThumbnail.WaterBodyIsland", new IMAGE_BRUSH_SVG("Icons/WaterBodyIsland_64", Icon64x64));
	// WaterBodyExclusionVolume
	StyleSet->Set("ClassIcon.WaterBodyExclusionVolume", new IMAGE_BRUSH_SVG("Icons/WaterBodyExclusionVolume", Icon16x16));
	StyleSet->Set("ClassThumbnail.WaterBodyExclusionVolume", new IMAGE_BRUSH_SVG("Icons/WaterBodyExclusionVolume_64", Icon64x64));
	// WaterZone
	StyleSet->Set("ClassIcon.WaterZone", new IMAGE_BRUSH_SVG("Icons/WaterZone", Icon16x16));
	StyleSet->Set("ClassThumbnail.WaterZone", new IMAGE_BRUSH_SVG("Icons/WaterZone_64", Icon64x64));
	// WaterLandscapeBrush
	StyleSet->Set("ClassIcon.WaterLandscapeBrush", new IMAGE_BRUSH_SVG("Icons/WaterLandscapeBrush", Icon16x16));
	StyleSet->Set("ClassThumbnail.WaterLandscapeBrush", new IMAGE_BRUSH_SVG("Icons/WaterLandscapeBrush_64", Icon64x64));
	// WaterWaves
	StyleSet->Set("ClassIcon.WaterWavesBase", new IMAGE_BRUSH_SVG("Icons/WaterWave", Icon16x16));
	StyleSet->Set("ClassThumbnail.WaterWavesBase", new IMAGE_BRUSH_SVG("Icons/WaterWave_64", Icon64x64));
	// WaterWavesAsset
	StyleSet->Set("ClassIcon.WaterWavesAsset", new IMAGE_BRUSH_SVG("Icons/WaterWave", Icon16x16));
	StyleSet->Set("ClassThumbnail.WaterWavesAsset", new IMAGE_BRUSH_SVG("Icons/WaterWave_64", Icon64x64));
	// WaterWaves Asset Editor
	StyleSet->Set("WaterWavesEditor.TogglePauseWaveTime", new IMAGE_BRUSH("Icons/PauseWaveTime_40x", Icon40x40));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

#undef RootToContentDir

void FWaterUIStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}
