// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterWavesAssetFactory.h"
#include "WaterWaves.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterWavesAssetFactory)

#define LOCTEXT_NAMESPACE "WaterWavesAssetFactory"

UWaterWavesAssetFactory::UWaterWavesAssetFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UWaterWavesAsset::StaticClass();
}

UObject* UWaterWavesAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UWaterWavesAsset* NewAsset = NewObject<UWaterWavesAsset>(InParent, Class, Name, Flags | RF_Transactional);
	return NewAsset;
}

FText UWaterWavesAssetFactory::GetToolTip() const
{
	return LOCTEXT("WavesAssetTooltip", "Creates a default water waves asset");
}

#undef LOCTEXT_NAMESPACE
