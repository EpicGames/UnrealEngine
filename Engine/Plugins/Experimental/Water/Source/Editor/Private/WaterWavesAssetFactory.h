// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "WaterWavesAssetFactory.generated.h"

UCLASS()
class UWaterWavesAssetFactory : public UFactory
{
	GENERATED_BODY()

public:

	UWaterWavesAssetFactory();

	// UFactory interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FText GetToolTip() const override;
	// End of UFactory interface
};
