// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterWavesEditorCommands.h"

#define LOCTEXT_NAMESPACE "WaterWavesEditorCommands"

void FWaterWavesEditorCommands::RegisterCommands()
{
	UI_COMMAND(TogglePauseWaveTime, "Pause Time", "Pause wave time", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE