// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "WaterUIStyle.h"

class FWaterWavesEditorCommands : public TCommands<FWaterWavesEditorCommands>
{
public:
	FWaterWavesEditorCommands() : TCommands<FWaterWavesEditorCommands>(
		"WaterWavesEditor", 
		NSLOCTEXT("WaterWavesEditor", "Water Waves Editor", "Water Waves Editor Commands"),
		"EditorViewport",
		FWaterUIStyle::GetStyleSetName())
	{}

	 /** Toggle if wave time should be paused */
	TSharedPtr<FUICommandInfo> TogglePauseWaveTime;

	/** Register commands */
	virtual void RegisterCommands() override;
};
