// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterWavesEditorToolkit.h"
#include "DetailsViewArgs.h"
#include "Styling/SlateStyleRegistry.h"
#include "Framework/Commands/UICommandList.h"
#include "Modules/ModuleManager.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "EditorViewportTabContent.h"
#include "WaterWaves.h"
#include "WaterWavesEditorCommands.h"
#include "WaterWavesEditorViewport.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "WaterEditorToolkit"

const FName FWaterWavesEditorToolkit::ViewportTabId(TEXT("WaterWavesEditor_Viewport"));
const FName FWaterWavesEditorToolkit::PropertiesTabId(TEXT("WaterWavesEditor_Properties"));

void FWaterWavesEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_WaterWavesEditor", "Water Waves Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(ViewportTabId, FOnSpawnTab::CreateSP(this, &FWaterWavesEditorToolkit::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("ViewportTab", "Viewport"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(PropertiesTabId, FOnSpawnTab::CreateSP(this, &FWaterWavesEditorToolkit::SpawnTab_Properties))
		.SetDisplayName(LOCTEXT("PropertiesTab", "Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

void FWaterWavesEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(ViewportTabId);
	InTabManager->UnregisterTabSpawner(PropertiesTabId);
}

void FWaterWavesEditorToolkit::InitWaterWavesEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UObject* ObjectToEdit)
{
	UWaterWavesAsset* WavesAsset = CastChecked<UWaterWavesAsset>(ObjectToEdit);

	WaterWavesAssetRef = NewObject<UWaterWavesAssetReference>();
	WaterWavesAssetRef->SetWaterWavesAsset(WavesAsset);

	FWaterWavesEditorCommands::Register();
	BindCommands();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.NotifyHook = this;

	WaterWavesDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	WaterWavesDetailsView->SetObject(WavesAsset);

	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_WaterWavesEditor_Layout_v2")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
					->SetSizeCoefficient(0.9f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.6f)
						->AddTab(ViewportTabId, ETabState::OpenedTab)
						->SetHideTabWell(true)
					)
					->Split
					(
						FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.2f)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.7f)
							->AddTab(PropertiesTabId, ETabState::OpenedTab)
						)
					)
				)
			)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, FName(TEXT("WaterWavesEditorApp")), StandaloneDefaultLayout, bCreateDefaultToolbar, bCreateDefaultStandaloneMenu, ObjectToEdit);
	ExtendToolbar();
	RegenerateMenusAndToolbars();
}

void FWaterWavesEditorToolkit::ExtendToolbar()
{
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP(this, &FWaterWavesEditorToolkit::FillToolbar, GetToolkitCommands())
	);

	AddToolbarExtender(ToolbarExtender);
}

void FWaterWavesEditorToolkit::FillToolbar(FToolBarBuilder& ToolbarBuilder, const TSharedRef<FUICommandList> InToolkitCommands)
{
	const ISlateStyle* WaterUIStyle = FSlateStyleRegistry::FindSlateStyle(FWaterUIStyle::GetStyleSetName());

	ToolbarBuilder.BeginSection("Water");
	{
		ToolbarBuilder.AddToolBarButton(FWaterWavesEditorCommands::Get().TogglePauseWaveTime);
	}
	ToolbarBuilder.EndSection();
}

void FWaterWavesEditorToolkit::BindCommands()
{
	const FWaterWavesEditorCommands& Commands = FWaterWavesEditorCommands::Get();

	const TSharedRef<FUICommandList>& UICommandList = GetToolkitCommands();

	UICommandList->MapAction(Commands.TogglePauseWaveTime,
		FExecuteAction::CreateSP(this, &FWaterWavesEditorToolkit::TogglePauseWaveTime),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FWaterWavesEditorToolkit::IsWaveTimePaused));
}

TSharedRef<SDockTab> FWaterWavesEditorToolkit::SpawnTab_Properties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == PropertiesTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("WaterWavesEditorProperties_TabTitle", "Details"))
		[
			WaterWavesDetailsView.ToSharedRef()
		];
}

TSharedRef<SDockTab> FWaterWavesEditorToolkit::SpawnTab_Viewport(const FSpawnTabArgs& Args)
{
	TSharedRef< SDockTab > DockableTab =
		SNew(SDockTab);

	ViewportTabContent = MakeShareable(new FEditorViewportTabContent());

	TWeakPtr<FWaterWavesEditorToolkit> WeakSharedThis = SharedThis(this);

	const FString LayoutId = FString("WaterWavesEditorViewport");
	ViewportTabContent->Initialize([WeakSharedThis](const FAssetEditorViewportConstructionArgs& InConstructionArgs)
	{
		return SNew(SWaterWavesEditorViewport)
			.WaterWavesEditorToolkit(WeakSharedThis);
	}, DockableTab, LayoutId);

	return DockableTab;
}

void FWaterWavesEditorToolkit::TogglePauseWaveTime()
{
	bWaveTimePaused = !bWaveTimePaused;

	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> TogglePauseWaveTimeFunc =
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
		{
			TSharedRef<SWaterWavesEditorViewport> WaterWavesEditorViewport = StaticCastSharedRef<SWaterWavesEditorViewport>(Entity->AsWidget());
			WaterWavesEditorViewport->SetShouldPauseWaveTime(bWaveTimePaused);
		};

	ViewportTabContent->PerformActionOnViewports(TogglePauseWaveTimeFunc);
}

bool FWaterWavesEditorToolkit::IsWaveTimePaused() const
{
	return bWaveTimePaused;
}

FName FWaterWavesEditorToolkit::GetToolkitFName() const
{
	return FName("WaterWavesEditor");
}

FText FWaterWavesEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "WaterWaves Editor");
}

FString FWaterWavesEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "WaterWaves ").ToString();
}

FLinearColor FWaterWavesEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

void FWaterWavesEditorToolkit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(WaterWavesAssetRef);
}

#undef LOCTEXT_NAMESPACE
