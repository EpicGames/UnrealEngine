// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Toolkits/AssetEditorToolkit.h"

class FSpawnTabArgs;
class FToolBarBuilder;
class UWaterWavesAssetReference;

class FEditorViewportTabContent;
class IDetailsView;
class FTabManager;
class IToolkitHost;

class FWaterWavesEditorToolkit : public FAssetEditorToolkit, public FNotifyHook, public FGCObject
{
public:
	void InitWaterWavesEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UObject* ObjectToEdit);

	UWaterWavesAssetReference* GetWavesAssetRef() { return WaterWavesAssetRef; }
public:
	//~ IToolkit Interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	//~ FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FWaterWavesEditorToolkit");
	}

private:
	void ExtendToolbar();

	void FillToolbar(FToolBarBuilder& ToolbarBuilder, const TSharedRef<FUICommandList> InToolkitCommands);

	void BindCommands();

	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;

	/* Tab spawners */
	TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Properties(const FSpawnTabArgs& Args);

	/* Command actions */
	void TogglePauseWaveTime();
	bool IsWaveTimePaused() const;
private:
	static const FName ViewportTabId;
	static const FName PropertiesTabId;

	TObjectPtr<UWaterWavesAssetReference> WaterWavesAssetRef = nullptr;

	// Tracking the active viewports in this editor.
	TSharedPtr<FEditorViewportTabContent> ViewportTabContent;

	/** Property View */
	TSharedPtr<IDetailsView> WaterWavesDetailsView;

	bool bWaveTimePaused = false;
};
