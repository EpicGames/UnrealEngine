// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterWavesEditorViewport.h"

#include "WaterWavesEditorToolkit.h"
#include "AdvancedPreviewScene.h"
#include "PreviewProfileController.h"
#include "WaterBodyCustomActor.h"
#include "WaterEditorSettings.h"
#include "WaterSplineComponent.h"

#include "WaterSubsystem.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "ToolMenus.h"

SWaterWavesEditorViewport::SWaterWavesEditorViewport()
{
	// Temporarily allow water subsystem to be created on preview worlds because we need one here : 
	UWaterSubsystem::FScopedAllowWaterSubsystemOnPreviewWorld AllowWaterSubsystemOnPreviewWorldScope(true);
	PreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues()));
}

void SWaterWavesEditorViewport::Construct(const FArguments& InArgs)
{
	WaterWavesEditorToolkitPtr = InArgs._WaterWavesEditorToolkit;

	TSharedPtr<FWaterWavesEditorToolkit> WaterWavesEditorToolkit = WaterWavesEditorToolkitPtr.Pin();
	check(WaterWavesEditorToolkitPtr.IsValid());

	UWaterWavesAssetReference* WaterWavesAssetRef = WaterWavesEditorToolkit->GetWavesAssetRef();

	SEditorViewport::Construct(SEditorViewport::FArguments());

	PreviewScene->SetFloorVisibility(false);

	CustomWaterBody = CastChecked<AWaterBodyCustom>(PreviewScene->GetWorld()->SpawnActor(AWaterBodyCustom::StaticClass()));
	UWaterBodyComponent* WaterBodyComponent = CustomWaterBody->GetWaterBodyComponent();
	check(WaterBodyComponent);
	WaterBodyComponent->SetWaterMeshOverride(GetDefault<UWaterEditorSettings>()->WaterBodyCustomDefaults.GetWaterMesh());
	WaterBodyComponent->SetWaterMaterial(GetDefault<UWaterEditorSettings>()->WaterBodyCustomDefaults.GetWaterMaterial());
	// Reduce the wave attenuation target depth otherwise we will show attenuated waves in the preview which is not representative of the actual wave parameters:
	WaterBodyComponent->TargetWaveMaskDepth = 1.f;


	UWaterSplineComponent* WaterSpline = CustomWaterBody->GetWaterSpline();
	check(WaterSpline);
	
	WaterSpline->ResetSpline({ FVector(0, 0, 0) });
	CustomWaterBody->SetWaterWaves(WaterWavesAssetRef);
	CustomWaterBody->SetActorScale3D(FVector(60, 60, 1));

	EditorViewportClient->MoveViewportCamera(FVector(-3000, 0, 2000), FRotator(-35.f, 0.f, 0.f));
}

TSharedRef<SEditorViewport> SWaterWavesEditorViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SWaterWavesEditorViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SWaterWavesEditorViewport::OnFloatingButtonClicked()
{
}

void SWaterWavesEditorViewport::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(CustomWaterBody);
}

TSharedRef<FEditorViewportClient> SWaterWavesEditorViewport::MakeEditorViewportClient()
{
	EditorViewportClient = MakeShareable(new FWaterWavesEditorViewportClient(PreviewScene.Get(), SharedThis(this)));
	return EditorViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SWaterWavesEditorViewport::BuildViewportToolbar()
{
	// Register the viewport toolbar if another viewport hasn't already (it's shared).
	const FName ViewportToolbarName = "WaterWavesEditor.ViewportToolbar";

	if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
	{
		UToolMenu* const ViewportToolbarMenu = UToolMenus::Get()->RegisterMenu(
			ViewportToolbarName, NAME_None /* parent */, EMultiBoxType::SlimHorizontalToolBar
		);

		ViewportToolbarMenu->StyleName = "ViewportToolbar";

		// Add the left-aligned part of the viewport toolbar.
		{
			FToolMenuSection& LeftSection = ViewportToolbarMenu->AddSection("Left");
		}

		// Add the right-aligned part of the viewport toolbar.
		{
			FToolMenuSection& RightSection = ViewportToolbarMenu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;

			// Add the "Camera" submenu.
			RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(UE::UnrealEd::FViewportCameraMenuOptions().ShowLensControls()));

			// Add the "View Modes" sub menu.
			{
				// Stay backward-compatible with the old viewport toolbar.
				{
					const FName ParentSubmenuName = "UnrealEd.ViewportToolbar.View";
					// Create our parent menu.
					if (!UToolMenus::Get()->IsMenuRegistered(ParentSubmenuName))
					{
						UToolMenus::Get()->RegisterMenu(ParentSubmenuName);
					}

					// Register our ToolMenu here first, before we create the submenu, so we can set our parent.
					UToolMenus::Get()->RegisterMenu("WaterWavesEditor.ViewportToolbar.ViewModes", ParentSubmenuName);
				}

				RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
			}

			RightSection.AddEntry(UE::UnrealEd::CreateDefaultShowSubmenu());
			RightSection.AddEntry(UE::UnrealEd::CreatePerformanceAndScalabilitySubmenu());
			RightSection.AddEntry(UE::UnrealEd::CreateAssetViewerProfileSubmenu());
		}
	}

	FToolMenuContext ViewportToolbarContext;
	{
		ViewportToolbarContext.AppendCommandList(GetCommandList());

		// Add the UnrealEd viewport toolbar context.
		{
			UUnrealEdViewportToolbarContext* const ContextObject =
				UE::UnrealEd::CreateViewportToolbarDefaultContext(SharedThis(this));

			// Hiding Grid toggle as grid is not showing in water waves editor
			ContextObject->ExcludedShowMenuFlags.Add(FEngineShowFlags::SF_Grid);

			ViewportToolbarContext.AddObject(ContextObject);
		}
	}

	return UToolMenus::Get()->GenerateWidget(ViewportToolbarName, ViewportToolbarContext);
}

TSharedPtr<IPreviewProfileController> SWaterWavesEditorViewport::CreatePreviewProfileController()
{
	return MakeShared<FPreviewProfileController>();
}

void SWaterWavesEditorViewport::SetShouldPauseWaveTime(bool bShouldFreeze)
{
	UWaterSubsystem* WaterSubsystem = EditorViewportClient->GetWorld()->GetSubsystem<UWaterSubsystem>();
	check(WaterSubsystem != nullptr);
	WaterSubsystem->SetShouldPauseWaveTime(bShouldFreeze);
}


// ----------------------------------------------------------------------------------

FWaterWavesEditorViewportClient::FWaterWavesEditorViewportClient(FPreviewScene* InPreviewScene, const TWeakPtr<SEditorViewport>& InEditorViewportWidget)
	: FEditorViewportClient(nullptr, InPreviewScene, InEditorViewportWidget)
{
	bSetListenerPosition = false;
	SetRealtime(true);
	EngineShowFlags.Grid = false;
}

void FWaterWavesEditorViewportClient::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

	// Tick the preview scene world.
	PreviewScene->GetWorld()->Tick(IsRealtime() ? LEVELTICK_All : LEVELTICK_TimeOnly, DeltaSeconds);
}

