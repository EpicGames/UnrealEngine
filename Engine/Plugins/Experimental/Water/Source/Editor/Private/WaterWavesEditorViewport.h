// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportClient.h"
#include "SAssetEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"

class FWaterWavesEditorToolkit;
class FAdvancedPreviewScene;
class FEditorViewportClient;
class AWaterBodyCustom;
class UWaterWavesAssetReference;
class SEditorViewport;


// ----------------------------------------------------------------------------------

class SWaterWavesEditorViewport : public SAssetEditorViewport, public ICommonEditorViewportToolbarInfoProvider, public FGCObject
{
public:
	SLATE_BEGIN_ARGS(SWaterWavesEditorViewport) {}
		SLATE_ARGUMENT(TWeakPtr<FWaterWavesEditorToolkit>, WaterWavesEditorToolkit)
	SLATE_END_ARGS()

	SWaterWavesEditorViewport();

	void Construct(const FArguments& InArgs);

	//~ ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;

	//~ FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SWaterWavesEditorViewport");
	}

public:
	void SetShouldPauseWaveTime(bool bShouldFreeze);

protected:
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual TSharedPtr<IPreviewProfileController> CreatePreviewProfileController() override;
private:
	/** The scene for this viewport. */
	TSharedPtr<FAdvancedPreviewScene> PreviewScene;

	/** Editor viewport client */
	TSharedPtr<FEditorViewportClient> EditorViewportClient;

	TWeakPtr<FWaterWavesEditorToolkit> WaterWavesEditorToolkitPtr;

	TObjectPtr<AWaterBodyCustom> CustomWaterBody = nullptr;
};


// ----------------------------------------------------------------------------------

class FWaterWavesEditorViewportClient : public FEditorViewportClient
{
public:
	using Super = FEditorViewportClient;

	FWaterWavesEditorViewportClient(FPreviewScene* InPreviewScene, const TWeakPtr<SEditorViewport>& InEditorViewportWidget = nullptr);

	// FEditorViewportClient interface
	virtual void Tick(float DeltaSeconds) override;
	// End of FEditorViewportClient
};
