// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterZoneActorFactory.h"
#include "WaterZoneActor.h"
#include "WaterMeshComponent.h"
#include "WaterEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterZoneActorFactory)

#define LOCTEXT_NAMESPACE "WaterZoneActorFactory"

UWaterZoneActorFactory::UWaterZoneActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("WaterZoneActorDisplayName", "Water Zone");
	NewActorClass = AWaterZone::StaticClass();
	bUseSurfaceOrientation = true;
}

void UWaterZoneActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);
	
	AWaterZone* WaterZoneActor = CastChecked<AWaterZone>(NewActor);

	const FWaterZoneActorDefaults& WaterZoneActorDefaults = GetDefault<UWaterEditorSettings>()->WaterZoneActorDefaults;
	WaterZoneActor->GetWaterMeshComponent()->FarDistanceMaterial = WaterZoneActorDefaults.GetFarDistanceMaterial();
	WaterZoneActor->GetWaterMeshComponent()->FarDistanceMeshExtent = WaterZoneActorDefaults.FarDistanceMeshExtent;

	WaterZoneActor->SetRenderTargetResolution(WaterZoneActorDefaults.RenderTargetResolution);
}

#undef LOCTEXT_NAMESPACE
