// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ActorFactories/ActorFactory.h"
#include "WaterZoneActorFactory.generated.h"

UCLASS(MinimalAPI, config = Editor)
class UWaterZoneActorFactory : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
};
