// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "CausticsGeneratorActor.generated.h"

class UHierarchicalInstancedStaticMeshComponent;

UCLASS(Blueprintable, hidecategories = (Replication, Input, LOD, Actor, Cooking, Rendering, Collision))
class ACausticsGeneratorActor: public AActor
{
public:
	GENERATED_BODY()

	//UPROPERTY(BlueprintReadWrite, NonTransactional, meta = (Category = "Default"))
	//UHierarchicalInstancedStaticMeshComponent* WaterPreviewGridHISMC;

	//UPROPERTY(BlueprintReadWrite, NonTransactional, meta = (Category = "Default"))
	//UHierarchicalInstancedStaticMeshComponent* CausticParticlesGridHISMC;

	UPROPERTY(BlueprintReadWrite, NonTransactional, meta = (Category = "Default"))
	TObjectPtr<USceneComponent> DefaultSceneRoot;


	UFUNCTION(BlueprintNativeEvent, CallInEditor, BlueprintCallable, Category = "Tick")
	void EditorTick(float DeltaSeconds);

	virtual bool ShouldTickIfViewportsOnly() const override;
	virtual void Tick(float DeltaSeconds) override;

	UFUNCTION(BlueprintCallable, category = "Default")
	void SetEditorTickEnabled(bool bEnabled) { EditorTickIsEnabled = bEnabled; }

	UPROPERTY(Transient)
	bool EditorTickIsEnabled = false;

	UFUNCTION(BlueprintCallable, category = "Water Mesh Preview")
	void SpawnWaterPreviewGrid(UHierarchicalInstancedStaticMeshComponent* HISMC, float GridSize, int GridTiles);

	UFUNCTION(BlueprintCallable, category = "Water Mesh Preview")
	void SpawnCausticParticleGrid(UHierarchicalInstancedStaticMeshComponent* HISMC, float GridSize, int GridTiles);

	ACausticsGeneratorActor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());


};
