// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailLayoutBuilder.h"
#include "IDetailCustomization.h"
#include "WaterBodyActor.h"

class FWaterBodyActorDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FWaterBodyActorDetailCustomization>();
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		DetailBuilder.EditCategory(TEXT("Water"), FText::GetEmpty(), ECategoryPriority::TypeSpecific);
		DetailBuilder.EditCategory(TEXT("Terrain"), FText::GetEmpty(), ECategoryPriority::TypeSpecific);
		DetailBuilder.HideCategory(TEXT("Spline"));
		DetailBuilder.HideCategory(TEXT("Editor"));

		// TODO [jonathan.bard] : remove after deprecation : 
		DetailBuilder.HideProperty(TEXT("Wave Spectrum Settings")); // BP-property that was deprecated

		if (IsWaveSupported(DetailBuilder.GetSelectedObjects()))
		{
			DetailBuilder.EditCategory(TEXT("Wave"), FText::GetEmpty(), ECategoryPriority::TypeSpecific);
		}
		else
		{
			DetailBuilder.HideCategory(TEXT("Wave"));
		}
	}

private:

	bool IsWaveSupported(const TArray<TWeakObjectPtr<UObject>>& SelectedObjects) const
	{
		bool bWaveSupported = true;
		for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
		{
			if (AWaterBody* WaterBody = Cast<AWaterBody>(SelectedObject.Get()))
			{
				if (UWaterBodyComponent* WaterBodyComponent = WaterBody->GetWaterBodyComponent())
				{
					if (!WaterBodyComponent->IsWaveSupported())
					{
						bWaveSupported = false;
						break;
					}
				}
			}
		}
		return bWaveSupported;
	}
};