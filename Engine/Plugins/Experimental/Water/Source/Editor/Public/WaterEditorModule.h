// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"
#include "AssetTypeCategories.h"
#include "Toolkits/IToolkit.h"

class AActor;
class IAssetTypeActions;

class FComponentVisualizer;
class FWaterWavesEditorToolkit;

WATEREDITOR_API DECLARE_LOG_CATEGORY_EXTERN(LogWaterEditor, Log, All);

class IWaterEditorModuleInterface : public IModuleInterface
{
public:

};

class FWaterEditorModule : public IWaterEditorModuleInterface
{
public:
	virtual void StartupModule() override;

	virtual void ShutdownModule() override;

	static EAssetTypeCategories::Type GetAssetCategory() { return WaterAssetCategory; }

	TSharedRef<FWaterWavesEditorToolkit> CreateWaterWaveAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UObject* WavesAsset);

private:
	void RegisterComponentVisualizer(FName ComponentClassName, TSharedPtr<FComponentVisualizer> Visualizer);

	void OnLevelActorAddedToWorld(AActor* Actor);

	void CheckForWaterCollisionProfile();

	void AddWaterCollisionProfile();

private:
	/** Array of component class names we have registered, so we know what to unregister afterwards */
	TArray<FName> RegisteredComponentClassNames;

	static EAssetTypeCategories::Type WaterAssetCategory;

	TArray<TSharedPtr<IAssetTypeActions>> CreatedAssetTypeActions;

	FDelegateHandle OnLoadCollisionProfileConfigHandle;
};
