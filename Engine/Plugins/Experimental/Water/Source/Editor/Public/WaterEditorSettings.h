// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "WaterCurveSettings.h"
#include "WaterBodyHeightmapSettings.h"
#include "WaterBodyWeightmapSettings.h"
#include "WaterSplineMetadata.h"
#include "WaterEditorSettings.generated.h"

#define UE_API WATEREDITOR_API

enum TextureGroup : int;

class UMaterialInterface;
class UMaterialParameterCollection;
class UStaticMesh;
class UWaterBodyComponent;
class UWaterWavesBase;
class AWaterLandscapeBrush;
class AWaterZone;

USTRUCT()
struct FWaterBrushActorDefaults
{
	GENERATED_BODY()

	UE_API FWaterBrushActorDefaults();

	UPROPERTY(EditAnywhere, config, Category = Terrain)
	FWaterCurveSettings CurveSettings;

	UPROPERTY(EditAnywhere, config, Category = Terrain)
	FWaterBodyHeightmapSettings HeightmapSettings;

	UPROPERTY(EditAnywhere, config, Category = Terrain)
	TMap<FName, FWaterBodyWeightmapSettings> LayerWeightmapSettings;
};

USTRUCT()
struct FWaterZoneActorDefaults
{
	GENERATED_BODY()

	UE_API FWaterZoneActorDefaults();

	UE_API UMaterialInterface* GetFarDistanceMaterial() const;

	UPROPERTY(EditAnywhere, config, Category = Mesh)
	float FarDistanceMeshExtent = 4000000.0f;

	UPROPERTY(EditAnywhere, config, Category = Mesh)
	float NewWaterZoneScale = 1.0f;

	UPROPERTY(EditAnywhere, config, Category = Texture)
	FIntPoint RenderTargetResolution = FIntPoint(1024, 1024);

protected:
	UPROPERTY(EditAnywhere, Config, Category = Mesh)
	TSoftObjectPtr<UMaterialInterface> FarDistanceMaterial;
};


USTRUCT()
struct FWaterBodyDefaults
{
	GENERATED_BODY()

	UE_API FWaterBodyDefaults();

	UPROPERTY(EditAnywhere, config, Category = "Water Spline")
	FWaterSplineCurveDefaults SplineDefaults;

public: 
	UE_API UMaterialInterface* GetWaterMaterial() const;
	FSoftObjectPath GetWaterMaterialPath() const { return WaterMaterial.ToSoftObjectPath(); }

	UE_API UMaterialInterface* GetWaterStaticMeshMaterial() const;
	FSoftObjectPath GetWaterStaticMeshMaterialPath() const { return WaterStaticMeshMaterial.ToSoftObjectPath(); }

	UE_API UMaterialInterface* GetWaterHLODMaterial() const;
	FSoftObjectPath GetWaterHLODMaterialPath() const { return WaterHLODMaterial.ToSoftObjectPath(); }

	UE_API UMaterialInterface* GetUnderwaterPostProcessMaterial() const;
	FSoftObjectPath GetUnderwaterPostProcessMaterialPath() const { return UnderwaterPostProcessMaterial.ToSoftObjectPath(); }

protected:
	UPROPERTY(EditAnywhere, config, Category = Rendering)
	TSoftObjectPtr<UMaterialInterface> WaterMaterial;

	UPROPERTY(EditAnywhere, config, Category = Rendering)
	TSoftObjectPtr<UMaterialInterface> WaterStaticMeshMaterial;

	UPROPERTY(EditAnywhere, config, Category = Rendering)
	TSoftObjectPtr<UMaterialInterface> WaterHLODMaterial;

	UPROPERTY(EditAnywhere, config, Category = Rendering)
	TSoftObjectPtr<UMaterialInterface> UnderwaterPostProcessMaterial;
};

USTRUCT()
struct FWaterBodyRiverDefaults : public FWaterBodyDefaults
{
	GENERATED_BODY()

	UE_API FWaterBodyRiverDefaults();

	UPROPERTY(EditAnywhere, config, Category = Terrain)
	FWaterBrushActorDefaults BrushDefaults;

	UE_API UMaterialInterface* GetRiverToOceanTransitionMaterial() const;
	FSoftObjectPath GetRiverToOceanTransitionTransitionMaterialPath() const { return RiverToOceanTransitionMaterial.ToSoftObjectPath(); }

	UE_API UMaterialInterface* GetRiverToLakeTransitionMaterial() const;
	FSoftObjectPath GetRiverToLakeTransitionTransitionMaterialPath() const { return RiverToLakeTransitionMaterial.ToSoftObjectPath(); }

protected:
	UPROPERTY(EditAnywhere, config, Category = Rendering)
	TSoftObjectPtr<UMaterialInterface> RiverToOceanTransitionMaterial;

	UPROPERTY(EditAnywhere, config, Category = Rendering)
	TSoftObjectPtr<UMaterialInterface> RiverToLakeTransitionMaterial;
};


USTRUCT()
struct FWaterBodyLakeDefaults : public FWaterBodyDefaults
{
	GENERATED_BODY()
	
	UE_API FWaterBodyLakeDefaults();

	UPROPERTY(EditAnywhere, config, Category = Terrain)
	FWaterBrushActorDefaults BrushDefaults;

	UPROPERTY(EditAnywhere, Instanced, Category = Wave)
	TObjectPtr<UWaterWavesBase> WaterWaves = nullptr;
};


USTRUCT()
struct FWaterBodyOceanDefaults : public FWaterBodyDefaults
{
	GENERATED_BODY()

	UE_API FWaterBodyOceanDefaults();

	UPROPERTY(EditAnywhere, config, Category = Terrain)
	FWaterBrushActorDefaults BrushDefaults;

	UPROPERTY(EditAnywhere, Instanced, Category = Wave)
	TObjectPtr<UWaterWavesBase> WaterWaves = nullptr;
};


USTRUCT()
struct FWaterBodyCustomDefaults : public FWaterBodyDefaults
{
	GENERATED_BODY()

	UE_API FWaterBodyCustomDefaults();

	UE_API UStaticMesh* GetWaterMesh() const;
	FSoftObjectPath GetWaterMeshPath() const { return WaterMesh.ToSoftObjectPath(); }

private:
	UPROPERTY(EditAnywhere, config, Category = Rendering)
	TSoftObjectPtr<UStaticMesh> WaterMesh;
};


USTRUCT()
struct FWaterBodyIslandDefaults 
{
	GENERATED_BODY()

	UE_API FWaterBodyIslandDefaults();

	UPROPERTY(EditAnywhere, config, Category = Terrain)
	FWaterBrushActorDefaults BrushDefaults;
};


/**
 * Implements the editor settings for the Water plugin.
 */
UCLASS(MinimalAPI, config = Engine, defaultconfig, meta=(DisplayName="Water Editor"))
class UWaterEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UWaterEditorSettings();

	virtual FName GetCategoryName() const override { return FName(TEXT("Plugins")); }

	TSubclassOf<AWaterZone> GetWaterZoneClass() const;
	FSoftClassPath GetWaterZoneClassPath() const { return WaterZoneClassPath; }

	TSubclassOf<AWaterLandscapeBrush> GetWaterManagerClass() const;
	FSoftClassPath GetWaterManagerClassPath() const { return WaterManagerClassPath; }

	UMaterialInterface* GetDefaultBrushAngleFalloffMaterial() const;
	FSoftObjectPath GetDefaultBrushAngleFalloffMaterialPath() const { return DefaultBrushAngleFalloffMaterial.ToSoftObjectPath(); }

	UMaterialInterface* GetDefaultBrushIslandFalloffMaterial() const;
	FSoftObjectPath GetDefaultBrushIslandFalloffMaterialPath() const { return DefaultBrushIslandFalloffMaterial.ToSoftObjectPath(); }

	UMaterialInterface* GetDefaultBrushWidthFalloffMaterial() const;
	FSoftObjectPath GetDefaultBrushWidthFalloffMaterialPath() const { return DefaultBrushWidthFalloffMaterial.ToSoftObjectPath(); }

	UMaterialInterface* GetDefaultBrushWeightmapMaterial() const;
	FSoftObjectPath GetDefaultBrushWeightmapMaterialPath() const { return DefaultBrushWeightmapMaterial.ToSoftObjectPath(); }

	UMaterialInterface* GetDefaultCacheDistanceFieldCacheMaterial() const;
	FSoftObjectPath GetDefaultCacheDistanceFieldCacheMaterialPath() const { return DefaultCacheDistanceFieldCacheMaterial.ToSoftObjectPath(); }

	UMaterialInterface* GetDefaultCompositeWaterBodyTextureMaterial() const;
	FSoftObjectPath GetDefaultCompositeWaterBodyTextureMaterialPath() const { return DefaultCompositeWaterBodyTextureMaterial.ToSoftObjectPath(); }

	UMaterialInterface* GetDefaultJumpFloodStepMaterial() const;
	FSoftObjectPath GetDefaultJumpFloodStepMaterialPath() const { return DefaultJumpFloodStepMaterial.ToSoftObjectPath(); }

	UMaterialInterface* GetDefaultBlurEdgesMaterial() const;
	FSoftObjectPath GetDefaultBlurEdgesMaterialPath() const { return DefaultBlurEdgesMaterial.ToSoftObjectPath(); }

	UMaterialInterface* GetDefaultFindEdgesMaterial() const;
	FSoftObjectPath GetDefaultFindEdgesMaterialPath() const { return DefaultFindEdgesMaterial.ToSoftObjectPath(); }

	UMaterialInterface* GetDefaultDrawCanvasMaterial() const;
	FSoftObjectPath GetDefaultDrawCanvasMaterialPath() const { return DefaultDrawCanvasMaterial.ToSoftObjectPath(); }

	UMaterialInterface* GetDefaultRenderRiverSplineDepthsMaterial() const;
	FSoftObjectPath GetDefaultRenderRiverSplineDepthsMaterialPath() const { return DefaultRenderRiverSplineDepthsMaterial.ToSoftObjectPath(); }

	bool GetShouldUpdateWaterMeshDuringInteractiveChanges() const { return bUpdateWaterMeshDuringInteractiveChanges; }

public:
	/** The texture group to use for generated textures such as the combined velocity and height texture */
	UPROPERTY(EditAnywhere, config, Category = Rendering)
	TEnumAsByte<TextureGroup> TextureGroupForGeneratedTextures;

	/** Maximum size of the water velocity/height texture for a WaterZoneActor */
	UPROPERTY(EditAnywhere, config, Category = Rendering, meta=(ClampMin=1, ClampMax=2048))
	int32 MaxWaterVelocityAndHeightTextureSize;

	/** Scale factor for visualizing water velocity */
	UPROPERTY(EditAnywhere, Category = "Rendering", meta = (ClampMin = "0.1", UIMin = "0.1"))
	float VisualizeWaterVelocityScale = 20.0f;

	/** Material Parameter Collection for everything landscape-related */
	UPROPERTY(EditAnywhere, config, Category = Brush, AdvancedDisplay)
	TSoftObjectPtr<UMaterialParameterCollection> LandscapeMaterialParameterCollection;

	/** Default values for base WaterMesh actor*/
	UPROPERTY(EditAnywhere, config, Category = ActorDefaults)
	FWaterZoneActorDefaults WaterZoneActorDefaults;
	/** Default values for base WaterBodyRiver actor */
	UPROPERTY(EditAnywhere, config, Category = ActorDefaults)
	FWaterBodyRiverDefaults WaterBodyRiverDefaults;

	/** Default values for base WaterBodyLake actor */
	UPROPERTY(EditAnywhere, config, Category = ActorDefaults)
	FWaterBodyLakeDefaults WaterBodyLakeDefaults;

	/** Default values for base WaterBodyOcean actor */
	UPROPERTY(EditAnywhere, config, Category = ActorDefaults)
	FWaterBodyOceanDefaults WaterBodyOceanDefaults;

	/** Default values for base WaterBodyCustom actor */
	UPROPERTY(EditAnywhere, config, Category = ActorDefaults)
	FWaterBodyCustomDefaults WaterBodyCustomDefaults;

	/** Default values for base WaterBodyIsland actor */
	UPROPERTY(EditAnywhere, config, Category = ActorDefaults)
	FWaterBodyIslandDefaults WaterBodyIslandDefaults;

private:
	/** Allows the water mesh to be updated when the water body's shape is modified interactively (e.g. when dragging a spline point). Set to false if the performance when editing a water body gets too bad (the water mesh will be properly updated when the dragging operation is done). */
	UPROPERTY(EditAnywhere, config, Category = Brush)
	bool bUpdateWaterMeshDuringInteractiveChanges = false;

	/** Class of the water zone to be used*/
	UPROPERTY(EditAnywhere, config, Category = Water, meta = (MetaClass = "/Script/Water.WaterZone"))
	FSoftClassPath WaterZoneClassPath;

	/** Class of the water brush to be used in landscape */
	UPROPERTY(EditAnywhere, config, Category = Brush, meta = (MetaClass = "/Script/WaterEditor.WaterLandscapeBrush"), AdvancedDisplay)
	FSoftClassPath WaterManagerClassPath;

	UPROPERTY(EditAnywhere, config, Category = Brush, AdvancedDisplay)
	TSoftObjectPtr<UMaterialInterface> DefaultBrushAngleFalloffMaterial;

	UPROPERTY(EditAnywhere, config, Category = Brush, AdvancedDisplay)
	TSoftObjectPtr<UMaterialInterface> DefaultBrushIslandFalloffMaterial;

	UPROPERTY(EditAnywhere, config, Category = Brush, AdvancedDisplay)
	TSoftObjectPtr<UMaterialInterface> DefaultBrushWidthFalloffMaterial;

	UPROPERTY(EditAnywhere, config, Category = Brush, AdvancedDisplay)
	TSoftObjectPtr<UMaterialInterface> DefaultBrushWeightmapMaterial;

	UPROPERTY(EditAnywhere, config, Category = Brush, AdvancedDisplay)
	TSoftObjectPtr<UMaterialInterface> DefaultCacheDistanceFieldCacheMaterial;

	UPROPERTY(EditAnywhere, config, Category = Brush, AdvancedDisplay)
	TSoftObjectPtr<UMaterialInterface> DefaultCompositeWaterBodyTextureMaterial;

	UPROPERTY(EditAnywhere, config, Category = Brush, AdvancedDisplay)
	TSoftObjectPtr<UMaterialInterface> DefaultJumpFloodStepMaterial;

	UPROPERTY(EditAnywhere, config, Category = Brush, AdvancedDisplay)
	TSoftObjectPtr<UMaterialInterface> DefaultBlurEdgesMaterial;

	UPROPERTY(EditAnywhere, config, Category = Brush, AdvancedDisplay)
	TSoftObjectPtr<UMaterialInterface> DefaultFindEdgesMaterial;

	UPROPERTY(EditAnywhere, config, Category = Brush, AdvancedDisplay)
	TSoftObjectPtr<UMaterialInterface> DefaultDrawCanvasMaterial;

	UPROPERTY(EditAnywhere, config, Category = Brush, AdvancedDisplay)
	TSoftObjectPtr<UMaterialInterface> DefaultRenderRiverSplineDepthsMaterial;
};

#undef UE_API
