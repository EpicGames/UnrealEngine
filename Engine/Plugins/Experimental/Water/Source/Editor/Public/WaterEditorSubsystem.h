// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "WaterEditorServices.h"
#include "UnrealEdMisc.h"
#include "WaterEditorSubsystem.generated.h"

class AWaterZone;
class UTexture2D;
class UTextureRenderTarget2D;
class UWorld;
class UWaterSubsystem;
class UMaterialParameterCollection;

UCLASS()
class UWaterEditorSubsystem : public UEditorSubsystem, public IWaterEditorServices
{
	GENERATED_BODY()

	UWaterEditorSubsystem();
public:
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

	UMaterialParameterCollection* GetLandscapeMaterialParameterCollection() const {	return LandscapeMaterialParameterCollection; }

	//~ Begin IWaterEditorServices interface
	virtual void RegisterWaterActorSprite(UClass* InClass, UTexture2D* Texture) override;
	virtual UTexture2D* GetWaterActorSprite(UClass* InClass) const override;
	virtual UTexture2D* GetErrorSprite() const override { return ErrorSprite; }

	virtual bool TryMarkPackageAsModified(UPackage* ModifiedPackage) override;
	virtual bool HasAnyModifiedPackages() const override;
	virtual void ForEachModifiedPackage(const TFunctionRef<bool(UPackage*)>& Func) const override;
	virtual void ClearModifiedPackages() override;
	virtual void DirtyAllModifiedPackages() override;

	virtual bool GetShouldUpdateWaterMeshDuringInteractiveChanges() const override;
	//~ End IWaterEditorServices interface

private:
	UWorld* GetEditorWorld() const;

	void UpdateModifiedPackagesMessage();

	void OnPackageDirtied(UPackage* Package);
	void OnMapChanged(UWorld* InWorld, EMapChangeType InChangeType);

	UPROPERTY()
	TObjectPtr<UMaterialParameterCollection> LandscapeMaterialParameterCollection;

	UPROPERTY()
	TMap<TObjectPtr<UClass>, TObjectPtr<UTexture2D>> WaterActorSprites;
	UPROPERTY()
	TObjectPtr<UTexture2D> DefaultWaterActorSprite;
	UPROPERTY()
	TObjectPtr<UTexture2D> ErrorSprite;

	/** Set of water body packages which have been silently modified but not dirtied. */
	TSet<TWeakObjectPtr<UPackage>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<UPackage>>> PackagesNeedingDirtying;

	bool bSuppressOnDirtyEvents = false;
};
