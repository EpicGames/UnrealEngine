// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WaterEditor : ModuleRules
{
	public WaterEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "Core",
				"CoreUObject",
                "ComponentVisualizers",
				"DetailCustomizations",
				"Engine",
				"InputCore",
				"SlateCore",
				"Slate",
				"UnrealEd",
				"AssetDefinition",
				"Water",
                "Projects",
				"PropertyEditor",
				"Landscape",
				"LandscapeEditorUtilities",
				"EditorFramework",
				"EditorSubsystem",
				"ComponentVisualizers",
				"DeveloperSettings",
				"AdvancedPreviewScene",
				"PlacementMode",
				"ActionableMessage",
				"ToolMenus"
			});

		PublicDependencyModuleNames.AddRange(
			new string[] {
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			});
	}
}
