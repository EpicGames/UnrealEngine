// Copyright Epic Games, Inc. All Rights Reserved.
#include "BakedShallowWaterSimulationComponent.h"
#include "WaterBodyActor.h"
#include "WaterModule.h"
#include "WaterVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BakedShallowWaterSimulationComponent)

UBakedShallowWaterSimulationComponent::UBakedShallowWaterSimulationComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

void UShallowWaterSimulationDataBase::SampleShallowWaterSimulationAtIndex(const FVector2D& QueryFloatIndex, FVector& OutWaterVelocity, float& OutWaterHeight, float& OutWaterDepth) const
{
	OutWaterHeight = 0;
	OutWaterDepth = 0;
	OutWaterVelocity = FVector(0, 0, 0);
	
	if (QueryFloatIndex.X > 0 && QueryFloatIndex.X < NumCells.X - 1 && QueryFloatIndex.Y > 0 && QueryFloatIndex.Y < NumCells.Y - 1)
	{
		check(GetTotalNumCells() == NumCells.X * NumCells.Y);

		const FIntVector2 BaseIndex = FIntVector2(QueryFloatIndex.X, QueryFloatIndex.Y);
		const FVector2D LerpValue = QueryFloatIndex - FVector2D(BaseIndex.X, BaseIndex.Y);

		const FVector4 Sample00 = QueryLowLevelGridAtIndex(BaseIndex);
		const FVector4 Sample10 = QueryLowLevelGridAtIndex(BaseIndex + FIntVector2(1, 0));
		const FVector4 Sample01 = QueryLowLevelGridAtIndex(BaseIndex + FIntVector2(0, 1));
		const FVector4 Sample11 = QueryLowLevelGridAtIndex(BaseIndex + FIntVector2(1, 1));
		
		const FVector4 Sample0 = Sample00 * (1. - LerpValue.X) + Sample10 * (LerpValue.X);
		const FVector4 Sample1 = Sample01 * (1. - LerpValue.X) + Sample11 * (LerpValue.X);

		const FVector4 Sample = Sample0 * (1. - LerpValue.Y) + Sample1 * (LerpValue.Y);

		OutWaterHeight = Sample.X + Position.Z;
		OutWaterDepth = Sample.Y;
		OutWaterVelocity = FVector(Sample.Z, Sample.W, 0);
	}
}

void UShallowWaterSimulationDataBase::QueryShallowWaterSimulationAtIndex(const FIntVector2& QueryIndex, FVector& OutWaterVelocity, float& OutWaterHeight, float& OutWaterDepth) const
{
	OutWaterHeight = 0;
	OutWaterDepth = 0;
	OutWaterVelocity = FVector(0,0,0);

	if (QueryIndex.X >= 0 && QueryIndex.X < NumCells.X && QueryIndex.Y >= 0 && QueryIndex.Y < NumCells.Y)
	{
		check(GetTotalNumCells() == NumCells.X * NumCells.Y);

		const FVector4 Sample = QueryLowLevelGridAtIndex(QueryIndex);

		OutWaterHeight = Sample.X + Position.Z;
		OutWaterDepth = Sample.Y;
		OutWaterVelocity = FVector(Sample.Z, Sample.W, 0);
	}
}

FVector UShallowWaterSimulationDataBase::ComputeShallowWaterSimulationNormalAtPosition(const FVector& QueryPos) const
{
	const FVector2D FloatIndexPos = WorldToFloatIndex(QueryPos);

	float HRight;
	float HUp;
	float H;

	float Depth;
	float DepthRight;
	float DepthUp;

	FVector VelocityIGNORE;

	FVector Normal(0, 0, 1);
	
	SampleShallowWaterSimulationAtIndex(FloatIndexPos + FVector2D::ZeroVector, VelocityIGNORE, H, Depth);
	SampleShallowWaterSimulationAtIndex(FloatIndexPos + FVector2D(1,0), VelocityIGNORE, HRight, DepthRight);
	SampleShallowWaterSimulationAtIndex(FloatIndexPos + FVector2D(0, 1), VelocityIGNORE, HUp, DepthUp);

	FVector WorldPos = FloatIndexToWorld(FloatIndexPos + FVector2D::ZeroVector);
	FVector WorldPosRight = FloatIndexToWorld(FloatIndexPos + FVector2D(1, 0));
	FVector WorldPosUp = FloatIndexToWorld(FloatIndexPos + FVector2D(0, 1));

	WorldPos.Z += H;
	WorldPosRight.Z += HRight;
	WorldPosUp.Z += HUp;

	const FVector CrossProd = (WorldPosRight - WorldPos).Cross(WorldPosUp - WorldPos);
	const float CrossProdLength = CrossProd.Length();

	// default to upward facing normal
	if (CrossProdLength > SMALL_NUMBER)
	{
		Normal = CrossProd / CrossProdLength;
	}

	return Normal;
}

void UBakedShallowWaterSimulationComponent::PostLoad()
{
	Super::PostLoad();

  #if WITH_EDITORONLY_DATA
	if (GetLinkerCustomVersion(FWaterCustomVersion::GUID) <
		FWaterCustomVersion::MigrateShallowWaterSimulationToUObject)
	{
		if (BakedSimulationData == nullptr && SimulationData_DEPRECATED.IsValid())
		{
			BakedSimulationData = NewObject<UShallowWaterSimulationData>(this);

			UShallowWaterSimulationData *BakedSimulationDataDerived = Cast<UShallowWaterSimulationData>(BakedSimulationData);

			BakedSimulationDataDerived->ArrayValues = MoveTemp(SimulationData_DEPRECATED.ArrayValues);
			BakedSimulationDataDerived->NumCells = SimulationData_DEPRECATED.NumCells;
			BakedSimulationDataDerived->Position = SimulationData_DEPRECATED.Position;
			BakedSimulationDataDerived->Size = SimulationData_DEPRECATED.Size;
			BakedSimulationDataDerived->BakedTexture = SimulationData_DEPRECATED.BakedTexture;

			SimulationData_DEPRECATED = FShallowWaterSimulationGrid_DEPRECATED();

			UE_LOGF(LogWater, Warning, "Water simulation data was stored in deprecated format.  Converted to new format.  Recommended resave.");
		}
	}
#endif
}
