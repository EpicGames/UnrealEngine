// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuoyancyComponent.h"
#include "BuoyancyComponentSimulation.h"
#include "BuoyancyManager.h"
#include "Engine/World.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "WaterSplineComponent.h"
#include "WaterBodyComponent.h"
#include "WaterVersion.h"
#include "Physics/SimpleSuspension.h"
#include "PBDRigidsSolver.h"
#include "WaterSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BuoyancyComponent)

TAutoConsoleVariable<int32> CVarWaterDebugBuoyancy(
	TEXT("r.Water.DebugBuoyancy"),
	0,
	TEXT("Enable debug drawing for water interactions."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarWaterBuoyancyDebugPoints(
	TEXT("r.Water.BuoyancyDebugPoints"),
	10,
	TEXT("Number of points in one dimension for buoyancy debug."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarWaterBuoyancyDebugSize(
	TEXT("r.Water.BuoyancyDebugSize"),
	1000,
	TEXT("Side length of square for buoyancy debug."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarWaterUseSplineKeyOptimization(
	TEXT("r.Water.UseSplineKeyOptimization"),
	1,
	TEXT("Whether to cache spline input key for water bodies."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarWaterBuoyancyUseAsyncPath(
	TEXT("r.Water.UseBuoyancyAsyncPath"),
	1,
	TEXT("Whether to use async physics callback for buoyancy."),
	ECVF_Default);

UBuoyancyComponent::UBuoyancyComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, SimulatingComponent(nullptr)
	, PontoonConfiguration(0)
	, VelocityPontoonIndex(0)
	, bIsOverlappingWaterBody(false)
	, bCanBeActive(true)
	, bIsInWaterBody(false)
	, bUseAsyncPath(true)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = false;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;

	CurAsyncInput = nullptr;
	CurAsyncOutput = nullptr;
}

void UBuoyancyComponent::BeginPlay()
{
	Super::BeginPlay();
	if (AActor* Owner = GetOwner())
	{
		SimulatingComponent = Cast<UPrimitiveComponent>(Owner->GetRootComponent());
	}
	if(SimulatingComponent)
	{
		for (FSphericalPontoon& Pontoon : BuoyancyData.Pontoons)
		{
			if (Pontoon.CenterSocket != NAME_None && SimulatingComponent->DoesSocketExist(Pontoon.CenterSocket))
			{
				Pontoon.bUseCenterSocket = true;
				Pontoon.SocketTransform = SimulatingComponent->GetSocketTransform(Pontoon.CenterSocket, RTS_Actor);
			}
		}
		SetupWaterBodyOverlaps();
	}

	// Call this before registering with manager
	FinalizeAuxData();

	if (UWorld* World = GetWorld())
	{
		if (UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(World))
		{
			if (ABuoyancyManager* Manager = WaterSubsystem->GetBuoyancyManager())
			{
				Manager->Register(this);
			}
		}
	}
}

void UBuoyancyComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (UWorld* World = GetWorld())
	{
		if (UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(World))
		{
			if (ABuoyancyManager* Manager = WaterSubsystem->GetBuoyancyManager())
			{
				Manager->Unregister(this);
			}
		}
	}
	Super::EndPlay(EndPlayReason);
}

void UBuoyancyComponent::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FWaterCustomVersion::GUID) < FWaterCustomVersion::UpdateBuoyancyComponentPontoonsData)
	{
		if (Pontoons_DEPRECATED.Num())
		{
			BuoyancyData.Pontoons = Pontoons_DEPRECATED;
		}
		Pontoons_DEPRECATED.Empty();
	}
}

void UBuoyancyComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FWaterCustomVersion::GUID);
}

const float ToKmh(float SpeedCms)
{
	return SpeedCms * 0.036f; //cm/s to km/h
}

void UBuoyancyComponent::Update(float DeltaTime)
{
	if (!SimulatingComponent)
	{
		return;
	}

	UpdatePontoonCoefficients();

	if (const FBuoyancyComponentBaseAsyncOutput* Output = static_cast<FBuoyancyComponentBaseAsyncOutput*>(CurAsyncOutput))
	{
		if (Output->bValid)
		{
			const FBuoyancySimOutput& SimOut = Output->SimOutput;
			bIsInWaterBody = SimOut.bIsInWaterBody;

			// We may have deleted/added a pontoon on the game thread
			int32 PontoonsNum = FMath::Min(BuoyancyData.Pontoons.Num(), Output->AuxData.Pontoons.Num());
			for (int32 i = 0; i < PontoonsNum; ++i)
			{
				BuoyancyData.Pontoons[i].CopyDataFromPT(Output->AuxData.Pontoons[i]);
			}
		}
	}

	if (CurAsyncInput)
	{
		if (const FBodyInstance* BodyInstance = SimulatingComponent->GetBodyInstance())
		{
			if (auto Handle = BodyInstance->GetPhysicsActor())
			{
				CurAsyncInput->Proxy = Handle;
			}
		}

		FBuoyancyComponentBaseAsyncInput* BuoyancyInputState = static_cast<FBuoyancyComponentBaseAsyncInput*>(CurAsyncInput);

		BuoyancyInputState->WaterBodyComponents = GetCurrentWaterBodyComponents();
		BuoyancyInputState->Pontoons = BuoyancyData.Pontoons;
		bool bSetSmoothedTime = false;
		for (UWaterBodyComponent* WaterBody : GetCurrentWaterBodyComponents())
		{
			if (WaterBody && WaterBody->HasWaves())
			{
				BuoyancyInputState->SmoothedWorldTimeSeconds = WaterBody->GetWaveReferenceTime();
				bSetSmoothedTime = true;
				break;
			}
		}
		if(!bSetSmoothedTime)
		{
			BuoyancyInputState->SmoothedWorldTimeSeconds = GetWorld()->GetTimeSeconds();
		}
	}

	if (!IsUsingAsyncPath())
	{
		const FVector PhysicsVelocity = SimulatingComponent->GetComponentVelocity();

		const FVector ForwardDir = SimulatingComponent->GetForwardVector();
		const FVector RightDir = SimulatingComponent->GetRightVector();

		const float ForwardSpeed = FVector::DotProduct(ForwardDir, PhysicsVelocity);
		const float ForwardSpeedKmh = ToKmh(ForwardSpeed);

		const float RightSpeed = FVector::DotProduct(RightDir, PhysicsVelocity);
		const float RightSpeedKmh = ToKmh(RightSpeed);
		ApplyForces(DeltaTime, PhysicsVelocity, ForwardSpeed, ForwardSpeedKmh, SimulatingComponent);
	}
}

void UBuoyancyComponent::ApplyForces(float DeltaTime, FVector LinearVelocity, float ForwardSpeed, float ForwardSpeedKmh, UPrimitiveComponent* PrimitiveComponent)
{
	if (IsUsingAsyncPath())
	{
		return;
	}
	
	const int32 NumPontoonsInWater = UpdatePontoons(DeltaTime, ForwardSpeed, ForwardSpeedKmh, SimulatingComponent);
	bIsInWaterBody = NumPontoonsInWater > 0;

	if (SimulatingComponent->IsSimulatingPhysics())
	{
		const ECollisionEnabled::Type Collision = SimulatingComponent->GetCollisionEnabled();
		if (Collision == ECollisionEnabled::QueryAndPhysics || Collision == ECollisionEnabled::PhysicsOnly)
		{
			ApplyBuoyancy(SimulatingComponent);

			FVector TotalForce = FVector::ZeroVector;
			FVector TotalTorque = FVector::ZeroVector;

			TotalForce += ComputeWaterForce(DeltaTime, LinearVelocity);

			if (BuoyancyData.bApplyDragForcesInWater)
			{
				TotalForce += ComputeLinearDragForce(LinearVelocity);
				TotalTorque += ComputeAngularDragTorque(SimulatingComponent->GetPhysicsAngularVelocityInDegrees());
			}

			SimulatingComponent->AddForce(TotalForce, NAME_None, /*bAccelChange=*/true);
			SimulatingComponent->AddTorqueInDegrees(TotalTorque, NAME_None, /*bAccelChange=*/true);
		}
	}
}

void UBuoyancyComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	Update(DeltaTime);
}

void UBuoyancyComponent::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// Account for all non-editor data properties :
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(CurAsyncInput) + sizeof(CurAsyncOutput) + sizeof(NextAsyncOutput) + sizeof(CurAsyncType) + sizeof(OutputInterpAlpha) + OutputsWaitingOn.GetAllocatedSize()
		+ sizeof(PontoonConfiguration) + ConfiguredPontoonCoefficients.GetAllocatedSize() + sizeof(VelocityPontoonIndex));
}

void UBuoyancyComponent::SetupWaterBodyOverlaps()
{
	//SimulatingComponent->SetCollisionObjectType(ECollisionChannel::ECC_PhysicsBody);
	if (SimulatingComponent->GetCollisionResponseToChannel(ECollisionChannel::ECC_WorldStatic) == ECollisionResponse::ECR_Ignore)
	{
		SimulatingComponent->SetCollisionResponseToChannel(ECollisionChannel::ECC_WorldStatic, ECollisionResponse::ECR_Overlap);
	}
	SimulatingComponent->SetGenerateOverlapEvents(true);
}

void UBuoyancyComponent::AddCustomPontoon(float Radius, FName CenterSocketName)
{
	FSphericalPontoon Pontoon;
	Pontoon.Radius = Radius;
	Pontoon.CenterSocket = CenterSocketName;
	BuoyancyData.Pontoons.Add(Pontoon);
}

void UBuoyancyComponent::AddCustomPontoon(float Radius, const FVector& RelativeLocation)
{
	FSphericalPontoon Pontoon;
	Pontoon.Radius = Radius;
	Pontoon.RelativeLocation = RelativeLocation;
	BuoyancyData.Pontoons.Add(Pontoon);
}

void UBuoyancyComponent::EnteredWaterBody(UWaterBodyComponent* WaterBodyComponent)
{
	bool bIsFirstBody = !CurrentWaterBodyComponents.Num() && WaterBodyComponent;
	CurrentWaterBodyComponents.AddUnique(WaterBodyComponent);
	for (FSphericalPontoon& Pontoon : BuoyancyData.Pontoons)
	{
		Pontoon.SplineSegments.FindOrAdd(WaterBodyComponent, -1);
	}
	if (bIsFirstBody)
	{
		bIsOverlappingWaterBody = -1;
	}
}

void UBuoyancyComponent::ExitedWaterBody(UWaterBodyComponent* WaterBodyComponent)
{
	GetCurrentWaterBodyComponents().Remove(WaterBodyComponent);
	for (FSphericalPontoon& Pontoon : BuoyancyData.Pontoons)
	{
		Pontoon.SplineSegments.Remove(WaterBodyComponent);
	}
	if (!CurrentWaterBodyComponents.Num())
	{
		bIsOverlappingWaterBody = false;
		bIsInWaterBody = false;
	}
}

void UBuoyancyComponent::ApplyBuoyancy(UPrimitiveComponent* PrimitiveComponent)
{
	check(GetOwner());

	if (PrimitiveComponent && bIsOverlappingWaterBody)
	{
		int PontoonIndex = 0;
		for (const FSphericalPontoon& Pontoon : BuoyancyData.Pontoons)
		{
			if (PontoonConfiguration & (1 << PontoonIndex))
			{
				PrimitiveComponent->AddForceAtLocation(Pontoon.LocalForce, Pontoon.CenterLocation);
			}
			PontoonIndex++;
		}
	}
}

void UBuoyancyComponent::ComputeBuoyancy(FSphericalPontoon& Pontoon, float ForwardSpeedKmh)
{
	AActor* Owner = GetOwner();
	check(Owner);

	auto ComputeBuoyantForce = [&](FVector CenterLocation, float Radius, float InBuoyancyCoefficient, float CurrentWaterLevel) -> float
	{
		const float Bottom = CenterLocation.Z - Radius;
		const float SubDiff = FMath::Clamp(CurrentWaterLevel - Bottom, 0.f, 2.f * Radius);

		// The following was obtained by integrating the volume of a sphere
		// over a linear section of SubmersionDiff length.
		static const float Pi = (float)PI;
		const float SubDiffSq = SubDiff * SubDiff;
		const float SubVolume = (Pi / 3.f) * SubDiffSq * ((3.f * Radius) - SubDiff);

#if ENABLE_DRAW_DEBUG
		if (CVarWaterDebugBuoyancy.GetValueOnAnyThread())
		{
			const FVector WaterPoint = FVector(CenterLocation.X, CenterLocation.Y, CurrentWaterLevel);
			DrawDebugLine(GetWorld(), WaterPoint - 50.f * FVector::ForwardVector, WaterPoint + 50.f * FVector::ForwardVector, FColor::Blue, false, -1.f, 0, 3.f);
			DrawDebugLine(GetWorld(), WaterPoint - 50.f * FVector::RightVector, WaterPoint + 50.f * FVector::RightVector, FColor::Blue, false, -1.f, 0, 3.f);
		}
#endif

		check(SimulatingComponent && SimulatingComponent->GetBodyInstance());
		const float VelocityZ = SimulatingComponent->GetBodyInstance()->GetUnrealWorldVelocity().Z;
		const float FirstOrderDrag = BuoyancyData.BuoyancyDamp * VelocityZ;
		const float SecondOrderDrag = FMath::Sign(VelocityZ) * BuoyancyData.BuoyancyDamp2 * VelocityZ * VelocityZ;
		const float DampingFactor = -FMath::Max(FirstOrderDrag + SecondOrderDrag, 0.f);
		// The buoyant force scales with submersed volume
		return SubVolume * (InBuoyancyCoefficient) + DampingFactor;
	};

	const float MinVelocity = BuoyancyData.BuoyancyRampMinVelocity;
	const float MaxVelocity = BuoyancyData.BuoyancyRampMaxVelocity;
	const float RampFactor = FMath::Clamp((ForwardSpeedKmh - MinVelocity) / (MaxVelocity - MinVelocity), 0.f, 1.f);
	const float BuoyancyRamp = RampFactor * (BuoyancyData.BuoyancyRampMax - 1);
	float BuoyancyCoefficientWithRamp = BuoyancyData.BuoyancyCoefficient * (1 + BuoyancyRamp);

	const float BuoyantForce = FMath::Clamp(ComputeBuoyantForce(Pontoon.CenterLocation, Pontoon.Radius, BuoyancyCoefficientWithRamp, Pontoon.WaterHeight), 0.f, BuoyancyData.MaxBuoyantForce);
	Pontoon.LocalForce = FVector::UpVector * BuoyantForce * Pontoon.PontoonCoefficient;
}

void UBuoyancyComponent::ComputePontoonCoefficients()
{
	TArray<float>& PontoonCoefficients = ConfiguredPontoonCoefficients.FindOrAdd(PontoonConfiguration);
	if (PontoonCoefficients.Num() == 0)
	{
		if (FBodyInstance* BodyInstance = SimulatingComponent->GetBodyInstance())
		{
			TArray<FVector> LocalPontoonLocations;
			const FVector& LocalCOM = BodyInstance->GetMassSpaceLocal().GetLocation();

			if (!SimulatingComponent)
			{
				return;
			}

			for (int32 PontoonIndex = 0; PontoonIndex < BuoyancyData.Pontoons.Num(); ++PontoonIndex)
			{
				const FSphericalPontoon& Pontoon = BuoyancyData.Pontoons[PontoonIndex];
				if (PontoonConfiguration & (1 << PontoonIndex))
				{
					if (Pontoon.bUseCenterSocket)
					{
						const FVector LocalPosition = SimulatingComponent->GetSocketTransform(Pontoon.CenterSocket, ERelativeTransformSpace::RTS_ParentBoneSpace).GetLocation();
						LocalPontoonLocations.Add(LocalPosition);
					}
					else
					{
						// If using the relative location for the pontoon and the buoyancydata indicates
						// that we should center on COM, then shift the relative location to be centered.
						const FVector PontoonRelativeLocation
							= BuoyancyData.bCenterPontoonsOnCOM
							? Pontoon.RelativeLocation - LocalCOM
							: Pontoon.RelativeLocation;
						LocalPontoonLocations.Add(PontoonRelativeLocation);
					}
				}
			}

			PontoonCoefficients.AddZeroed(LocalPontoonLocations.Num());
			//Distribute a mass of 1 to each pontoon so that we get a scaling factor based on position relative to CoM
			FString ErrMsg;
			bool ComputeSuccess = FSimpleSuspensionHelpers::ComputeSprungMasses(LocalPontoonLocations, LocalCOM, 1.f, PontoonCoefficients, &ErrMsg);
			ensureMsgf(ComputeSuccess, TEXT("Failed to compute %d sprung masses for: %s\nErrMsg: \"%s\""), LocalPontoonLocations.Num(), *GetOwner()->GetActorNameOrLabel(), *ErrMsg);
		}
	}

	// Apply the coefficients
	for (int32 PontoonIndex = 0, CoefficientIdx = 0; PontoonIndex < BuoyancyData.Pontoons.Num(); ++PontoonIndex)
	{
		if (PontoonConfiguration & (1 << PontoonIndex))
		{
			BuoyancyData.Pontoons[PontoonIndex].PontoonCoefficient = PontoonCoefficients[CoefficientIdx++];
		}
	}
}

int32 UBuoyancyComponent::UpdatePontoons(float DeltaTime, float ForwardSpeed, float ForwardSpeedKmh, UPrimitiveComponent* PrimitiveComponent)
{
	AActor* Owner = GetOwner();
	check(Owner);

	int32 NumPontoonsInWater = 0;
	if (bIsOverlappingWaterBody)
	{
		int PontoonIndex = 0;
		for (FSphericalPontoon& Pontoon : BuoyancyData.Pontoons)
		{
			if (PontoonConfiguration & (1 << PontoonIndex))
			{
				if (Pontoon.bUseCenterSocket)
				{
					const FTransform& SimulatingComponentTransform = PrimitiveComponent->GetSocketTransform(Pontoon.CenterSocket);
					Pontoon.CenterLocation = SimulatingComponentTransform.GetLocation() + Pontoon.Offset;
					Pontoon.SocketRotation = SimulatingComponentTransform.GetRotation();
				}
				else
				{
					Pontoon.CenterLocation = PrimitiveComponent->GetComponentTransform().TransformPosition(Pontoon.RelativeLocation);
				}
				GetWaterSplineKey(Pontoon.CenterLocation, Pontoon.SplineInputKeys, Pontoon.SplineSegments);
				const FVector PontoonBottom = Pontoon.CenterLocation - FVector(0, 0, Pontoon.Radius);
				UWaterBodyComponent* TempWaterBodyComponent = Pontoon.CurrentWaterBodyComponent;
				/*Pass in large negative default value so we don't accidentally assume we're in water when we're not.*/
				Pontoon.WaterHeight = GetWaterHeight(PontoonBottom - FVector::UpVector * 100.f, Pontoon.SplineInputKeys, -100000.f, TempWaterBodyComponent, Pontoon.WaterDepth, Pontoon.WaterPlaneLocation, Pontoon.WaterPlaneNormal, Pontoon.WaterSurfacePosition, Pontoon.WaterVelocity, Pontoon.WaterBodyIndex);
				Pontoon.CurrentWaterBodyComponent = TempWaterBodyComponent;

				const bool bPrevIsInWater = Pontoon.bIsInWater;
				const float ImmersionDepth = Pontoon.WaterHeight - PontoonBottom.Z;
				/*check if the pontoon is currently in water*/
				if (ImmersionDepth >= 0.f)
				{
					Pontoon.bIsInWater = true;
					Pontoon.ImmersionDepth = ImmersionDepth;
					NumPontoonsInWater++;
				}
				else
				{
					Pontoon.bIsInWater = false;
					Pontoon.ImmersionDepth = 0.f;
				}

#if ENABLE_DRAW_DEBUG
				if (CVarWaterDebugBuoyancy.GetValueOnAnyThread())
				{
					DrawDebugSphere(GetWorld(), Pontoon.CenterLocation, Pontoon.Radius, 16, FColor::Red, false, -1.f, 0, 1.f);
				}
#endif
				ComputeBuoyancy(Pontoon, ForwardSpeedKmh);

				if (Pontoon.bIsInWater && !bPrevIsInWater)
				{
					Pontoon.SplineSegments.Reset();
					// BlueprintImplementables don't really work on the actor component level unfortunately, so call back in to the function defined on the actor itself.
					OnPontoonEnteredWater(Pontoon);
				}
				if (!Pontoon.bIsInWater && bPrevIsInWater)
				{
					Pontoon.SplineSegments.Reset();
					OnPontoonExitedWater(Pontoon);
				}
			}
			PontoonIndex++;
		}

#if ENABLE_DRAW_DEBUG
		if (CVarWaterDebugBuoyancy.GetValueOnAnyThread())
		{
			const float NumPoints = CVarWaterBuoyancyDebugPoints.GetValueOnAnyThread();
			const float Size = CVarWaterBuoyancyDebugSize.GetValueOnAnyThread();
			const float StartOffset = NumPoints * 0.5f;
			const float Scale = Size / NumPoints;
			TMap<const UWaterBodyComponent*, float> DebugSplineKeyMap;
			TMap<const UWaterBodyComponent*, float> DebugSplineSegmentsMap;
			for (int i = 0; i < NumPoints; ++i)
			{
				for (int j = 0; j < NumPoints; ++j)
				{
					FVector Location = PrimitiveComponent->GetComponentLocation() + (FVector::RightVector * (i - StartOffset) * Scale) + (FVector::ForwardVector * (j - StartOffset) * Scale);
					GetWaterSplineKey(Location, DebugSplineKeyMap, DebugSplineSegmentsMap);
					FVector Point(Location.X, Location.Y, GetWaterHeight(Location - FVector::UpVector * 200.f, DebugSplineKeyMap, GetOwner()->GetActorLocation().Z));
					DrawDebugPoint(GetWorld(), Point, 5.f, IsOverlappingWaterBody() ? FColor::Green : FColor::Red, false, -1.f, 0);
				}
			}
		}
#endif
	}
	return NumPontoonsInWater;
}

float GetWaterSplineKeyFast(FVector Location, const UWaterBodyComponent* WaterBodyComponent, TMap<const UWaterBodyComponent*, float>& OutSegmentMap)/*const*/
{
	if (!OutSegmentMap.Contains(WaterBodyComponent))
	{
		OutSegmentMap.Add(WaterBodyComponent, -1);
	}

	const UWaterSplineComponent* WaterSpline = WaterBodyComponent->GetWaterSpline();
	const FVector LocalLocation = WaterBodyComponent->GetComponentTransform().InverseTransformPosition(Location);
	const FInterpCurveVector& InterpCurve = WaterSpline->GetSplinePointsPosition();
	float& Segment = OutSegmentMap[WaterBodyComponent];

	if (Segment == -1)
	{
		float DummyDistance;
		return InterpCurve.FindNearest(LocalLocation, DummyDistance, Segment);
	}

	//We have the cached segment, so search for the best point as in FInterpCurve<T>::FindNearest
	//but only in the current segment and the two immediate neighbors

	//River splines aren't looped, so we don't have to handle that case
	const int32 NumPoints = InterpCurve.Points.Num();
	const int32 LastSegmentIdx = FMath::Max(0, NumPoints - 2);
	if (NumPoints > 1)
	{
		float BestDistanceSq = BIG_NUMBER;
		float BestResult = BIG_NUMBER;
		float BestSegment = Segment;
		for (int32 i = Segment - 1; i <= Segment + 1; ++i)
		{
			const int32 SegmentIdx = FMath::Clamp(i, 0, LastSegmentIdx);
			float LocalDistanceSq;
			float LocalResult = InterpCurve.FindNearestOnSegment(LocalLocation, SegmentIdx, LocalDistanceSq);
			if (LocalDistanceSq < BestDistanceSq)
			{
				BestDistanceSq = LocalDistanceSq;
				BestResult = LocalResult;
				BestSegment = SegmentIdx;
			}
		}

		if (FMath::IsNearlyEqual(BestResult, Segment - 1) || FMath::IsNearlyEqual(BestResult, Segment + 1))
		{
			//We're at either end of the search - it's possible we skipped a segment so just do a full lookup in this case
			float DummyDistance;
			return InterpCurve.FindNearest(LocalLocation, DummyDistance, Segment);
		}

		Segment = BestSegment;
		return BestResult;
	}

	if (NumPoints == 1)
	{
		Segment = 0;
		return InterpCurve.Points[0].InVal;
	}

	return 0.0f;
}

void UBuoyancyComponent::GetWaterSplineKey(FVector Location, TMap<const UWaterBodyComponent*, float>& OutMap, TMap<const UWaterBodyComponent*, float>& OutSegmentMap) const
{
	OutMap.Reset();
	for (const UWaterBodyComponent* WaterBodyComponent : CurrentWaterBodyComponents)
	{
		if (WaterBodyComponent && WaterBodyComponent->GetWaterBodyType() == EWaterBodyType::River)
		{
			float SplineInputKey;
			if (CVarWaterUseSplineKeyOptimization.GetValueOnAnyThread())
			{
				SplineInputKey = GetWaterSplineKeyFast(Location, WaterBodyComponent, OutSegmentMap);
			}
			else
			{
				SplineInputKey = WaterBodyComponent->FindInputKeyClosestToWorldLocation(Location);
			}
			OutMap.Add(WaterBodyComponent, SplineInputKey);
		}
	}
}

float UBuoyancyComponent::GetWaterHeight(FVector Position, const TMap<const UWaterBodyComponent*, float>& SplineKeyMap, float DefaultHeight, UWaterBodyComponent*& OutWaterBodyComponent, float& OutWaterDepth, FVector& OutWaterPlaneLocation, FVector& OutWaterPlaneNormal, FVector& OutWaterSurfacePosition, FVector& OutWaterVelocity, int32& OutWaterBodyIdx, bool bShouldIncludeWaves)
{
	float WaterHeight = DefaultHeight;
	OutWaterBodyComponent = nullptr;
	OutWaterDepth = 0.f;
	OutWaterPlaneLocation = FVector::ZeroVector;
	OutWaterPlaneNormal = FVector::UpVector;

	float MaxImmersionDepth = -1.f;
	for (UWaterBodyComponent* CurrentWaterBodyComponent : CurrentWaterBodyComponents)
	{
		if (CurrentWaterBodyComponent)
		{
			const float SplineInputKey = SplineKeyMap.FindRef(CurrentWaterBodyComponent);

			EWaterBodyQueryFlags QueryFlags =
				EWaterBodyQueryFlags::ComputeLocation
				| EWaterBodyQueryFlags::ComputeNormal
				| EWaterBodyQueryFlags::ComputeImmersionDepth
				| EWaterBodyQueryFlags::ComputeVelocity;

			if (bShouldIncludeWaves)
			{
				QueryFlags |= EWaterBodyQueryFlags::IncludeWaves;
			}

			TValueOrError<FWaterBodyQueryResult, EWaterBodyQueryError> QueryResult = CurrentWaterBodyComponent->TryQueryWaterInfoClosestToWorldLocation(Position, QueryFlags, SplineInputKey);
			if (QueryResult.HasValue())
			{
				const FWaterBodyQueryResult& Query = QueryResult.GetValue();
				if (Query.IsInWater() && Query.GetImmersionDepth() > MaxImmersionDepth)
				{
					check(!Query.IsInExclusionVolume());
					WaterHeight = Position.Z + Query.GetImmersionDepth();
					OutWaterBodyComponent = CurrentWaterBodyComponent;
					if (EnumHasAnyFlags(Query.GetQueryFlags(), EWaterBodyQueryFlags::ComputeDepth))
					{
						OutWaterDepth = Query.GetWaterSurfaceDepth();
					}
					OutWaterPlaneLocation = Query.GetWaterPlaneLocation();
					OutWaterPlaneNormal = Query.GetWaterPlaneNormal();
					OutWaterSurfacePosition = Query.GetWaterSurfaceLocation();
					OutWaterVelocity = Query.GetVelocity();
					OutWaterBodyIdx = CurrentWaterBodyComponent ? CurrentWaterBodyComponent->GetWaterBodyIndex() : 0;
					MaxImmersionDepth = Query.GetImmersionDepth();
				}
			}
		}
	}
	return WaterHeight;
}

float UBuoyancyComponent::GetWaterHeight(FVector Position, const TMap<const UWaterBodyComponent*, float>& SplineKeyMap, float DefaultHeight, bool bShouldIncludeWaves /*= true*/)
{
	UWaterBodyComponent* DummyComponent;
	float DummyDepth;
	FVector DummyWaterPlaneLocation;
	FVector DummyWaterPlaneNormal;
	FVector DummyWaterSurfacePosition;
	FVector DummyWaterVelocity;
	int32 DummyWaterBodyIndex;
	return GetWaterHeight(Position, SplineKeyMap, DefaultHeight, DummyComponent, DummyDepth, DummyWaterPlaneLocation, DummyWaterPlaneNormal, DummyWaterSurfacePosition, DummyWaterVelocity, DummyWaterBodyIndex, bShouldIncludeWaves);
}

void UBuoyancyComponent::OnPontoonEnteredWater(const FSphericalPontoon& Pontoon)
{
	OnEnteredWaterDelegate.Broadcast(Pontoon);
}

void UBuoyancyComponent::OnPontoonExitedWater(const FSphericalPontoon& Pontoon)
{
	OnExitedWaterDelegate.Broadcast(Pontoon);
}

void UBuoyancyComponent::GetLastWaterSurfaceInfo(FVector& OutWaterPlaneLocation, FVector& OutWaterPlaneNormal, FVector& OutWaterSurfacePosition, float& OutWaterDepth, int32& OutWaterBodyIdx, FVector& OutWaterVelocity)
{
	for (int i = 0; i < BuoyancyData.Pontoons.Num(); ++i)
	{
		if (BuoyancyData.Pontoons[i].ImmersionDepth > 0.0f)
		{
			OutWaterPlaneLocation = BuoyancyData.Pontoons[i].WaterPlaneLocation;
			OutWaterPlaneNormal = BuoyancyData.Pontoons[i].WaterPlaneNormal;
			OutWaterSurfacePosition = BuoyancyData.Pontoons[i].WaterSurfacePosition;
			OutWaterDepth = BuoyancyData.Pontoons[i].WaterDepth;
			OutWaterBodyIdx = BuoyancyData.Pontoons[i].WaterBodyIndex;
			OutWaterVelocity = BuoyancyData.Pontoons[i].WaterVelocity;
			return;
		}
	}
}

void UBuoyancyComponent::UpdatePontoonCoefficients()
{
	// Get current configuration mask
	uint32 NewPontoonConfiguration = 0;
	for (int32 PontoonIndex = 0; PontoonIndex < BuoyancyData.Pontoons.Num(); ++PontoonIndex)
	{
		if (BuoyancyData.Pontoons[PontoonIndex].bEnabled)
		{
			NewPontoonConfiguration |= 1 << PontoonIndex;
		}
	}

	// Store the new configuration, and return true if its value has changed.
	const bool bConfigurationChanged = PontoonConfiguration != NewPontoonConfiguration;
	PontoonConfiguration = NewPontoonConfiguration;

	// If the configuration changed, update coefficients
	if (bConfigurationChanged)
	{
		// Apply new configuration, recomputing coefficients if necessary
		ComputePontoonCoefficients();
	}
}

FVector UBuoyancyComponent::ComputeWaterForce(const float DeltaTime, const FVector LinearVelocity) const
{
	AActor* Owner = GetOwner();
	check(Owner);

	if (BuoyancyData.Pontoons.Num())
	{
		const FSphericalPontoon& Pontoon = BuoyancyData.Pontoons[VelocityPontoonIndex];
		const UWaterBodyComponent* WaterBodyComponent = Pontoon.CurrentWaterBodyComponent;;
		if (WaterBodyComponent && WaterBodyComponent->GetWaterBodyType() == EWaterBodyType::River)
		{
			float InputKey = Pontoon.SplineInputKeys[WaterBodyComponent];
			const float WaterSpeed = WaterBodyComponent->GetWaterVelocityAtSplineInputKey(InputKey);

			const FVector SplinePointLocation = WaterBodyComponent->GetWaterSpline()->GetLocationAtSplineInputKey(InputKey, ESplineCoordinateSpace::World);
			// Move away from spline
			const FVector ShoreDirection = (Pontoon.CenterLocation - SplinePointLocation).GetSafeNormal2D();

			const float WaterShorePushFactor = BuoyancyData.WaterShorePushFactor;
			const FVector WaterDirection = WaterBodyComponent->GetWaterSpline()->GetDirectionAtSplineInputKey(InputKey, ESplineCoordinateSpace::World) * (1 - WaterShorePushFactor)
				+ ShoreDirection * (WaterShorePushFactor);
			const FVector WaterVelocity = WaterDirection * WaterSpeed;
			check(SimulatingComponent && SimulatingComponent->GetBodyInstance());
			const FVector ActorVelocity = SimulatingComponent->GetBodyInstance()->GetUnrealWorldVelocity();
			const float ActorSpeedInWaterDir = FMath::Abs(FVector::DotProduct(ActorVelocity, WaterDirection));
			if (ActorSpeedInWaterDir < WaterSpeed)
			{
				const FVector Acceleration = (WaterVelocity / DeltaTime) * BuoyancyData.WaterVelocityStrength;
				const float MaxWaterAcceleration = BuoyancyData.MaxWaterForce;
				return Acceleration.GetClampedToSize(-MaxWaterAcceleration, MaxWaterAcceleration);
			}
		}
	}
	return FVector::ZeroVector;
}
FVector UBuoyancyComponent::ComputeLinearDragForce(const FVector& PhyiscsVelocity) const
{
	FVector DragForce = FVector::ZeroVector;
	if (BuoyancyData.bApplyDragForcesInWater && IsInWaterBody() && SimulatingComponent)
	{
		FVector PlaneVelocity = PhyiscsVelocity;
		PlaneVelocity.Z = 0;
		const FVector VelocityDir = PlaneVelocity.GetSafeNormal();
		const float SpeedKmh = ToKmh(PlaneVelocity.Size());
		const float ClampedSpeed = FMath::Clamp(SpeedKmh, -BuoyancyData.MaxDragSpeed, BuoyancyData.MaxDragSpeed);

		const float Resistance = ClampedSpeed * BuoyancyData.DragCoefficient;
		DragForce += -Resistance * VelocityDir;

		const float Resistance2 = ClampedSpeed * ClampedSpeed * BuoyancyData.DragCoefficient2;
		DragForce += -Resistance2 * VelocityDir * FMath::Sign(SpeedKmh);
	}
	return DragForce;
}

FVector UBuoyancyComponent::ComputeAngularDragTorque(const FVector& AngularVelocity) const
{
	FVector DragTorque = FVector::ZeroVector;
	if (BuoyancyData.bApplyDragForcesInWater && IsInWaterBody())
	{
		DragTorque = -AngularVelocity * BuoyancyData.AngularDragCoefficient;
	}
	return DragTorque;
}

TUniquePtr<FBuoyancyComponentAsyncInput> UBuoyancyComponent::SetCurrentAsyncInputOutput(int32 InputIdx, FBuoyancyManagerAsyncOutput* CurOutput, FBuoyancyManagerAsyncOutput* NextOutput, float Alpha, int32 BuoyancyManagerTimestamp)
{
	if (IsUsingAsyncPath())
	{
		TUniquePtr<FBuoyancyComponentBaseAsyncInput> CurInput = MakeUnique<FBuoyancyComponentBaseAsyncInput>();
		SetCurrentAsyncInputOutputInternal(CurInput.Get(), InputIdx, CurOutput, NextOutput, Alpha, BuoyancyManagerTimestamp);
		return CurInput;
	}
	return nullptr;
}

void UBuoyancyComponent::SetCurrentAsyncInputOutputInternal(FBuoyancyComponentAsyncInput* CurInput, int32 InputIdx, FBuoyancyManagerAsyncOutput* CurOutput, FBuoyancyManagerAsyncOutput* NextOutput, float Alpha, int32 BuoyancyManagerTimestamp)
{
	ensure(CurAsyncInput == nullptr);	//should be reset after it was filled
	ensure(CurAsyncOutput == nullptr);	//should get reset after update is done

	CurAsyncInput = CurInput;
	CurAsyncInput->BuoyancyComponent = this;
	CurAsyncType = CurInput->Type;
	NextAsyncOutput = nullptr;
	OutputInterpAlpha = 0.f;

	// We need to find our component in the output given
	if (CurOutput)
	{
		for (int32 PendingOutputIdx = 0; PendingOutputIdx < OutputsWaitingOn.Num(); ++PendingOutputIdx)
		{
			// Found the correct pending output, use index to get the component.
			if (OutputsWaitingOn[PendingOutputIdx].Timestamp == CurOutput->Timestamp)
			{
				const int32 ComponentIdx = OutputsWaitingOn[PendingOutputIdx].Idx;
				FBuoyancyComponentAsyncOutput* ComponentOutput = CurOutput->Outputs[ComponentIdx].Get();
				if (ComponentOutput && ComponentOutput->bValid && ComponentOutput->Type == CurAsyncType)
				{
					CurAsyncOutput = ComponentOutput;

					if (NextOutput && NextOutput->Timestamp == CurOutput->Timestamp)
					{
						// This can occur when substepping - in this case, VehicleOutputs will be in the same order in NextOutput and CurOutput.
						FBuoyancyComponentAsyncOutput* ComponentNextOutput = NextOutput->Outputs[ComponentIdx].Get();
						if (ComponentNextOutput && ComponentNextOutput->bValid && ComponentNextOutput->Type == CurAsyncType)
						{
							NextAsyncOutput = ComponentNextOutput;
							OutputInterpAlpha = Alpha;
						}
					}
				}

				// these are sorted by timestamp, we are using latest, so remove entries that came before it.
				TArray<FAsyncOutputWrapper> NewOutputsWaitingOn;
				for (int32 CopyIndex = PendingOutputIdx; CopyIndex < OutputsWaitingOn.Num(); ++CopyIndex)
				{
					NewOutputsWaitingOn.Add(OutputsWaitingOn[CopyIndex]);
				}

				OutputsWaitingOn = MoveTemp(NewOutputsWaitingOn);
				break;
			}
		}
	}

	if (NextOutput && CurOutput)
	{
		if (NextOutput->Timestamp != CurOutput->Timestamp)
		{
			// NextOutput and CurOutput occurred in different steps, so we need to search for our specific component.
			for (int32 PendingOutputIdx = 0; PendingOutputIdx < OutputsWaitingOn.Num(); ++PendingOutputIdx)
			{
				// Found the correct pending output, use index to get the vehicle.
				if (OutputsWaitingOn[PendingOutputIdx].Timestamp == NextOutput->Timestamp)
				{
					FBuoyancyComponentAsyncOutput* ComponentOutput = NextOutput->Outputs[OutputsWaitingOn[PendingOutputIdx].Idx].Get();
					if (ComponentOutput && ComponentOutput->bValid && ComponentOutput->Type == CurAsyncType)
					{
						NextAsyncOutput = ComponentOutput;
						OutputInterpAlpha = Alpha;
					}
					break;
				}
			}
		}
	}

	FAsyncOutputWrapper& NewOutput = OutputsWaitingOn.AddDefaulted_GetRef();
	NewOutput.Timestamp = BuoyancyManagerTimestamp;
	NewOutput.Idx = InputIdx;
}

void UBuoyancyComponent::FinalizeSimCallbackData(FBuoyancyManagerAsyncInput& Input)
{
	for (UWaterBodyComponent* WaterBodyComponent : GetCurrentWaterBodyComponents())
	{
		if (WaterBodyComponent && !Input.WaterBodyComponentToSolverData.Contains(WaterBodyComponent))
		{
			TUniquePtr<FSolverSafeWaterBodyData> WaterBodyData = MakeUnique<FSolverSafeWaterBodyData>(WaterBodyComponent);
			Input.WaterBodyComponentToSolverData.Add(WaterBodyComponent, MoveTemp(WaterBodyData));
		}
	}

	CurAsyncInput = nullptr;
	CurAsyncOutput = nullptr;
}

void UBuoyancyComponent::GameThread_ProcessIntermediateAsyncOutput(const FBuoyancyComponentAsyncOutput& Output)
{
	if (Output.Type != EAsyncBuoyancyComponentDataType::AsyncBuoyancyInvalid)
	{
		const FBuoyancyComponentBaseAsyncOutput& BaseOutput = static_cast<const FBuoyancyComponentBaseAsyncOutput&>(Output);
		for (const TPair<FSphericalPontoon, EBuoyancyEvent>& Event : BaseOutput.SimOutput.Events)
		{
			if (Event.Value == EBuoyancyEvent::EnteredWaterBody)
			{
				OnPontoonEnteredWater(Event.Key);
			}
			else if (Event.Value == EBuoyancyEvent::ExitedWaterBody)
			{
				OnPontoonExitedWater(Event.Key);
			}
		}
	}
}

void UBuoyancyComponent::GameThread_ProcessIntermediateAsyncOutput(const FBuoyancyManagerAsyncOutput& AsyncOutput)
{
	if (!IsUsingAsyncPath())
	{
		return;
	}

	for (int32 PendingOutputIdx = 0; PendingOutputIdx < OutputsWaitingOn.Num(); ++PendingOutputIdx)
	{
		// Found the correct pending output, use index to get the vehicle.
		if (OutputsWaitingOn[PendingOutputIdx].Timestamp == AsyncOutput.Timestamp)
		{
			const FBuoyancyComponentAsyncOutput* Output = AsyncOutput.Outputs[OutputsWaitingOn[PendingOutputIdx].Idx].Get();
			if (Output && Output->bValid)
			{
				GameThread_ProcessIntermediateAsyncOutput(*Output);
			}
		}
	}
}

bool UBuoyancyComponent::IsUsingAsyncPath() const
{
	bool bAsyncSolver = false;
	if (UWorld* World = GetWorld())
	{
		if (const FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			if (const Chaos::FPBDRigidsSolver* Solver = PhysScene->GetSolver())
			{
				bAsyncSolver = Solver->IsUsingAsyncResults();
			}
		}
	}
	return bAsyncSolver && bUseAsyncPath && (CVarWaterBuoyancyUseAsyncPath.GetValueOnAnyThread() > 0);
}

TUniquePtr<FBuoyancyComponentAsyncAux> UBuoyancyComponent::CreateAsyncAux() const
{
	TUniquePtr<FBuoyancyComponentBaseAsyncAux> Aux = MakeUnique<FBuoyancyComponentBaseAsyncAux>();
	Aux->BuoyancyData = BuoyancyData;
	return Aux;
}

