// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuoyancyComponentSimulation.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

TUniquePtr<struct FBuoyancyComponentAsyncOutput> FBuoyancyComponentBaseAsyncInput::PreSimulate(UWorld* World, const float DeltaSeconds, const float TotalSeconds, FBuoyancyComponentAsyncAux* Aux, const TMap<UWaterBodyComponent*, TUniquePtr<FSolverSafeWaterBodyData>>& WaterBodyComponentData) const
{
	TUniquePtr<FBuoyancyComponentBaseAsyncOutput> Output = MakeUnique<FBuoyancyComponentBaseAsyncOutput>();
	if (Proxy && Aux)
	{
		// Copy sparsely changing properties into aux state
		FBuoyancyComponentBaseAsyncAux* BaseAux = static_cast<FBuoyancyComponentBaseAsyncAux*>(Aux);
		FBuoyancyAuxData& AuxData = BaseAux->AuxData;
		AuxData.Pontoons = Pontoons;
		AuxData.WaterBodyComponents = WaterBodyComponents;
		AuxData.SmoothedWorldTimeSeconds = SmoothedWorldTimeSeconds;

		// Perform the simulation

		FBuoyancyComponentSim::Update(DeltaSeconds, TotalSeconds, World, Proxy->GetPhysicsThreadAPI(), BaseAux->BuoyancyData, AuxData, WaterBodyComponentData, Output->SimOutput);
		Output->AuxData = AuxData;
		Output->bValid = true;
	}

	return MoveTemp(Output);
}
