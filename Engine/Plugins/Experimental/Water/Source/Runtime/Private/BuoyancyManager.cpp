// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuoyancyManager.h"
#include "BuoyancyComponent.h"
#include "PBDRigidsSolver.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "WaterSubsystem.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BuoyancyManager)

ABuoyancyManager::ABuoyancyManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Timestamp = 0;

#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
#endif
}

ABuoyancyManager* ABuoyancyManager::Get(const UObject* WorldContextObject)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::ReturnNull))
	{
		if (UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(World))
		{
			return WaterSubsystem->GetBuoyancyManager();
		}
	}

	return nullptr;
}

void ABuoyancyManager::OnCreatePhysics(UActorComponent* Component)
{
	if (AActor* OwningActor = Component->GetOwner())
	{
		if (UBuoyancyComponent* BuoyancyComp = OwningActor->FindComponentByClass<UBuoyancyComponent>())
		{
			if (BuoyancyComp->GetSimulatingComponent() == Cast<UPrimitiveComponent>(Component))
			{
				InitializeAsyncAux(BuoyancyComp);
			}
		}
	}
}

void ABuoyancyManager::OnDestroyPhysics(UActorComponent* Component)
{
	if (AActor* OwningActor = Component->GetOwner())
	{
		if (UBuoyancyComponent* BuoyancyComp = OwningActor->FindComponentByClass<UBuoyancyComponent>())
		{
			if (PhysicsInitializedSimulatingComponents.Contains(Cast<UPrimitiveComponent>(Component)))
			{
				ClearAsyncInputs(BuoyancyComp);
				if (UPrimitiveComponent* SimulatingComp = BuoyancyComp->GetSimulatingComponent())
				{
					if (AsyncCallback)
					{
						if (FBodyInstance* BI = SimulatingComp->GetBodyInstance())
						{
							if (auto ActorHandle = BI->GetPhysicsActor())
							{
								AsyncCallback->ClearAsyncAux_External(ActorHandle->GetGameThreadAPI().UniqueIdx());
							}
						}
					}

					PhysicsInitializedSimulatingComponents.Remove(SimulatingComp);
				}
			}
		}
	}
}

void ABuoyancyManager::ClearAsyncInputs(UBuoyancyComponent* Component)
{
	if (AsyncCallback)
	{
		//simulating component is destroyed so make sure callback does not still it
		FBuoyancyManagerAsyncInput* AsyncInput = AsyncCallback->GetProducerInputData_External();
		for (TUniquePtr<FBuoyancyComponentAsyncInput>& Input : AsyncInput->Inputs)
		{
			if (Input.Get())
			{
				if (Input->BuoyancyComponent == Component)
				{
					Input->Proxy = nullptr;
					break;
				}
			}
		}
	}
}

bool ABuoyancyManager::GetBuoyancyComponentManager(const UObject* WorldContextObject, ABuoyancyManager*& Manager)
{
	Manager = Get(WorldContextObject);

	return (Manager != nullptr);
}

void ABuoyancyManager::Register(UBuoyancyComponent* BuoyancyComponent)
{
	check(BuoyancyComponent);
	BuoyancyComponents.AddUnique(BuoyancyComponent);
	if(AsyncCallback)
	{
		InitializeAsyncAux(BuoyancyComponent);
	}
	else // AsyncCallback is not setup yet. We need to register this component at a later time.
	{
		BuoyancyComponentsToRegister.Emplace(BuoyancyComponent);
	}
}

void ABuoyancyManager::Unregister(UBuoyancyComponent* BuoyancyComponent)
{
	check(BuoyancyComponent);
	BuoyancyComponents.Remove(BuoyancyComponent);
}

void ABuoyancyManager::Update(FPhysScene* PhysScene, float DeltaTime)
{
	UWorld* World = GetWorld();
	if (World && AsyncCallback)
	{
		// Collect active buoyancy components
		{
			BuoyancyComponentsActive.Empty(BuoyancyComponents.Num());
			for (UBuoyancyComponent* BuoyancyComp : BuoyancyComponents)
			{
				if (BuoyancyComp && BuoyancyComp->IsActive())
				{
					BuoyancyComponentsActive.Add(BuoyancyComp);
				}
			}
		}

		FBuoyancyManagerAsyncInput* AsyncInput = AsyncCallback->GetProducerInputData_External();
		AsyncInput->Reset();	//only want latest frame's data
		{
			//We pass pointers from TArray so this reserve is critical. Otherwise realloc happens
			if (AsyncCallback->GetSolver()->IsUsingAsyncResults())
			{
				AsyncInput->Inputs.Reserve(BuoyancyComponentsActive.Num());
			}
			AsyncInput->Timestamp = Timestamp;
			AsyncInput->World = GetWorld();
		}

		//todo: handle this better using timestamp etc...

		// Grab all outputs for processing, even future ones for interpolation.
		Chaos::TSimCallbackOutputHandle<FBuoyancyManagerAsyncOutput> AsyncOutputLatest;
		while ((AsyncOutputLatest = AsyncCallback->PopFutureOutputData_External()))
		{
			PendingOutputs.Emplace(MoveTemp(AsyncOutputLatest));
		}

		const double ResultsTime = AsyncCallback->GetSolver()->GetPhysicsResultsTime_External() + DeltaTime;

		// Find index of first non-consumable output (first one after current time)
		int32 LastOutputIdx = 0;
		for (; LastOutputIdx < PendingOutputs.Num(); ++LastOutputIdx)
		{
			if (PendingOutputs[LastOutputIdx]->InternalTime > ResultsTime)
			{
				break;
			}
		}

		// Process events on outputs
		for (int32 OutputIdx = 0; OutputIdx < LastOutputIdx; ++OutputIdx)
		{
			for (UBuoyancyComponent* BuoyancyComp : BuoyancyComponentsActive)
			{
				BuoyancyComp->GameThread_ProcessIntermediateAsyncOutput(*PendingOutputs[OutputIdx]);
			}
		}

		// Cache the last consumed output for interpolation
		if (LastOutputIdx > 0)
		{
			LatestOutput = MoveTemp(PendingOutputs[LastOutputIdx - 1]);
		}

		// Remove all consumed outputs
		{
			TArray<Chaos::TSimCallbackOutputHandle<FBuoyancyManagerAsyncOutput>> NewPendingOutputs;
			for (int32 OutputIdx = LastOutputIdx; OutputIdx < PendingOutputs.Num(); ++OutputIdx)
			{
				NewPendingOutputs.Emplace(MoveTemp(PendingOutputs[OutputIdx]));
			}
			PendingOutputs = MoveTemp(NewPendingOutputs);
		}


		// Set async input/output
		for (UBuoyancyComponent* BuoyancyComponent : BuoyancyComponentsActive)
		{
			if (UPrimitiveComponent* PrimComp = BuoyancyComponent->GetSimulatingComponent())
			{
				if (FBodyInstance* BI = PrimComp->GetBodyInstance())
				{
					if (AsyncCallback->GetSolver()->IsUsingAsyncResults())
					{
						auto NextOutput = PendingOutputs.Num() > 0 ? PendingOutputs[0].Get() : nullptr;
						float Alpha = 0.f;
						if (NextOutput && LatestOutput)
						{
							const float Denom = NextOutput->InternalTime - LatestOutput->InternalTime;
							if (Denom > SMALL_NUMBER)
							{
								Alpha = (ResultsTime - LatestOutput->InternalTime) / Denom;
							}
						}
						AsyncInput->Inputs.Add(BuoyancyComponent->SetCurrentAsyncInputOutput(AsyncInput->Inputs.Num(), LatestOutput.Get(), NextOutput, Alpha, Timestamp));
					}
					BuoyancyComponent->Update(DeltaTime);
					BuoyancyComponent->FinalizeSimCallbackData(*AsyncInput);
				}
			}
		}
		++Timestamp;
	}
}

void ABuoyancyManager::InitializeAsyncAux(UBuoyancyComponent* Component)
{
	UPrimitiveComponent* SimulatingComponent = Component->GetSimulatingComponent();
	if (!SimulatingComponent)
	{
		return;
	}
	if (PhysicsInitializedSimulatingComponents.Contains(SimulatingComponent))
	{
		return;
	}

	if (AsyncCallback)
	{
		if (FBodyInstance* BI = SimulatingComponent->GetBodyInstance())
		{
			if (auto ActorHandle = BI->GetPhysicsActor())
			{
				AsyncCallback->CreateAsyncAux_External(ActorHandle->GetGameThreadAPI().UniqueIdx(), Component->CreateAsyncAux());
				PhysicsInitializedSimulatingComponents.Add(SimulatingComponent);
			}
		}
	}
}

void ABuoyancyManager::BeginPlay()
{
	OnCreateDelegateHandle = UActorComponent::GlobalCreatePhysicsDelegate.AddUObject(this, &ABuoyancyManager::OnCreatePhysics);
	OnDestroyDelegateHandle = UActorComponent::GlobalDestroyPhysicsDelegate.AddUObject(this, &ABuoyancyManager::OnDestroyPhysics);

	if (UWorld* World = GetWorld())
	{
		if (FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			OnPhysScenePreTickHandle = PhysScene->OnPhysScenePreTick.AddUObject(this, &ABuoyancyManager::Update);
			AsyncCallback = PhysScene->GetSolver()->CreateAndRegisterSimCallbackObject_External<FBuoyancyManagerAsyncCallback>();
		}

		for (const TWeakObjectPtr<UBuoyancyComponent>& BuoyancyComponentPtr : BuoyancyComponentsToRegister)
		{
			if (UBuoyancyComponent* BuoyancyComponent = BuoyancyComponentPtr.Get())
			{
				InitializeAsyncAux(BuoyancyComponent);
			}
		}
		BuoyancyComponentsToRegister.Empty();
	}

	Super::BeginPlay();
}

void ABuoyancyManager::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

	UActorComponent::GlobalCreatePhysicsDelegate.Remove(OnCreateDelegateHandle);
	UActorComponent::GlobalDestroyPhysicsDelegate.Remove(OnDestroyDelegateHandle);

	PendingOutputs.Empty();
	LatestOutput = Chaos::TSimCallbackOutputHandle<FBuoyancyManagerAsyncOutput>();

	if (UWorld* World = GetWorld())
	{
		if (FPhysScene* PhysScene = World->GetPhysicsScene())
		{
			PhysScene->OnPhysScenePreTick.Remove(OnPhysScenePreTickHandle);

			if (AsyncCallback)
			{
				PhysScene->GetSolver()->UnregisterAndFreeSimCallbackObject_External(AsyncCallback);
				PhysScene->OnPhysScenePreTick.Remove(OnPhysScenePreTickHandle);
				AsyncCallback = nullptr;
			}
		}
	}
}

FName FBuoyancyManagerAsyncCallback::GetFNameForStatId() const
{
	const static FLazyName StaticName("FBuoyancyManagerAsyncCallback");
	return StaticName;
}

void FBuoyancyManagerAsyncCallback::CreateAsyncAux_External(Chaos::FUniqueIdx HandleIndex, TUniquePtr<FBuoyancyComponentAsyncAux>&& AsyncAux)
{
	GetSolver()->EnqueueCommandImmediate([this, HandleIndex, AsyncAux = MoveTemp(AsyncAux)]() mutable
	{
		BuoyancyComponentToAux_Internal.Add(HandleIndex, MoveTemp(AsyncAux));
	});
}

void FBuoyancyManagerAsyncCallback::ClearAsyncAux_External(Chaos::FUniqueIdx HandleIndex)
{
	GetSolver()->EnqueueCommandImmediate([this, HandleIndex]()
	{
		BuoyancyComponentToAux_Internal.Remove(HandleIndex);
	});
}

void FBuoyancyManagerAsyncCallback::OnPreSimulate_Internal()
{
	using namespace Chaos;
	const FBuoyancyManagerAsyncInput* Input = GetConsumerInput_Internal();
	if (Input == nullptr)
	{
		return;
	}

	const int32 NumBuoyancyComponents = Input->Inputs.Num();

	UWorld* World = Input->World.Get();	//only safe to access for scene queries
	if (World == nullptr)
	{
		//world is gone so don't bother.
		return;
	}

	Chaos::FPhysicsSolver* PhysicsSolver = static_cast<Chaos::FPhysicsSolver*>(GetSolver());
	if (PhysicsSolver == nullptr)
	{
		return;
	}

	FBuoyancyManagerAsyncOutput& Output = GetProducerOutputData_Internal();
	Output.Outputs.AddDefaulted(NumBuoyancyComponents);
	Output.Timestamp = Input->Timestamp;

	//todo go wide
	for (int32 ObjectIdx = 0; ObjectIdx < Input->Inputs.Num(); ++ObjectIdx)
	{
		Output.Outputs[ObjectIdx] = nullptr;

		if (Input->Inputs[ObjectIdx] == nullptr)
		{
			continue;
		}
		const FBuoyancyComponentAsyncInput& BuoyancyComponentInput = *Input->Inputs[ObjectIdx];

		if (BuoyancyComponentInput.Proxy == nullptr)
		{
			continue;
		}

		Chaos::FRigidBodyHandle_Internal* Body_Internal = BuoyancyComponentInput.Proxy->GetPhysicsThreadAPI();

		if (Body_Internal == nullptr || Body_Internal->ObjectState() != Chaos::EObjectStateType::Dynamic)
		{
			continue;
		}

		bool bWake = false;

		TUniquePtr<FBuoyancyComponentAsyncAux>* AuxPtr = BuoyancyComponentToAux_Internal.Find(Body_Internal->UniqueIdx());
		FBuoyancyComponentAsyncAux* Aux = AuxPtr ? AuxPtr->Get() : nullptr;
		Output.Outputs[ObjectIdx] = BuoyancyComponentInput.PreSimulate(World, GetDeltaTime_Internal(), GetSimTime_Internal(), Aux, Input->WaterBodyComponentToSolverData);
	}
}

