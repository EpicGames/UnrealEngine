// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvQueryTest_InsideWaterBody.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "WaterBodyComponent.h"
#include "WaterBodyManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryTest_InsideWaterBody)

UEnvQueryTest_InsideWaterBody::UEnvQueryTest_InsideWaterBody(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
	Cost = EEnvTestCost::High;
	ValidItemType = UEnvQueryItemType_VectorBase::StaticClass();
	SetWorkOnFloatValues(false);
}

void UEnvQueryTest_InsideWaterBody::RunTest(FEnvQueryInstance& QueryInstance) const
{	
	BoolValue.BindData(QueryInstance.Owner.Get(), QueryInstance.QueryID);
	const bool bWantsInside = BoolValue.GetValue();

	for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It)
	{
		const FVector ItemLocation = GetItemLocation(QueryInstance, It.GetIndex());

		bool bInside = false;
		FWaterBodyManager::ForEachWaterBodyComponent(GetWorld(), [this, ItemLocation, &bInside](UWaterBodyComponent* WaterBodyComponent)
		{
			EWaterBodyQueryFlags QueryFlags = EWaterBodyQueryFlags::ComputeImmersionDepth;
			if (bIncludeWaves)
			{
				QueryFlags |= EWaterBodyQueryFlags::IncludeWaves;

				if (bSimpleWaves)
				{
					QueryFlags |= EWaterBodyQueryFlags::SimpleWaves;
				}
			}
			
			if (bIgnoreExclusionVolumes)
			{
				QueryFlags |= EWaterBodyQueryFlags::IgnoreExclusionVolumes;
			}

			const TValueOrError<FWaterBodyQueryResult, EWaterBodyQueryError> QueryResult = WaterBodyComponent->TryQueryWaterInfoClosestToWorldLocation(ItemLocation, QueryFlags);
			if (QueryResult.HasValue() && QueryResult.GetValue().IsInWater())
			{
				bInside = true;
				return false;
			}

			return true;
		});

		It.SetScore(TestPurpose, FilterType, bInside, bWantsInside);
	}
}

FText UEnvQueryTest_InsideWaterBody::GetDescriptionDetails() const
{
	return DescribeBoolTestParams("inside water body");
}

