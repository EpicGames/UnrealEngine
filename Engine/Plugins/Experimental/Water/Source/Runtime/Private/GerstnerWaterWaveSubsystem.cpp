// Copyright Epic Games, Inc. All Rights Reserved.

#include "GerstnerWaterWaveSubsystem.h"
#include "WaterViewExtension.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GerstnerWaterWaveSubsystem)

UGerstnerWaterWaveSubsystem::UGerstnerWaterWaveSubsystem()
{
}

void UGerstnerWaterWaveSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	FCoreDelegates::OnBeginFrame.AddUObject(this, &UGerstnerWaterWaveSubsystem::BeginFrameCallback);
}


void UGerstnerWaterWaveSubsystem::Deinitialize()
{
	WaterViewExtensions.Empty();

	FCoreDelegates::OnBeginFrame.RemoveAll(this);

	Super::Deinitialize();
}

void UGerstnerWaterWaveSubsystem::Register(FWaterViewExtension* InViewExtension)
{
	check(!WaterViewExtensions.Contains(InViewExtension));
	WaterViewExtensions.Add(InViewExtension);
}

void UGerstnerWaterWaveSubsystem::Unregister(FWaterViewExtension* InViewExtension)
{
	check(WaterViewExtensions.Contains(InViewExtension));
	WaterViewExtensions.Remove(InViewExtension);
}

void UGerstnerWaterWaveSubsystem::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(WaterViewExtensions.GetAllocatedSize());
}

void UGerstnerWaterWaveSubsystem::BeginFrameCallback()
{
	// In case there was a change, all registered view extensions need to update their GPU data : 
	if (bRebuildGPUData)
	{
		for (FWaterViewExtension* WaterViewExtension : WaterViewExtensions)
		{
			WaterViewExtension->MarkGPUDataDirty();
		}
	}
	bRebuildGPUData = false;
}

