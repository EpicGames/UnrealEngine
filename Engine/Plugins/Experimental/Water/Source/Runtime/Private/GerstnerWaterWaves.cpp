// Copyright Epic Games, Inc. All Rights Reserved.

#include "GerstnerWaterWaves.h"
#include "GerstnerWaveEvaluation.h"
#include "Engine/Engine.h"
#include "GerstnerWaterWaveSubsystem.h"
#include "Math/RandomStream.h"
#include "Misc/LargeWorldRenderPosition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GerstnerWaterWaves)

// ----------------------------------------------------------------------------------

void FGerstnerWave::Recompute()
{
	const float Gravity = 980.0f;
	const float Dispersion = 2.0f * PI / WaveLength;
	WaveVector = FVector2D(Direction.X, Direction.Y) * Dispersion;
	WaveSpeed = FMath::Sqrt(Dispersion * Gravity);
	WKA = Amplitude * Dispersion;
	Q = Amplitude * (Steepness / WKA);
}

// ----------------------------------------------------------------------------------

UGerstnerWaterWaves::UGerstnerWaterWaves()
{
	// Default generator
	GerstnerWaveGenerator = CreateDefaultSubobject<UGerstnerWaterWaveGeneratorSimple>(TEXT("WaterWaves"), /* bTransient = */false);

	if (!IsTemplate())
	{
		RecomputeWaves(/* bAllowBPScript = */false); // for the default one, we don't want / cannot call a BP event 
	}
}

void UGerstnerWaterWaves::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	if (UGerstnerWaterWaveSubsystem* GerstnerWaterWaveSubsystem = GEngine ? GEngine->GetEngineSubsystem<UGerstnerWaterWaveSubsystem>() : nullptr)
	{
		GerstnerWaterWaveSubsystem->RebuildGPUData();
	}
}

#if WITH_EDITOR
void UGerstnerWaterWaves::PostEditUndo()
{
	Super::PostEditUndo();

	RecomputeWaves(/* bAllowBPScript = */true);
}

void UGerstnerWaterWaves::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	RecomputeWaves(/* bAllowBPScript = */true);
}
#endif // WITH_EDITOR

float UGerstnerWaterWaves::GetWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime, FVector& OutNormal) const
{
	// Use the offset of the normalized tile as world position to match the shader behavior (see GerstnerWaveFunctions.ush).
	FVector WorldPosition(FLargeWorldRenderPosition(InPosition).GetOffset());
	return GerstnerWaveEvaluation::GetWaveHeightAtPosition(GetGerstnerWaves(), WorldPosition, InTime, OutNormal, /*bBlendLWCTiles=*/true);
}

float UGerstnerWaterWaves::GetSimpleWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime) const
{
	// Use the offset of the normalized tile as world position to match the shader behavior (see GerstnerWaveFunctions.ush).
	FVector WorldPosition(FLargeWorldRenderPosition(InPosition).GetOffset());
	return GerstnerWaveEvaluation::GetSimpleWaveHeightAtPosition(GetGerstnerWaves(), WorldPosition, InTime, /*bBlendLWCTiles=*/true);
}

float UGerstnerWaterWaves::GetWaveAttenuationFactor(const FVector& InPosition, float InWaterDepth, float InTargetWaveMaskDepth) const
{
	return GerstnerWaveEvaluation::GetWaveAttenuationFactor(InWaterDepth, InTargetWaveMaskDepth);
}

void UGerstnerWaterWaves::RecomputeWaves(bool bAllowBPScript)
{
	GerstnerWaves.Empty();
	MaxWaveHeight = 0.0f;

	// Generate new waves if there is a generator. Make sure that the wave list has been cleared before generating new ones.
	if (GerstnerWaveGenerator)
	{
		if (bAllowBPScript)
		{
			GerstnerWaveGenerator->GenerateGerstnerWaves(GerstnerWaves);
		}
		else
		{
			GerstnerWaveGenerator->GenerateGerstnerWaves_Implementation(GerstnerWaves);
		}

		// Automatically recompute the waves internals after waves have been regenerated :
		for (FGerstnerWave& Params : GerstnerWaves)
		{
			Params.Recompute();
			MaxWaveHeight += Params.Amplitude;
		}
	}

	if (UGerstnerWaterWaveSubsystem* GerstnerWaterWaveSubsystem = GEngine ? GEngine->GetEngineSubsystem<UGerstnerWaterWaveSubsystem>() : nullptr)
	{
		GerstnerWaterWaveSubsystem->RebuildGPUData();
	}
}

// ----------------------------------------------------------------------------------

void UGerstnerWaterWaveGeneratorSimple::GenerateGerstnerWaves_Implementation(TArray<FGerstnerWave>& OutWaves) const
{
	ensure(OutWaves.Num() == 0);

	FRandomStream LocalSeed(Seed);
	//int32 Quality = GetQualityWaveCount(); // Replaced by NumWaves

	for (int i = 0; i < NumWaves; ++i)
	{
		float Alpha = FMath::Clamp(1.f - ((float)i / (float)NumWaves) + LocalSeed.FRandRange(Randomness * (-1.0f / (float)NumWaves), Randomness * (1.0f / (float)NumWaves)), 0.0f, 1.0f);

		FGerstnerWave& Params = OutWaves.AddDefaulted_GetRef();

		Params.Direction = FVector(EForceInit::ForceInitToZero);
		FMath::SinCos(&Params.Direction.Y, &Params.Direction.X, FMath::DegreesToRadians((FVector::FReal)WindAngleDeg));
		if (i > 0)
		{
			Params.Direction = Params.Direction.RotateAngleAxis(LocalSeed.FRandRange(-DirectionAngularSpreadDeg, DirectionAngularSpreadDeg), FVector::UpVector);
		}

		Params.WaveLength = FMath::Lerp(MinWavelength, MaxWavelength, FMath::Pow(Alpha, WavelengthFalloff));
		Params.Amplitude = FMath::Max(FMath::Lerp(MinAmplitude, MaxAmplitude, FMath::Pow(Alpha, AmplitudeFalloff)), 0.0001f);
		Params.Steepness = FMath::Lerp(LargeWaveSteepness, SmallWaveSteepness, FMath::Pow((float)i / NumWaves, SteepnessFalloff));
	}
}

// ----------------------------------------------------------------------------------

void UGerstnerWaterWaveGeneratorSpectrum::GenerateGerstnerWaves_Implementation(TArray<FGerstnerWave>& OutWaves) const
{
	// [todo] kevin.ortegren: implement	
}


