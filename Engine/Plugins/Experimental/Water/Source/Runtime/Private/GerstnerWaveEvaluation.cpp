// Copyright Epic Games, Inc. All Rights Reserved.

#include "GerstnerWaveEvaluation.h"
#include "GerstnerWaterWaves.h"
#include "Misc/LargeWorldRenderPosition.h"

namespace
{
	/** Blend wave sin/cos near LWC tile boundaries to avoid seams. */
	void BlendWaveBetweenLWCTiles(const FGerstnerWave& InWaveParams, const FVector& InPosition, float InTime, float& WaveSin, float& WaveCos)
	{
		const FVector TileBorderDist = FVector(FLargeWorldRenderScalar::GetTileSize() * 0.5) - InPosition.GetAbs();
		const double BlendZoneWidth = 400.0; // Blend over a range of 4 meters on each side of the tile
		if (TileBorderDist.X < BlendZoneWidth || TileBorderDist.Y < BlendZoneWidth)
		{
			const FVector2D BlendWorldPos = FVector2D(TileBorderDist.X, TileBorderDist.Y);
			const double BlendAlpha = FMath::Clamp(BlendWorldPos.X / BlendZoneWidth, 0.0, 1.0) * FMath::Clamp(BlendWorldPos.Y / BlendZoneWidth, 0.0, 1.0);

			const float WaveTime = InWaveParams.WaveSpeed * InTime;
			const float BlendWavePos = FVector2D::DotProduct(BlendWorldPos, InWaveParams.WaveVector) - WaveTime;
			float BlendWaveSin = 0.0f;
			float BlendWaveCos = 0.0f;
			FMath::SinCos(&BlendWaveSin, &BlendWaveCos, BlendWavePos);
			WaveSin = FMath::Lerp(BlendWaveSin, WaveSin, BlendAlpha);
			WaveCos = FMath::Lerp(BlendWaveCos, WaveCos, BlendAlpha);
		}
	}
}

namespace GerstnerWaveEvaluation
{
	FVector GetWaveOffsetAtPosition(const FGerstnerWave& InWaveParams, const FVector& InPosition, float InTime,
		FVector& OutNormal, float& OutOffset1D, bool bBlendLWCTiles)
	{
		const float WaveTime = InWaveParams.WaveSpeed * InTime;
		const float WavePosition = FVector2D::DotProduct(FVector2D(InPosition.X, InPosition.Y), InWaveParams.WaveVector) - WaveTime;

		float WaveSin = 0, WaveCos = 0;
		FMath::SinCos(&WaveSin, &WaveCos, WavePosition);

		if (bBlendLWCTiles)
		{
			BlendWaveBetweenLWCTiles(InWaveParams, InPosition, InTime, WaveSin, WaveCos);
		}

		FVector Offset;
		OutOffset1D = -InWaveParams.Q * WaveSin;
		Offset.X = OutOffset1D * InWaveParams.Direction.X;
		Offset.Y = OutOffset1D * InWaveParams.Direction.Y;
		Offset.Z = WaveCos * InWaveParams.Amplitude;

		OutNormal = FVector(WaveSin * InWaveParams.WKA * InWaveParams.Direction.X, WaveSin * InWaveParams.WKA * InWaveParams.Direction.Y, /*WaveCos*InWaveParams.WKA*(InWaveParams.Steepness / InWaveParams.WKA)*/0.f); //match the material

		return Offset;
	}

	float GetSimpleWaveOffsetAtPosition(const FGerstnerWave& InWaveParams, const FVector& InPosition, float InTime, bool bBlendLWCTiles)
	{
		const float WaveTime = InWaveParams.WaveSpeed * InTime;
		const float WavePosition = FVector2D::DotProduct(FVector2D(InPosition.X, InPosition.Y), InWaveParams.WaveVector) - WaveTime;
		float WaveCos = FMath::Cos(WavePosition);

		if (bBlendLWCTiles)
		{
			float WaveSinDummy = 0.0f;
			BlendWaveBetweenLWCTiles(InWaveParams, InPosition, InTime, WaveSinDummy, WaveCos);
		}

		const float HeightOffset = WaveCos * InWaveParams.Amplitude;
		return HeightOffset;
	}

	float GetWaveHeightAtPosition(TArrayView<const FGerstnerWave> WaveParams, const FVector& InPosition, float InTime,
		FVector& OutNormal, bool bBlendLWCTiles)
	{
		float WaveHeight = 0.f;
		FVector SummedNormal(ForceInitToZero);

		for (const FGerstnerWave& Params : WaveParams)
		{
			float FirstOffset1D;
			FVector FirstNormal;
			FVector FirstOffset = GetWaveOffsetAtPosition(Params, InPosition, InTime, FirstNormal, FirstOffset1D, bBlendLWCTiles);

			// Only non-zero steepness requires a second sample.
			if (Params.Q != 0)
			{
				//Approximate wave height by taking two samples on each side of the current sample position and lerping.
				//Keep one query point fixed since sampling is going to move the points - if on the left half of wavelength, only add a right offset query point and vice-versa.
				//Choose q as the factor to offset by (max horizontal displacement).
				//Lerp between the two sampled heights and normals.
				const float TwoPi = 2 * PI;
				const float WaveTime = Params.WaveSpeed * InTime;
				float Position1D = FVector2D::DotProduct(FVector2D(InPosition.X, InPosition.Y), Params.WaveVector) - WaveTime;
				float MappedPosition1D = Position1D >= 0.f ? FMath::Fmod(Position1D, TwoPi) : TwoPi - FMath::Abs(FMath::Fmod(Position1D, TwoPi)); //get positive modulos from negative numbers too

				FVector SecondNormal;
				float SecondOffset1D;
				FVector GuessOffset;
				if (MappedPosition1D < PI)
				{
					GuessOffset = Params.Direction * Params.Q;
				}
				else
				{
					GuessOffset = -Params.Direction * Params.Q;
				}
				const FVector GuessPosition = InPosition + GuessOffset;
				FVector SecondOffset = GetWaveOffsetAtPosition(Params, GuessPosition, InTime, SecondNormal, SecondOffset1D, bBlendLWCTiles);
				SecondOffset1D += (MappedPosition1D < PI) ? Params.Q : -Params.Q;
				if (!(MappedPosition1D < PI))
				{
					Swap<FVector>(FirstOffset, SecondOffset);
					Swap<float>(FirstOffset1D, SecondOffset1D);
					Swap<FVector>(FirstNormal, SecondNormal);
				}
				const float LerpDenominator = (SecondOffset1D - FirstOffset1D);
				float LerpVal = (0 - FirstOffset1D) / (LerpDenominator > 0.f ? LerpDenominator : 1.f);
				const float FinalHeight = FMath::Lerp(FirstOffset.Z, SecondOffset.Z, LerpVal);
				const FVector WaveNormal = FMath::Lerp(FirstNormal, SecondNormal, LerpVal);

				SummedNormal += WaveNormal;
				WaveHeight += FinalHeight;
			}
			else
			{
				SummedNormal += FirstNormal;
				WaveHeight += FirstOffset.Z;
			}
		}
		SummedNormal.Z = 1.0f - SummedNormal.Z;
		OutNormal = SummedNormal.GetSafeNormal();

		return WaveHeight;
	}

	float GetSimpleWaveHeightAtPosition(TArrayView<const FGerstnerWave> WaveParams, const FVector& InPosition, float InTime, bool bBlendLWCTiles)
	{
		float WaveHeight = 0.f;

		for (const FGerstnerWave& Wave : WaveParams)
		{
			WaveHeight += GetSimpleWaveOffsetAtPosition(Wave, InPosition, InTime, bBlendLWCTiles);
		}

		return WaveHeight;
	}

	float GetWaveAttenuationFactor(float InWaterDepth, float InTargetWaveMaskDepth)
	{
		const float StrengthCoefficient = FMath::Exp(-FMath::Max(InWaterDepth, 0.0f) / (InTargetWaveMaskDepth / 2.0f));
		return FMath::Clamp(1.f - StrengthCoefficient, 0.f, 1.f);
	}
}
