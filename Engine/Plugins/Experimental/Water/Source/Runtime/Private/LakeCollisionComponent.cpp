// Copyright Epic Games, Inc. All Rights Reserved.

#include "LakeCollisionComponent.h"
#include "BodySetupEnums.h"
#include "SceneView.h"
#include "WaterBodyActor.h"
#include "Physics/PhysicsInterfaceTypes.h"
#include "WaterSplineComponent.h"
#include "GeomTools.h"
#include "PrimitiveSceneProxy.h"
#include "PhysicsEngine/BodySetup.h"
#include "AI/NavigationSystemHelpers.h"
#include "PrimitiveViewRelevance.h"
#include "SceneManagement.h"
#include "ShaderCore.h"
#include "WaterUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LakeCollisionComponent)


// ----------------------------------------------------------------------------------

extern TAutoConsoleVariable<float> CVarWaterSplineResampleMaxDistance;


// ----------------------------------------------------------------------------------

ULakeCollisionComponent::ULakeCollisionComponent(const FObjectInitializer& ObjectInitializer)
	: UPrimitiveComponent(ObjectInitializer)
{
	bHiddenInGame = true;
	bCastDynamicShadow = false;
	SetIgnoreStreamingManagerUpdate(true);
	bUseEditorCompositing = true;
}

void ULakeCollisionComponent::UpdateCollision(FVector InBoxExtent, bool bSplinePointsChanged)
{
	bool bNeedsUpdatedBody = bSplinePointsChanged || CachedBodySetup == nullptr;
	if (BoxExtent != InBoxExtent)
	{
		bNeedsUpdatedBody = true;
		BoxExtent = InBoxExtent;
		UpdateBounds();
	}

	if (bNeedsUpdatedBody)
	{
		UpdateBodySetup();
	}

	if (bPhysicsStateCreated)
	{
		// Update physics engine collision shapes
		BodyInstance.UpdateBodyScale(GetComponentTransform().GetScale3D(), true);
	}
}

#if UE_ENABLE_DEBUG_DRAWING

FPrimitiveSceneProxy* ULakeCollisionComponent::CreateSceneProxy()
{
	/** Represents a ULakeCollisionComponent to the scene manager. */
	class FLakeCollisionSceneProxy final : public FPrimitiveSceneProxy
	{
	public:
		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		FLakeCollisionSceneProxy(const ULakeCollisionComponent* InComponent)
			: FPrimitiveSceneProxy(InComponent)
		{
			bWillEverBeLit = false;

			if (InComponent->CachedBodySetup)
			{
				// copy the geometry for being able to access it on the render thread : 
				AggregateGeom = InComponent->CachedBodySetup->AggGeom;
			}
		}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			const FMatrix& LocalToWorld = GetLocalToWorld();
			const FTransform LocalToWorldTransform(LocalToWorld);

			const bool bDrawCollision = ViewFamily.EngineShowFlags.Collision && IsCollisionEnabled();

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];

					if (bDrawCollision && AllowDebugViewmodes())
					{
						FColor CollisionColor(157, 149, 223, 255);
						const bool bPerHullColor = false;
						const bool bDrawSolid = false;
						AggregateGeom.GetAggGeom(LocalToWorldTransform, GetSelectionColor(CollisionColor, IsSelected(), IsHovered()).ToFColor(true), nullptr, bPerHullColor, bDrawSolid, AlwaysHasVelocity(), ViewIndex, Collector);
					}
	
					RenderBounds(Collector.GetPDI(ViewIndex), View->Family->EngineShowFlags, GetBounds(), IsSelected());
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			// Should we draw this because collision drawing is enabled, and we have collision
			const bool bShowForCollision = View->Family->EngineShowFlags.Collision && IsCollisionEnabled();

			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = IsShown(View) || bShowForCollision;
			Result.bDynamicRelevance = true;
			Result.bShadowRelevance = false;
			Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
			return Result;
		}
		virtual uint32 GetMemoryFootprint(void) const override { return(sizeof(*this) + GetAllocatedSize()); }
		uint32 GetAllocatedSize(void) const { return FPrimitiveSceneProxy::GetAllocatedSize() + AggregateGeom.GetAllocatedSize(); }

	private:
		FKAggregateGeom AggregateGeom;
	};

	return new FLakeCollisionSceneProxy(this);
}
#endif // UE_ENABLE_DEBUG_DRAWING

FBoxSphereBounds ULakeCollisionComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds(FBox(-BoxExtent, BoxExtent)).TransformBy(LocalToWorld);
}

void ULakeCollisionComponent::CreateLakeBodySetupIfNeeded()
{
	if (!IsValid(CachedBodySetup))
	{
		CachedBodySetup = NewObject<UBodySetup>(this, TEXT("BodySetup")); // a name needs to be provided to ensure determinism
		CachedBodySetup->CollisionTraceFlag = CTF_UseSimpleAsComplex;
		// HACK [jonathan.bard] : to avoid non-determinitic cook issues which can occur as new collision components are created on construction, which generates a random GUID for UBodySetup, 
		//  we use a GUID based on the (deterministic) full name of the component, tweaked so as not to collide with standard GUIDs. BodySetupGuid should be removed altogether and UBodySetup's DDC
		//  key should be based only on its actual content but for now, this is one way around the determinism issue :
		CachedBodySetup->BodySetupGuid = FWaterUtils::StringToGuid(GetFullName(nullptr, EObjectFullNameFlags::IncludeClassPackage));
	}
}

bool TriangulateSimpleXYPlanarPolygon(const TArray<FVector>& VertexPositions, TArray<FIntVector>& OutTriangles);

void ExtrudeZSimplePolygon(const TArray<FVector>& InVertices, float BottomZ, float TopZ,
	TArray<FVector>& OutVertices, TArray<FIntVector>& OutTriangles)
{
	int32 InVerticesNum = InVertices.Num();
	if (InVerticesNum < 3) // degenerate case w/ zero volume; make empty mesh
	{
		OutVertices.Reset();
		OutTriangles.Reset();
		return;
	}

	// triangulate top/bottom shape
	bool bTrianglesAreClockwise = TriangulateSimpleXYPlanarPolygon(InVertices, OutTriangles);
	if (bTrianglesAreClockwise)
	{
		Swap(BottomZ, TopZ);
	}
	int32 TopTriNum = OutTriangles.Num();

	// set vertices
	OutVertices.SetNum(2 * InVerticesNum);
	for (int32 InIdx = 0; InIdx < InVerticesNum; ++InIdx)
	{
		OutVertices[InIdx] = InVertices[InIdx];
		OutVertices[InIdx].Z = BottomZ;
		OutVertices[InVerticesNum + InIdx] = InVertices[InIdx];
		OutVertices[InVerticesNum + InIdx].Z = TopZ;
	}

	// set triangles
	OutTriangles.SetNum(TopTriNum * 2 + InVerticesNum * 2); // top and bottom are placed first, then sides
	for (int32 TriIdx = 0; TriIdx < TopTriNum; TriIdx++)
	{
		FIntVector& TopTri = OutTriangles[TopTriNum + TriIdx];
		const FIntVector& BottomTri = OutTriangles[TriIdx];
		TopTri.X = BottomTri.X + InVerticesNum;
		// Y,Z intentionally swizzled to reverse triangle orientation for top vs bottom
		TopTri.Y = BottomTri.Z + InVerticesNum;
		TopTri.Z = BottomTri.Y + InVerticesNum;
	}

	// offsets for the clockwise and counter-clockwise vertices
	for (int32 NextIdx = 0, LastIdx = InVerticesNum - 1, SideTriIdx = TopTriNum * 2; NextIdx < InVerticesNum; LastIdx = NextIdx++)
	{
		OutTriangles[SideTriIdx++] = FIntVector(InVerticesNum + LastIdx, InVerticesNum + NextIdx, NextIdx);
		OutTriangles[SideTriIdx++] = FIntVector(InVerticesNum + LastIdx, NextIdx, LastIdx);
	}
}


/**
* Triangulate a polygon as projected to the XY plane, using ear clipping.  Orientation of triangles will match orientation of input curve
* Adapted from TriangulateSimplePolygon in GeometryProcessing's PolygonTriangulation.cpp, to avoid using any types/functions in GeometryProcessing
*
* @return bool indicating orientation of output triangles
*/
bool TriangulateSimpleXYPlanarPolygon(const TArray<FVector>& VertexPositions, TArray<FIntVector>& OutTriangles)
{
	// helper functions for analyzing XY-projected triangles
	struct Local
	{
		// returns 2*signed_area of the triangle formed by pts A, B, C
		static inline float XYArea2(const FVector& A, const FVector& B, const FVector& C)
		{
			return (A.X*B.Y - A.Y*B.X) + (B.X*C.Y - B.Y*C.X) + (C.X*A.Y - C.Y*A.X);
		}
		static inline bool XYIsTriangleFlipped(float OrientationSign, const FVector& A, const FVector& B, const FVector& C)
		{
			float XYSignedDoubleArea = XYArea2(A, B, C);
			return XYSignedDoubleArea * OrientationSign < 0;
		}
		static inline bool XYIsInsideTriangle(const FVector& A, const FVector& B, const FVector& C, const FVector& P)
		{
			float Sign1 = XYArea2(A, B, P);
			float Sign2 = XYArea2(B, C, P);
			float Sign3 = XYArea2(C, A, P);
			return (Sign1*Sign2 > 0) && (Sign2*Sign3 > 0) && (Sign3*Sign1 > 0); // true if all same (and non-zero) sign
		}
	};

	// Polygon must have at least three vertices/edges
	int32 PolygonVertexCount = VertexPositions.Num();
	check(PolygonVertexCount >= 3);


	// compute signed area of polygon
	double PolySignedArea = 0;
	for (int32 Idx = 0, LastIdx = PolygonVertexCount - 1; Idx < PolygonVertexCount; LastIdx = Idx++)
	{
		const FVector& v1 = VertexPositions[LastIdx];
		const FVector& v2 = VertexPositions[Idx];
		PolySignedArea += v1.X*v2.Y - v1.Y*v2.X;
	}
	PolySignedArea *= 0.5;
	bool bIsClockwise = PolySignedArea < 0;
	double OrientationSign = (bIsClockwise) ? -1.0 : 1.0;

	OutTriangles.Reset();

	// If perimeter has 3 vertices, just copy content of perimeter out 
	if (PolygonVertexCount == 3)
	{
		OutTriangles.Add(FIntVector(0, 1, 2));
		return bIsClockwise;
	}

	// Make a simple linked list array of the previous and next vertex numbers, for each vertex number
	// in the polygon.  This will just save us having to iterate later on.
	static TArray<int32> PrevVertexNumbers, NextVertexNumbers;

	PrevVertexNumbers.SetNumUninitialized(PolygonVertexCount, EAllowShrinking::No);
	NextVertexNumbers.SetNumUninitialized(PolygonVertexCount, EAllowShrinking::No);

	for (int32 VertexNumber = 0; VertexNumber < PolygonVertexCount; ++VertexNumber)
	{
		PrevVertexNumbers[VertexNumber] = VertexNumber - 1;
		NextVertexNumbers[VertexNumber] = VertexNumber + 1;
	}
	PrevVertexNumbers[0] = PolygonVertexCount - 1;
	NextVertexNumbers[PolygonVertexCount - 1] = 0;


	int32 EarVertexNumber = 0;
	int32 EarTestCount = 0;
	for (int32 RemainingVertexCount = PolygonVertexCount; RemainingVertexCount >= 3; )
	{
		bool bIsEar = true;

		// If we're down to only a triangle, just treat it as an ear.  Also, if we've tried every possible candidate
		// vertex looking for an ear, go ahead and just treat the current vertex as an ear.  This can happen when 
		// vertices are collinear or other degenerate cases.
		if (RemainingVertexCount > 3 && EarTestCount < RemainingVertexCount)
		{
			const FVector& PrevVertexPosition = VertexPositions[PrevVertexNumbers[EarVertexNumber]];
			const FVector& EarVertexPosition = VertexPositions[EarVertexNumber];
			const FVector& NextVertexPosition = VertexPositions[NextVertexNumbers[EarVertexNumber]];

			// Figure out whether the potential ear triangle is facing the same direction as the polygon
			// itself.  If it's facing the opposite direction, then we're dealing with a concave triangle
			// and we'll skip it for now.
			if (!Local::XYIsTriangleFlipped(
				OrientationSign, PrevVertexPosition, EarVertexPosition, NextVertexPosition))
			{
				int32 TestVertexNumber = NextVertexNumbers[NextVertexNumbers[EarVertexNumber]];

				do
				{
					// Test every other remaining vertex to make sure that it doesn't lie inside our potential ear
					// triangle.  If we find a vertex that's inside the triangle, then it cannot actually be an ear.
					const FVector& TestVertexPosition = VertexPositions[TestVertexNumber];
					if (Local::XYIsInsideTriangle(PrevVertexPosition, EarVertexPosition, NextVertexPosition, TestVertexPosition))
					{
						bIsEar = false;
						break;
					}

					TestVertexNumber = NextVertexNumbers[TestVertexNumber];
				} while (TestVertexNumber != PrevVertexNumbers[EarVertexNumber]);
			}
			else
			{
				bIsEar = false;
			}
		}

		if (bIsEar)
		{
			// OK, we found an ear!  Let's save this triangle in our output buffer.
			{
				FIntVector& Triangle = OutTriangles.Emplace_GetRef();
				Triangle.X = PrevVertexNumbers[EarVertexNumber];
				Triangle.Y = EarVertexNumber;
				Triangle.Z = NextVertexNumbers[EarVertexNumber];
			}

			// Update our linked list.  We're effectively cutting off the ear by pointing the ear vertex's neighbors to
			// point at their next sequential neighbor, and reducing the remaining vertex count by one.
			{
				NextVertexNumbers[PrevVertexNumbers[EarVertexNumber]] = NextVertexNumbers[EarVertexNumber];
				PrevVertexNumbers[NextVertexNumbers[EarVertexNumber]] = PrevVertexNumbers[EarVertexNumber];
				--RemainingVertexCount;
			}

			// Move on to the previous vertex in the list, now that this vertex was cut
			EarVertexNumber = PrevVertexNumbers[EarVertexNumber];

			EarTestCount = 0;
		}
		else
		{
			// The vertex is not the ear vertex, because it formed a triangle that either had a normal which pointed in the opposite direction
			// of the polygon, or at least one of the other polygon vertices was found to be inside the triangle.  Move on to the next vertex.
			EarVertexNumber = NextVertexNumbers[EarVertexNumber];

			// Keep track of how many ear vertices we've tested, so that if we exhaust all remaining vertices, we can
			// fall back to clipping the triangle and adding it to our mesh anyway.  This is important for degenerate cases.
			++EarTestCount;
		}
	}

	ensure(OutTriangles.Num() > 0);
	return bIsClockwise;
}

void ULakeCollisionComponent::UpdateBodySetup()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULakeCollisionComponent::UpdateBodySetup);

	CreateLakeBodySetupIfNeeded();
	
	FGuid PreviousBodySetupGuid = CachedBodySetup->BodySetupGuid;
	CachedBodySetup->RemoveSimpleCollision();
	// Removing the collision will needlessly generate a new Guid : restore the old one if valid to avoid invalidating the DDC :
	if (PreviousBodySetupGuid.IsValid())
	{
		CachedBodySetup->BodySetupGuid = PreviousBodySetupGuid;
	}

	FMemMark Mark(FMemStack::Get());

	TArray<FVector2D> SplineVerts;

	AWaterBody* OwningBody = GetTypedOuter<AWaterBody>();
	if (OwningBody && OwningBody->GetWaterSpline())
	{
		const UWaterSplineComponent* SplineComp = OwningBody->GetWaterSpline();

		const float MaxZ = -BoxExtent.Z;
		const float MinZ = BoxExtent.Z;

		// Generate planes
		const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();
		// lakes are closed loops so add 1 to the end 
		const int32 NumSteps = NumPoints + 1;

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ResampleSpline);

			TArray<FVector> PolyLineVertices;
			SplineComp->ConvertSplineToPolyLine(ESplineCoordinateSpace::World, FMath::Square(CVarWaterSplineResampleMaxDistance.GetValueOnGameThread()), PolyLineVertices);
			// Transform to local space of this component :
			Algo::Transform(PolyLineVertices, SplineVerts, [this](const FVector& Vertex) { return FVector2D(GetComponentToWorld().InverseTransformPosition(Vertex)); });
		}

		TArray<FVector2D> CorrectedSplineVertices;
		FGeomTools2D::CorrectPolygonWinding(CorrectedSplineVertices, SplineVerts, false);

		TArray<FVector2D> TriangulatedPolygonVertices;
		FGeomTools2D::TriangulatePoly(/*out*/TriangulatedPolygonVertices, SplineVerts, false);

		TArray<FVector2D> OutCleanTris;
		FGeomTools2D::RemoveRedundantTriangles(OutCleanTris, TriangulatedPolygonVertices);

		TArray<TArray<FVector2D>> ConvexHulls;
		FGeomTools2D::GenerateConvexPolygonsFromTriangles(ConvexHulls, OutCleanTris);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GenerateHull);
			for (const auto& Hull : ConvexHulls)
			{
				TArray<FVector> Hull3DVerts;
				Hull3DVerts.Reserve(Hull.Num());

				for (int32 PointIdx = 0; PointIdx < Hull.Num(); ++PointIdx)
				{
					Hull3DVerts.Emplace(FVector(Hull[PointIdx], 0));
				}

				if(Hull3DVerts.Num() > 2)
				{
					TArray<FVector> ExtrudedVerts;
					TArray<FIntVector> Indices;
					ExtrudeZSimplePolygon(Hull3DVerts, MinZ, MaxZ, ExtrudedVerts, Indices);

					FKConvexElem Convex;
					Convex.VertexData = MoveTemp(ExtrudedVerts);
					Convex.UpdateElemBox();

					CachedBodySetup->AggGeom.ConvexElems.Add(Convex);
				}
			}
		}

		CachedBodySetup->CreatePhysicsMeshes();

		RecreatePhysicsState();

		MarkRenderStateDirty();
	}
}

UBodySetup* ULakeCollisionComponent::GetBodySetup()
{
	return CachedBodySetup;
}

// Apply offset (substract MaxWaveHeight) to the Lake collision so nav mesh geometry is exported at ground level
bool ULakeCollisionComponent::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	const AWaterBody* OwningBody = GetTypedOuter<AWaterBody>();
	if (CachedBodySetup && OwningBody && OwningBody->GetWaterBodyComponent())
	{		
		FTransform GeomTransform(GetComponentTransform());
		GeomTransform.AddToTranslation(OwningBody->GetWaterBodyComponent()->GetWaterNavCollisionOffset());
		GeomExport.ExportRigidBodySetup(*CachedBodySetup, GeomTransform);
		return false;
	}

	return true;
}

