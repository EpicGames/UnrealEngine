// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceWater.h"

#include "NiagaraSystemInstance.h"
#include "WaterBodyActor.h"
#include "WaterBodyComponent.h"
#include "WaterModule.h"

#include "EngineUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceWater)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceWater"

// this cvar controls whether we'll allow depth queries into water bodies from the worker task.  This has been shown to be unsafe
// because the water body and the underlying assets (landscape proxies) could be in flux on the gamethread as assets stream in.
static int GNiagaraWaterDepthQuerySupported = 0;
static FAutoConsoleVariableRef CVarNiagaraSoloTickEarly(
	TEXT("fx.Niagara.Water.DepthQuerySupported"),
	GNiagaraWaterDepthQuerySupported,
	TEXT("When enabled water body queries will include unsafe access to the depth."),
	ECVF_Default
);

struct FNDIWater_InstanceData
{
	bool								bFindClosestBody = false;
	bool								bEvaluateSystemDepth = true;
	bool								bEvaluateSystemDepthPerFrame = false;
	TWeakObjectPtr<UWaterBodyComponent>	WaterBodyComponent;
	uint32								WaterBodyChangeId = 0;
	float								SystemInstanceWaterDepth = 0.0f;
	FNiagaraLWCConverter				LWCConverter;
};

namespace NDIWaterPrivate
{
	const FName IsValidName(TEXT("IsValid"));
	const FName GetWaterDataAtPointName(TEXT("GetWaterDataAtPoint"));
	const FName GetWaterSurfaceInfoName(TEXT("GetWaterSurfaceInfo"));

	const FName GetWaveParamLookupTableName(TEXT("GetWaveParamLookupTableOffset"));

	UWaterBodyComponent* FindClosestWaterBody(UWorld* World, const FVector QueryLocation)
	{
		if (World == nullptr)
		{
			return nullptr;
		}

		const EWaterBodyQueryFlags QueryFlags = (EWaterBodyQueryFlags::ComputeLocation | EWaterBodyQueryFlags::IncludeWaves);
		UWaterBodyComponent* ClosestComponent = nullptr;
		double ClosestWaterSq = 0.0f;

		for (TActorIterator<AWaterBody> It(World); It; ++It)
		{
			AWaterBody* WaterBody = *It;
			UWaterBodyComponent* WaterComponent = WaterBody ? WaterBody->GetWaterBodyComponent() : nullptr;
			if (!WaterComponent)
			{
				continue;
			}

			const TValueOrError<FWaterBodyQueryResult, EWaterBodyQueryError> CurrentQueryResult = WaterComponent->TryQueryWaterInfoClosestToWorldLocation(QueryLocation, QueryFlags);
			if (!CurrentQueryResult.HasValue() || CurrentQueryResult.GetValue().IsInExclusionVolume())
			{
				continue;
			}

			const FVector WaterLocation = CurrentQueryResult.GetValue().GetWaterPlaneLocation();
			const double DistanceSq = (WaterLocation - QueryLocation).SquaredLength();
			if (!ClosestComponent || DistanceSq < ClosestWaterSq)
			{
				ClosestWaterSq = DistanceSq;
				ClosestComponent = WaterComponent;
			}
		}
		return ClosestComponent;
	}

	void VMIsValid(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIWater_InstanceData> InstData(Context);
		FNDIOutputParam<bool> OutIsValid(Context);

		const bool bIsValid = InstData->WaterBodyComponent.IsValid();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutIsValid.SetAndAdvance(bIsValid);
		}
	}

	void VMGetWaterSurfaceInfo(FVectorVMExternalFunctionContext& Context)
	{
		QUICK_SCOPE_CYCLE_COUNTER(NiagaraDataInterfaceWater_GetWaterSurfaceInfo);

		// Inputs
		VectorVM::FUserPtrHandler<FNDIWater_InstanceData> InstData(Context);
		FNDIInputParam<bool>				InExecuteQuery(Context);
		FNDIInputParam<FNiagaraPosition>	InQueryPosition(Context);
		FNDIInputParam<bool>				InIncludeDepth(Context);
		FNDIInputParam<bool>				InIncludeWaves(Context);
		FNDIInputParam<bool>				InSimpleWaves(Context);

		// Outputs
		FNDIOutputParam<FNiagaraPosition>	OutWaterPlanePosition(Context);
		FNDIOutputParam<FVector3f>			OutWaterPlaneNormal(Context);
		FNDIOutputParam<FNiagaraPosition>	OutWaterSurfacePosition(Context);
		FNDIOutputParam<float>				OutWaterDepth(Context);
		FNDIOutputParam<FVector3f>			OutWaterVelocity(Context);
		FNDIOutputParam<bool>				OutInExclusionVolume(Context);

		UWaterBodyComponent* Component = InstData->WaterBodyComponent.Get();
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			const bool bExecuteQuery = InExecuteQuery.GetAndAdvance() && Component != nullptr;
			const FVector3f QueryPosition = InQueryPosition.GetAndAdvance();
			const bool bIncludeDepth = InIncludeDepth.GetAndAdvance();
			const bool bIncludeWaves = InIncludeWaves.GetAndAdvance();
			const bool bSimpleWaves = InSimpleWaves.GetAndAdvance();
			const bool bDoDepthQuery = bIncludeDepth && GNiagaraWaterDepthQuerySupported;

			if (bExecuteQuery)
			{
				const EWaterBodyQueryFlags QueryFlags =
					EWaterBodyQueryFlags::ComputeLocation |
					EWaterBodyQueryFlags::ComputeNormal |
					EWaterBodyQueryFlags::ComputeVelocity |
					(bDoDepthQuery ? EWaterBodyQueryFlags::ComputeDepth : EWaterBodyQueryFlags::None) |
					(bIncludeWaves ? EWaterBodyQueryFlags::IncludeWaves : EWaterBodyQueryFlags::None) |
					(bIncludeWaves && bSimpleWaves ? EWaterBodyQueryFlags::SimpleWaves : EWaterBodyQueryFlags::None);

				const TValueOrError<FWaterBodyQueryResult, EWaterBodyQueryError> QueryResult = Component->TryQueryWaterInfoClosestToWorldLocation(InstData->LWCConverter.ConvertSimulationPositionToWorld(QueryPosition), QueryFlags);
				if (QueryResult.HasError())
				{
					UE_LOGF(LogWater, Error, "WaterInfoQuery returned error :%ls", *UEnum::GetValueAsString(QueryResult.GetError()));
				}

				if (!QueryResult.HasValue())
				{
					continue;
				}
				const FWaterBodyQueryResult& Query = QueryResult.GetValue();

				const FVector3f WaterPlaneLocation = InstData->LWCConverter.ConvertWorldToSimulationPosition(Query .GetWaterPlaneLocation());
				const FVector3f WaterSurfacePosition = InstData->LWCConverter.ConvertWorldToSimulationPosition(Query .GetWaterSurfaceLocation());

				float DepthValue = bIncludeDepth ? InstData->SystemInstanceWaterDepth : 0.0f;
				if (bDoDepthQuery)
				{
					DepthValue = Query .GetWaterSurfaceDepth();
				}

				OutWaterPlanePosition.SetAndAdvance(WaterPlaneLocation);
				OutWaterPlaneNormal.SetAndAdvance(FVector3f(Query .GetWaterPlaneNormal()));
				OutWaterSurfacePosition.SetAndAdvance(WaterSurfacePosition);
				OutWaterDepth.SetAndAdvance(DepthValue);
				OutWaterVelocity.SetAndAdvance(FVector3f(Query .GetVelocity()));
				OutInExclusionVolume.SetAndAdvance(Query .IsInExclusionVolume());
			}
			else
			{
				OutWaterPlanePosition.SetAndAdvance(FVector3f::ZeroVector);
				OutWaterPlaneNormal.SetAndAdvance(FVector3f::UpVector);
				OutWaterSurfacePosition.SetAndAdvance(FVector3f::ZeroVector);
				OutWaterDepth.SetAndAdvance(0.0f);
				OutWaterVelocity.SetAndAdvance(FVector3f::ZeroVector);
				OutInExclusionVolume.SetAndAdvance(false);
			}
		}
	}
}

struct FNiagaraWaterDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		LargeWorldCoordinates = 1,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

void UNiagaraDataInterfaceWater::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceWater::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	{
		FNiagaraFunctionSignature & Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIWaterPrivate::IsValidName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Water"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsValid"));

		Sig.bMemberFunction = true;
		Sig.bExperimental = true;
		Sig.SetDescription(LOCTEXT("DataInterfaceWater_IsValidDesc", "Returns true if we are reading from a valid water component, false if not."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIWaterPrivate::GetWaterSurfaceInfoName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Water"));
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("ExecuteQuery")).SetValue(true);
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("QueryPosition"));
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IncludeDepth")).SetValue(true);
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IncludeWaves")).SetValue(true);
		Sig.Inputs.Emplace_GetRef(FNiagaraTypeDefinition::GetBoolDef(), TEXT("SimpleWaves")).SetValue(true);

		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("WaterPlanePosition"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("WaterPlaneNormal"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("WaterSurfacePosition"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("WaterDepth"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("WaterVelocity"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetBoolDef(), TEXT("InExclusionVolume"));

		Sig.bMemberFunction = true;
		Sig.bExperimental = true;
		Sig.SetDescription(LOCTEXT("DataInterfaceWater_GetWaterSurfaceInfo", "Get the water surface information at the provided world position."));
		Sig.SetInputDescription(Sig.Inputs[1], LOCTEXT("DataInterfaceWater_GetWaterSurfaceInfo_ExecuteQuery", "When disabled the water query will not run and the results are invalid / defaulted."));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIWaterPrivate::GetWaterDataAtPointName;

		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Water"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("WorldPosition"));
		Sig.Inputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time"));

		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("WaveHeight"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Depth"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetPositionDef(), TEXT("SurfacePosition"));
		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetVec3Def(), TEXT("SurfaceNormal"));

		Sig.bMemberFunction = true;
		Sig.bExperimental = true;
		Sig.bSoftDeprecatedFunction = true;
		Sig.SetDescription(LOCTEXT("DataInterfaceWater_GetWaterDataAtPoint", "Get the water data at the provided world position and time"));
	}

	{
		FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
		Sig.Name = NDIWaterPrivate::GetWaveParamLookupTableName;
		Sig.Inputs.Emplace(FNiagaraTypeDefinition(GetClass()), TEXT("Water"));

		Sig.Outputs.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Offset"));

		Sig.bMemberFunction = true;
		Sig.bExperimental = true;
		Sig.SetDescription(LOCTEXT("DataInterfaceWater_GetWaveParamLookupTableOffset", "Get the lookup table offset into the wave data texture for the data interface's water body"));
	}
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceWater, GetWaterDataAtPoint);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceWater, GetWaveParamLookupTableOffset);

void UNiagaraDataInterfaceWater::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	if (BindingInfo.Name == NDIWaterPrivate::IsValidName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(&NDIWaterPrivate::VMIsValid);
	}
	else if (BindingInfo.Name == NDIWaterPrivate::GetWaterSurfaceInfoName)
	{
		OutFunc = FVMExternalFunction::CreateStatic(&NDIWaterPrivate::VMGetWaterSurfaceInfo);
	}
	else if (BindingInfo.Name == NDIWaterPrivate::GetWaterDataAtPointName)
	{
		if(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 11)
		{
			NDI_FUNC_BINDER(UNiagaraDataInterfaceWater, GetWaterDataAtPoint)::Bind(this, OutFunc);
		}
	}
	else if (BindingInfo.Name == NDIWaterPrivate::GetWaveParamLookupTableName)
	{
		if (BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1)
		{
			NDI_FUNC_BINDER(UNiagaraDataInterfaceWater, GetWaveParamLookupTableOffset)::Bind(this, OutFunc);
		}
	}
}

bool UNiagaraDataInterfaceWater::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceWater* OtherTyped = CastChecked<const UNiagaraDataInterfaceWater>(Other);
	return
		OtherTyped->bFindWaterBodyOnSpawn == bFindWaterBodyOnSpawn &&
		OtherTyped->bEvaluateSystemDepth == bEvaluateSystemDepth &&
		OtherTyped->bEvaluateSystemDepthPerFrame == bEvaluateSystemDepthPerFrame &&
		OtherTyped->SourceBodyComponent == SourceBodyComponent;
}

bool UNiagaraDataInterfaceWater::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceWater* OtherTyped = CastChecked<UNiagaraDataInterfaceWater>(Destination);
	OtherTyped->bFindWaterBodyOnSpawn = bFindWaterBodyOnSpawn;
	OtherTyped->bEvaluateSystemDepth = bEvaluateSystemDepth;
	OtherTyped->bEvaluateSystemDepthPerFrame = bEvaluateSystemDepthPerFrame;
	OtherTyped->SourceBodyComponent = SourceBodyComponent;

	return true;
}

int32 UNiagaraDataInterfaceWater::PerInstanceDataSize() const
{
	return sizeof(FNDIWater_InstanceData);
}

bool UNiagaraDataInterfaceWater::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIWater_InstanceData* InstData = new (PerInstanceData) FNDIWater_InstanceData();
	InstData->bFindClosestBody		= bFindWaterBodyOnSpawn;
	InstData->bEvaluateSystemDepth	= bEvaluateSystemDepth;
	InstData->bEvaluateSystemDepthPerFrame = bEvaluateSystemDepthPerFrame;
	InstData->WaterBodyComponent	= nullptr;
	InstData->WaterBodyChangeId		= SourceBodyChangeId - 1;
	InstData->LWCConverter			= SystemInstance->GetLWCConverter();

	return true;
}

void UNiagaraDataInterfaceWater::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIWater_InstanceData* InstData = static_cast<FNDIWater_InstanceData*>(PerInstanceData);
	InstData->~FNDIWater_InstanceData();
}

bool UNiagaraDataInterfaceWater::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	check(SystemInstance);
	FNDIWater_InstanceData* InstData = static_cast<FNDIWater_InstanceData*>(PerInstanceData);

	bool bCalcDepth = bEvaluateSystemDepth && bEvaluateSystemDepthPerFrame;

	// Do we need to update the water body component?
	if (InstData->WaterBodyChangeId != SourceBodyChangeId)
	{
		// If the search for closest was enabled, perform the search
		// Note: we do this here rather than in Init as the system might be auto activate and the user parameter not set until after the spawn
		if (SourceBodyComponent.IsNull() && InstData->bFindClosestBody)
		{
			const FVector QueryLocation = SystemInstance->GetWorldTransform().GetTranslation();
			InstData->WaterBodyComponent = NDIWaterPrivate::FindClosestWaterBody(SystemInstance->GetWorld(), QueryLocation);
		}
		else
		{
			UObject* RawSourceBodyComponent = SourceBodyComponent.Get();
			if (AActor* SourceActor = Cast<AActor>(RawSourceBodyComponent))
			{
				InstData->WaterBodyComponent = SourceActor->GetComponentByClass<UWaterBodyComponent>();
			}
			else
			{
				InstData->WaterBodyComponent = Cast<UWaterBodyComponent>(RawSourceBodyComponent);
			}
		}
		InstData->bFindClosestBody = false;
		InstData->WaterBodyChangeId = SourceBodyChangeId;

		bCalcDepth = bEvaluateSystemDepth;
	}

	if (bCalcDepth)
	{
		if (UWaterBodyComponent* WaterBodyComponent = InstData->WaterBodyComponent.Get())
		{
			const FVector QueryLocation = SystemInstance->GetWorldTransform().GetTranslation();
			const EWaterBodyQueryFlags QueryFlags = EWaterBodyQueryFlags::ComputeDepth;

			const TValueOrError<FWaterBodyQueryResult, EWaterBodyQueryError> QueryResult = WaterBodyComponent->TryQueryWaterInfoClosestToWorldLocation(QueryLocation, QueryFlags);
			if (QueryResult.HasValue())
			{
				InstData->SystemInstanceWaterDepth = QueryResult.GetValue().IsInExclusionVolume() ? 0.0f : QueryResult.GetValue().GetWaterSurfaceDepth();
			}
			else if (QueryResult.HasError())
			{
				UE_LOGF(LogWater, Error, "NiagaraWaterDataInterface: attempting to compute the water body depth returned error: %ls", *UEnum::GetValueAsString(QueryResult.GetError()));
			}

		}
	}

	return false;
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceWater::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	bool bChanged = false;
	
	// upgrade from lwc changes, only parameter types changed there
	if (FunctionSignature.FunctionVersion < FNiagaraWaterDIFunctionVersion::LargeWorldCoordinates)
	{
		if (FunctionSignature.Name == NDIWaterPrivate::GetWaterDataAtPointName && ensure(FunctionSignature.Inputs.Num() == 3) && ensure(FunctionSignature.Outputs.Num() == 5))
		{
			FunctionSignature.Inputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			FunctionSignature.Outputs[3].SetType(FNiagaraTypeDefinition::GetPositionDef());
			bChanged = true;
		}
	}
	FunctionSignature.FunctionVersion = FNiagaraWaterDIFunctionVersion::LatestVersion;

	return bChanged;
}
#endif

#if WITH_NIAGARA_DEBUGGER
void UNiagaraDataInterfaceWater::DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const
{
	Super::DrawDebugHud(DebugHudContext);

	const FNDIWater_InstanceData* InstanceData_GT = DebugHudContext.GetSystemInstance()->FindTypedDataInterfaceInstanceData<FNDIWater_InstanceData>(this);
	if (InstanceData_GT == nullptr)
	{
		return;
	}

	UWaterBodyComponent* WaterBodyComponent = InstanceData_GT->WaterBodyComponent.Get();
	DebugHudContext.GetOutputString().Appendf(TEXT("WaterBodyComponent(%s)"), *GetNameSafe(WaterBodyComponent));
}
#endif

void UNiagaraDataInterfaceWater::GetWaterDataAtPoint(FVectorVMExternalFunctionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(NiagaraDataInterfaceWater_GetWaterDataAtPoint);

	VectorVM::FUserPtrHandler<FNDIWater_InstanceData> InstData(Context);
	
	// Inputs
	FNDIInputParam<FNiagaraPosition> WorldPos(Context);
	FNDIInputParam<float> Time(Context);

	// Outputs
	FNDIOutputParam<float> OutHeight(Context);
	FNDIOutputParam<float> OutDepth(Context);
	FNDIOutputParam<FVector3f> OutVelocity(Context);
	FNDIOutputParam<FNiagaraPosition> OutSurfacePos(Context);
	FNDIOutputParam<FVector3f> OutSurfaceNormal(Context);

	UWaterBodyComponent* Component = InstData->WaterBodyComponent.Get();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		FWaterBodyQueryResult QueryResult;

		bool bIsValid = false;
		if (Component != nullptr)
		{
			FVector QueryPos = InstData->LWCConverter.ConvertSimulationPositionToWorld(WorldPos.GetAndAdvance());
			EWaterBodyQueryFlags QueryFlags = EWaterBodyQueryFlags::ComputeLocation
				| EWaterBodyQueryFlags::ComputeVelocity
				| EWaterBodyQueryFlags::ComputeNormal
				| EWaterBodyQueryFlags::IncludeWaves;

			if (GNiagaraWaterDepthQuerySupported)
			{
				QueryFlags |= EWaterBodyQueryFlags::ComputeDepth;
			}

			TValueOrError<FWaterBodyQueryResult, EWaterBodyQueryError> Query = Component->TryQueryWaterInfoClosestToWorldLocation(QueryPos, QueryFlags);
			if (Query.HasError())
			{
				UE_LOGF(LogWater, Error, "WaterInfoQuery returned error :%ls", *UEnum::GetValueAsString(Query.GetError()));
			}

			if (!Query.HasValue())
			{
				continue;
			}

			QueryResult = Query.GetValue();

			bIsValid = !QueryResult.IsInExclusionVolume();
		}

		float DepthValue = InstData->SystemInstanceWaterDepth;
		if (GNiagaraWaterDepthQuerySupported)
		{
			DepthValue = QueryResult.GetWaterSurfaceDepth();
		}

		OutHeight.SetAndAdvance(bIsValid ? QueryResult.GetWaveInfo().Height : 0.0f);
		OutDepth.SetAndAdvance(bIsValid ? DepthValue : 0.0f);
		OutVelocity.SetAndAdvance(bIsValid ? FVector3f(QueryResult.GetVelocity()) : FVector3f::ZeroVector);		// LWC_TODO: Precision loss

		// Note we assume X and Y are in water by the time this is queried
		const FVector& AdjustedSurfaceLoc = bIsValid ? QueryResult.GetWaterSurfaceLocation() : FVector::ZeroVector;
		OutSurfacePos.SetAndAdvance(InstData->LWCConverter.ConvertWorldToSimulationPosition(AdjustedSurfaceLoc));

		OutSurfaceNormal.SetAndAdvance(bIsValid ? FVector3f(QueryResult.GetWaterSurfaceNormal()) : FVector3f::UpVector);

		Time.GetAndAdvance();
	}
}

void UNiagaraDataInterfaceWater::GetWaveParamLookupTableOffset(FVectorVMExternalFunctionContext& Context)
{
	// Inputs
	VectorVM::FUserPtrHandler<FNDIWater_InstanceData> InstData(Context);

	// Outputs
	VectorVM::FExternalFuncRegisterHandler<int> OutLookupTableOffset(Context);
	if (UWaterBodyComponent* Component = InstData->WaterBodyComponent.Get())
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			*OutLookupTableOffset.GetDestAndAdvance() = Component->GetWaterBodyIndex();
		}
	}
	else
	{
		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			*OutLookupTableOffset.GetDestAndAdvance() = 0;
		}
	}
}

void UNiagaraDataInterfaceWater::SetWaterBodyComponent(UWaterBodyComponent* InWaterBodyComponent)
{
	SourceBodyComponent = InWaterBodyComponent;
	++SourceBodyChangeId;
}

#undef LOCTEXT_NAMESPACE
