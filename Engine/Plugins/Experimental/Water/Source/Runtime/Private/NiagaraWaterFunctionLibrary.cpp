// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraWaterFunctionLibrary.h"
#include "NiagaraComponent.h"
#include "NiagaraDataInterfaceWater.h"
#include "WaterBodyComponent.h"
#include "WaterModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraWaterFunctionLibrary)

void UNiagaraWaterFunctionLibrary::SetWaterBodyComponent(UNiagaraComponent* NiagaraSystem, const FString& OverrideName, UWaterBodyComponent* WaterBodyComponent)
{
	if (!NiagaraSystem)
	{
		UE_LOGF(LogWater, Warning, "NiagaraSystem in \"Set Water Body Component\" is NULL, OverrideName \"%ls\" and WaterBodyComponent \"%ls\", skipping.", *OverrideName, WaterBodyComponent ? *WaterBodyComponent->GetOwner()->GetActorNameOrLabel() : TEXT("NULL"));
		return;
	}

	if (!WaterBodyComponent)
	{
		UE_LOGF(LogWater, Warning, "WaterBodyComponent in \"Set Water Body Component\" is NULL, OverrideName \"%ls\" and NiagaraSystem \"%ls\", skipping.", *OverrideName, *NiagaraSystem->GetOwner()->GetActorNameOrLabel());
		return;
	}


	const FNiagaraParameterStore& OverrideParameters = NiagaraSystem->GetOverrideParameters();

	FNiagaraVariable Variable(FNiagaraTypeDefinition(UNiagaraDataInterfaceWater::StaticClass()), *OverrideName);

	int32 Index = OverrideParameters.IndexOf(Variable);
	if (Index == INDEX_NONE)
	{
		UE_LOGF(LogWater, Warning, "Could not find index of variable \"%ls\" in the OverrideParameters map of NiagaraSystem \"%ls\".", *OverrideName, *NiagaraSystem->GetOwner()->GetActorNameOrLabel());
		return;
	}

	UNiagaraDataInterfaceWater* WaterInterface = Cast<UNiagaraDataInterfaceWater>(OverrideParameters.GetDataInterface(Index));
	if (!WaterInterface)
	{
		UE_LOGF(LogWater, Warning, "Did not find a matching Water Data Interface variable named \"%ls\" in the User variables of NiagaraSystem \"%ls\" .", *OverrideName, *NiagaraSystem->GetOwner()->GetActorNameOrLabel());
		return;
	}

	WaterInterface->SetWaterBodyComponent(WaterBodyComponent);
}
