// Copyright Epic Games, Inc. All Rights Reserved.

#include "OceanCollisionComponent.h"
#include "BodySetupEnums.h"
#include "SceneView.h"
#include "WaterBodyActor.h"
#include "Chaos/CollisionConvexMesh.h"
#include "PrimitiveSceneProxy.h"
#include "Physics/PhysicsInterfaceTypes.h"
#include "PhysicsEngine/BodySetup.h"
#include "PrimitiveViewRelevance.h"
#include "SceneManagement.h"
#include "ShaderCore.h"
#include "AI/NavigationSystemHelpers.h"
#include "WaterUtils.h"

// for working around Chaos issue
#include "Chaos/Convex.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OceanCollisionComponent)

UOceanCollisionComponent::UOceanCollisionComponent(const FObjectInitializer& ObjectInitializer)
	: UPrimitiveComponent(ObjectInitializer)
{
	bHiddenInGame = true;
	bCastDynamicShadow = false;
	SetIgnoreStreamingManagerUpdate(true);
	bUseEditorCompositing = true;
}


void UOceanCollisionComponent::InitializeFromConvexElements(const TArray<FKConvexElem>& ConvexElements)
{
	bool bNeedsUpdatedBody = true;

	UpdateBodySetup(ConvexElements);

	if (bPhysicsStateCreated)
	{
		// Update physics engine collision shapes
		BodyInstance.UpdateBodyScale(GetComponentTransform().GetScale3D(), true);
	}
}




FBoxSphereBounds UOceanCollisionComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds(BoundingBox).TransformBy(LocalToWorld);
}


void UOceanCollisionComponent::CreateOceanBodySetupIfNeeded()
{
	if (!IsValid(CachedBodySetup))
	{
		CachedBodySetup = NewObject<UBodySetup>(this, TEXT("BodySetup")); // a name needs to be provided to ensure determinism
		CachedBodySetup->CollisionTraceFlag = CTF_UseSimpleAsComplex;
		// HACK [jonathan.bard] : to avoid non-determinitic cook issues which can occur as new collision components are created on construction, which generates a random GUID for UBodySetup, 
		//  we use a GUID based on the (deterministic) full name of the component, tweaked so as not to collide with standard GUIDs. BodySetupGuid should be removed altogether and UBodySetup's DDC
		//  key should be based only on its actual content but for now, this is one way around the determinism issue :
		CachedBodySetup->BodySetupGuid = FWaterUtils::StringToGuid(GetFullName(nullptr, EObjectFullNameFlags::IncludeClassPackage));
	}
}


void UOceanCollisionComponent::UpdateBodySetup(const TArray<FKConvexElem>& ConvexElements)
{
	CreateOceanBodySetupIfNeeded();

	FGuid PreviousBodySetupGuid = CachedBodySetup->BodySetupGuid;
	CachedBodySetup->RemoveSimpleCollision();

	FMemMark Mark(FMemStack::Get());

	CachedBodySetup->AggGeom.ConvexElems = ConvexElements;
	CachedBodySetup->bHasCookedCollisionData = true;		// not clear what this flag is for, and true is the default in UBodySetup constructor?

	// This currently doesn't appear to work at Runtime!!!
	// Chaos paths in CreatePhysicsMeshes() will only try to get the data from the DDC.
	CachedBodySetup->InvalidatePhysicsData();

	// Removing the collision will needlessly generate a new Guid : restore the old one if valid to avoid invalidating the DDC / making the cook non-deterministic :
	if (PreviousBodySetupGuid.IsValid())
	{
		CachedBodySetup->BodySetupGuid = PreviousBodySetupGuid;
	}

	CachedBodySetup->CreatePhysicsMeshes();

	//
	// GROSS HACK to work around the issue above that CreatePhysicsMeshes() doesn't currently work at Runtime.
	// The "cook" for a FKConvexElem just creates and caches a Chaos::FConvex instance, and the restore from cooked
	// data just restores that and pases it to the FKConvexElem. So we're just going to do that ourselves.
	// Code below is taken from FChaosDerivedDataCooker::BuildConvexMeshes() 
	//
	for (FKConvexElem& Elem : CachedBodySetup->AggGeom.ConvexElems)
	{
		int32 NumHullVerts = Elem.VertexData.Num();
		TArray<Chaos::FConvex::FVec3Type> ConvexVertices;
		ConvexVertices.SetNum(NumHullVerts);
		for (int32 VertIndex = 0; VertIndex < NumHullVerts; ++VertIndex)
		{
			ConvexVertices[VertIndex] = Elem.VertexData[VertIndex];
		}

		Chaos::FConvexPtr ChaosConvex( new Chaos::FConvex(ConvexVertices, 0.0f));
		Elem.SetConvexMeshObject(MoveTemp(ChaosConvex));
	}

	RecreatePhysicsState();

	MarkRenderStateDirty();

	BoundingBox = CachedBodySetup->AggGeom.CalcAABB(FTransform::Identity);
	UpdateBounds();
}

UBodySetup* UOceanCollisionComponent::GetBodySetup()
{
	return CachedBodySetup;
}

#if UE_ENABLE_DEBUG_DRAWING

FPrimitiveSceneProxy* UOceanCollisionComponent::CreateSceneProxy()
{
	/** Represents a UBoxComponent to the scene manager. */
	class FOceanCollisionSceneProxy final : public FPrimitiveSceneProxy
	{
	public:
		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		FOceanCollisionSceneProxy(const UOceanCollisionComponent* InComponent)
			: FPrimitiveSceneProxy(InComponent)
		{
			bWillEverBeLit = false;

			if (InComponent->CachedBodySetup)
			{
				// copy the geometry for being able to access it on the render thread : 
				AggregateGeom = InComponent->CachedBodySetup->AggGeom;
			}
		}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			const FMatrix& LocalToWorld = GetLocalToWorld();
			const FTransform LocalToWorldTransform(LocalToWorld);

			const bool bDrawCollision = ViewFamily.EngineShowFlags.Collision && IsCollisionEnabled();

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];

					if (bDrawCollision && AllowDebugViewmodes())
					{
						FColor CollisionColor(157, 149, 223, 255);
						const bool bPerHullColor = false;
						const bool bDrawSolid = false;
						AggregateGeom.GetAggGeom(LocalToWorldTransform, GetSelectionColor(CollisionColor, IsSelected(), IsHovered()).ToFColor(true), nullptr, bPerHullColor, bDrawSolid, AlwaysHasVelocity(), ViewIndex, Collector);
					}

					RenderBounds(Collector.GetPDI(ViewIndex), View->Family->EngineShowFlags, GetBounds(), IsSelected());
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			// Should we draw this because collision drawing is enabled, and we have collision
			const bool bShowForCollision = View->Family->EngineShowFlags.Collision && IsCollisionEnabled();

			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = IsShown(View) || bShowForCollision;
			Result.bDynamicRelevance = true;
			Result.bShadowRelevance = false;
			Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
			return Result;
		}
		virtual uint32 GetMemoryFootprint(void) const override { return(sizeof(*this) + GetAllocatedSize()); }
		uint32 GetAllocatedSize(void) const { return FPrimitiveSceneProxy::GetAllocatedSize() + AggregateGeom.GetAllocatedSize(); }

	private:
		FKAggregateGeom AggregateGeom;
	};

	return new FOceanCollisionSceneProxy(this);
}

#endif // UE_ENABLE_DEBUG_DRAWING

bool UOceanCollisionComponent::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	const AWaterBody* OwningBody = GetTypedOuter<AWaterBody>();
	if (CachedBodySetup && OwningBody && OwningBody->GetWaterBodyComponent())
	{
		FTransform GeomTransform(GetComponentTransform());
		GeomTransform.AddToTranslation(OwningBody->GetWaterBodyComponent()->GetWaterNavCollisionOffset());
		GeomExport.ExportRigidBodySetup(*CachedBodySetup, GeomTransform);
		return false;
	}

	return true;
}

// ----------------------------------------------------------------------------------

UOceanBoxCollisionComponent::UOceanBoxCollisionComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UOceanBoxCollisionComponent::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	const AWaterBody* OwningBody = GetTypedOuter<AWaterBody>();
	if (ShapeBodySetup && OwningBody && OwningBody->GetWaterBodyComponent())
	{
		FTransform GeomTransform(GetComponentTransform());
		GeomTransform.AddToTranslation(OwningBody->GetWaterBodyComponent()->GetWaterNavCollisionOffset());
		GeomExport.ExportRigidBodySetup(*ShapeBodySetup, GeomTransform);
		return false;
	}
	return true;
}
