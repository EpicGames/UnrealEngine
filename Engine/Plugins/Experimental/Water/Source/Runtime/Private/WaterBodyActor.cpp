// Copyright Epic Games, Inc. All Rights Reserved.


#include "WaterBodyActor.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Components/BoxComponent.h"
#include "BuoyancyComponent.h"
#include "PropertyPairsMap.h"
#include "WaterSubsystem.h"
#include "WaterBodyIslandActor.h"
#include "WaterSplineComponent.h"
#include "WaterRuntimeSettings.h"
#include "WaterBodyCustomComponent.h"
#include "WaterBodyLakeComponent.h"
#include "WaterBodyOceanComponent.h"
#include "WaterBodyRiverComponent.h"
#include "WaterBodyInfoMeshComponent.h"
#include "WaterBodyStaticMeshComponent.h"
#include "WaterModule.h"
#include "WaterVersion.h"
#include "AI/NavigationSystemBase.h"
#include "Algo/RemoveIf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyActor)

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "Landscape.h"
#include "Engine/StaticMesh.h"
#include "Engine/Level.h"
#endif

#define LOCTEXT_NAMESPACE "Water"

TAutoConsoleVariable<float> CVarWaterSplineResampleMaxDistance(
	TEXT("r.Water.WaterSplineResampleMaxDistance"),
	50.0f,
	TEXT("Maximum distance between the sample segments and the spline when converting the water spline shapes to polygons (as distance discreases, the number of vertices will increase, \
		, the physics shapes will be more accurate, the water tiles will match more closely, but the computational cost will also increase)."),
	ECVF_Default);

// ----------------------------------------------------------------------------------

AWaterBody::AWaterBody(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, WaterBodyRiverComponentClass(UWaterBodyRiverComponent::StaticClass())
	, WaterBodyLakeComponentClass(UWaterBodyLakeComponent::StaticClass())
	, WaterBodyOceanComponentClass(UWaterBodyOceanComponent::StaticClass())
	, WaterBodyCustomComponentClass(UWaterBodyCustomComponent::StaticClass())
{
	SetCanBeDamaged(false);
	bNetLoadOnClient = true;

	SplineComp = CreateDefaultSubobject<UWaterSplineComponent>(TEXT("WaterSpline"));
	SplineComp->SetMobility(EComponentMobility::Static);

	WaterSplineMetadata = ObjectInitializer.CreateDefaultSubobject<UWaterSplineMetadata>(this, TEXT("WaterSplineMetadata"));
	WaterSplineMetadata->Reset(3);
	WaterSplineMetadata->AddPoint(0.0f);
	WaterSplineMetadata->AddPoint(1.0f);
	WaterSplineMetadata->AddPoint(2.0f);

	// Temporarily set the root component to the spline because the WaterBodyComponent has not yet been created
	RootComponent = SplineComp;

	WaterInfoMeshComponent = CreateDefaultSubobject<UWaterBodyInfoMeshComponent>(TEXT("WaterInfoMeshComponent"));
	WaterInfoMeshComponent->SetMobility(EComponentMobility::Static);
	WaterInfoMeshComponent->SetupAttachment(RootComponent);
	WaterInfoMeshComponent->bIsDilatedMesh = false;

	DilatedWaterInfoMeshComponent = CreateDefaultSubobject<UWaterBodyInfoMeshComponent>(TEXT("DilatedWaterInfoMeshComponent"));
	DilatedWaterInfoMeshComponent->SetMobility(EComponentMobility::Static);
	DilatedWaterInfoMeshComponent->SetupAttachment(RootComponent);
	DilatedWaterInfoMeshComponent->bIsDilatedMesh = true;

#if WITH_EDITORONLY_DATA
	bAffectsLandscape_DEPRECATED = true;
	CollisionProfileName_DEPRECATED = GetDefault<UWaterRuntimeSettings>()->GetDefaultWaterCollisionProfileName();

	WaterMID_DEPRECATED = nullptr;
	TargetWaveMaskDepth_DEPRECATED = 2048.f;
	bCanAffectNavigation_DEPRECATED = false;
	bFillCollisionUnderWaterBodiesForNavmesh_DEPRECATED = false;
#endif // WITH_EDITORONLY_DATA

}

void AWaterBody::PreRegisterAllComponents()
{
	Super::PreRegisterAllComponents();

	SetRootComponent(WaterBodyComponent);
	if (SplineComp)
	{
		SplineComp->AttachToComponent(WaterBodyComponent, FAttachmentTransformRules::KeepRelativeTransform);
	}
	
	if (IsValid(WaterInfoMeshComponent))
	{
		WaterInfoMeshComponent->SetMobility(WaterBodyComponent->Mobility);
		WaterInfoMeshComponent->AttachToComponent(WaterBodyComponent, FAttachmentTransformRules::SnapToTargetIncludingScale);
	}

	if (IsValid(DilatedWaterInfoMeshComponent))
	{
		DilatedWaterInfoMeshComponent->SetMobility(WaterBodyComponent->Mobility);
		DilatedWaterInfoMeshComponent->AttachToComponent(WaterBodyComponent, FAttachmentTransformRules::SnapToTargetIncludingScale);
	}
}

void AWaterBody::NotifyActorBeginOverlap(AActor* OtherActor)
{
	Super::NotifyActorBeginOverlap(OtherActor);
	if (UBuoyancyComponent* BuoyancyComponent = OtherActor->FindComponentByClass<UBuoyancyComponent>())
	{
		BuoyancyComponent->EnteredWaterBody(GetWaterBodyComponent());
	}
}

void AWaterBody::NotifyActorEndOverlap(AActor* OtherActor)
{
	Super::NotifyActorEndOverlap(OtherActor);
	if (UBuoyancyComponent* BuoyancyComponent = OtherActor->FindComponentByClass<UBuoyancyComponent>())
	{
		BuoyancyComponent->ExitedWaterBody(GetWaterBodyComponent());
	}
}

void AWaterBody::PreInitializeComponents()
{
	Super::PreInitializeComponents();

	check(WaterBodyComponent != nullptr);
	// some water bodies are dynamic (e.g. Ocean) and thus need to be regenerated at runtime :
	FOnWaterBodyChangedParams Params;
	Params.bShapeOrPositionChanged = true;
	Params.bWeightmapSettingsChanged = true;
	WaterBodyComponent->UpdateAll(Params);
}

#if WITH_EDITOR
void AWaterBody::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);

	// It's possible that the WaterBodyComponent is invalid here if, for example, we are part of a ChildActorComponent and are being destroyed/recreated every move.
	if (!IsValid(WaterBodyComponent))
	{
		return;
	}

	// We want to avoid modifying the water body if the world is being cleaned up.
	// World Partition unloads will call PostEditMove during the cleanup process and we don't want to unnecessarily make changes to the water body now.
	if (GetWorld() && GetWorld()->IsBeingCleanedUp())
	{
		return;
	}

	WaterBodyComponent->FixupEditorTransform();

	if (bFinished)
	{
		WaterBodyComponent->UpdateWaterHeight();
	}

	FOnWaterBodyChangedParams Params;
	Params.PropertyChangedEvent.ChangeType = bFinished ? EPropertyChangeType::ValueSet : EPropertyChangeType::Interactive;
	Params.bShapeOrPositionChanged = true;
	Params.bUserTriggered = true;
	WaterBodyComponent->OnWaterBodyChanged(Params);
}

void AWaterBody::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	const FName PropertyName = PropertyThatWillChange ? PropertyThatWillChange->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(AWaterBody, WaterWaves))
	{
		WaterBodyComponent->RegisterOnUpdateWavesData(WaterWaves, /* bRegister = */false);
	}
}

void AWaterBody::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(AWaterBody, WaterBodyType))
	{
		UWaterBodyComponent* OldComponent = WaterBodyComponent;
		InitializeBody();
		UEditorEngine::CopyPropertiesForUnrelatedObjects(OldComponent, WaterBodyComponent);
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(AWaterBody, WaterWaves))
	{
		// Avoid calling OnWaterBodyChanged on templates. When changing properties from the BP editor this prevents
		// the CDO from generating and adding collision components to the Blueprint unintentionally
		if (!IsTemplate())
		{
			WaterBodyComponent->RegisterOnUpdateWavesData(WaterWaves, /* bRegister = */true);

			WaterBodyComponent->RequestGPUWaveDataUpdate();

			FOnWaterBodyChangedParams Params;
			// Waves data affect the navigation : 
			Params.bShapeOrPositionChanged = true;
			WaterBodyComponent->OnWaterBodyChanged(Params);
		}
	}
}

void AWaterBody::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
}
#endif

void AWaterBody::SetWaterWaves(UWaterWavesBase* InWaterWaves)
{
	SetWaterWavesInternal(InWaterWaves);
}

void AWaterBody::SetWaterWavesInternal(UWaterWavesBase* InWaterWaves)
{
	if (InWaterWaves != WaterWaves)
	{
#if WITH_EDITOR
		WaterBodyComponent->RegisterOnUpdateWavesData(WaterWaves, /*bRegister = */false);
#endif // WITH_EDITOR

		WaterWaves = InWaterWaves;

#if WITH_EDITOR
		WaterBodyComponent->RegisterOnUpdateWavesData(WaterWaves, /*bRegister = */true);
#endif // WITH_EDITOR		

		WaterBodyComponent->RequestGPUWaveDataUpdate();

		FOnWaterBodyChangedParams Params;
		// Waves data can affect the navigation: 
		Params.bShapeOrPositionChanged = true;
		WaterBodyComponent->OnWaterBodyChanged(Params);
	}
}

void AWaterBody::CleanupInvalidStaticMeshComponents()
{
	WaterBodyStaticMeshComponents.SetNum(Algo::RemoveIf(WaterBodyStaticMeshComponents, [](const TObjectPtr<UWaterBodyStaticMeshComponent>& StaticMeshComponent)
	{
		return !IsValid(StaticMeshComponent);
	}));
}

void AWaterBody::SetWaterBodyStaticMeshComponents(TArrayView<TObjectPtr<UWaterBodyStaticMeshComponent>> NewComponentList, TConstArrayView<TObjectPtr<UWaterBodyStaticMeshComponent>> ComponentsToUnregister)
{
	for (const TObjectPtr<UWaterBodyStaticMeshComponent>& StaticMeshComponent : ComponentsToUnregister)
	{
		StaticMeshComponent.Get()->UnregisterComponent();
		StaticMeshComponent.Get()->DestroyComponent();
	}
	WaterBodyStaticMeshComponents = NewComponentList;
}

static FName GetWaterBodyComponentName(EWaterBodyType Type)
{
	switch (Type)
	{
	case EWaterBodyType::River:
		return FName(TEXT("WaterBodyRiverComponent"));
	case EWaterBodyType::Lake:
		return FName(TEXT("WaterBodyLakeComponent"));
	case EWaterBodyType::Ocean:
		return FName(TEXT("WaterBodyOceanComponent"));
	case EWaterBodyType::Transition:
		return FName(TEXT("WaterBodyCustomComponent"));
	default:
		checkf(false, TEXT("Invalid Water Body Type"));
		return FName(TEXT("WaterBodyUnknownComponent"));
	}
}

void AWaterBody::InitializeBody()
{
	const UWaterRuntimeSettings* WaterSettings = GetDefault<UWaterRuntimeSettings>();
	check(WaterSettings != nullptr);

	UClass* WaterBodyComponentClass = nullptr;
	UWaterBodyComponent* Template = nullptr;

	// If we are a template or this actor has a different type than its CDO we need to create the component from the settings
	if (IsTemplate() || GetClass()->GetDefaultObject<AWaterBody>()->GetWaterBodyType() != GetWaterBodyType())
	{
		switch (GetWaterBodyType())
		{
		case EWaterBodyType::River:
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			WaterBodyComponentClass = WaterSettings->GetWaterBodyRiverComponentClass().Get();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			if (WaterBodyComponentClass)
			{
				UE_LOGF(LogWater, Error, "Found non-null deprecated setting for %ls water bodies. Global WaterBodyComponent class overrides are no longer supported. Please create a new WaterBodyActor class within the same plugin as the component class to override the setting.", *UEnum::GetValueAsString(GetWaterBodyType()));
			}
			else
			{
				WaterBodyComponentClass = WaterBodyRiverComponentClass.Get();
			}
			break;
		case EWaterBodyType::Lake:
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			WaterBodyComponentClass = WaterSettings->GetWaterBodyLakeComponentClass().Get();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			if (WaterBodyComponentClass)
			{
				UE_LOGF(LogWater, Error, "Found non-null deprecated setting for %ls water bodies. Global WaterBodyComponent class overrides are no longer supported. Please create a new WaterBodyActor class within the same plugin as the component class to override the setting.", *UEnum::GetValueAsString(GetWaterBodyType()));
			}
			else
			{
				WaterBodyComponentClass = WaterBodyLakeComponentClass.Get();
			}
			break;
		case EWaterBodyType::Ocean:
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			WaterBodyComponentClass = WaterSettings->GetWaterBodyOceanComponentClass().Get();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			if (WaterBodyComponentClass)
			{
				UE_LOGF(LogWater, Error, "Found non-null deprecated setting for %ls water bodies. Global WaterBodyComponent class overrides are no longer supported. Please create a new WaterBodyActor class within the same plugin as the component class to override the setting.", *UEnum::GetValueAsString(GetWaterBodyType()));
			}
			else
			{
				WaterBodyComponentClass = WaterBodyOceanComponentClass.Get();
			}
			break;
		case EWaterBodyType::Transition:
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			WaterBodyComponentClass = WaterSettings->GetWaterBodyCustomComponentClass().Get();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			if (WaterBodyComponentClass)
			{
				UE_LOGF(LogWater, Error, "Found non-null deprecated setting for %ls water bodies. Global WaterBodyComponent class overrides are no longer supported. Please create a new WaterBodyActor class within the same plugin as the component class to override the setting.", *UEnum::GetValueAsString(GetWaterBodyType()));
			}
			else
			{
				WaterBodyComponentClass = WaterBodyCustomComponentClass.Get();
			}
			break;
		default:
			checkf(false, TEXT("Invalid Water Body Type"));
			break;
		}

		checkf(WaterBodyComponentClass, TEXT("Ensure there is a proper class for this water type: %s"), *UEnum::GetValueAsString(GetWaterBodyType()));
	}
	else
	{
		//check(WaterBodyComponent == nullptr || WaterBodyComponent->GetWaterBodyType() == GetWaterBodyType())

		AWaterBody* DefaultActor = GetClass()->GetDefaultObject<AWaterBody>();
		Template = DefaultActor->GetWaterBodyComponent();

		check(Template);
		//check(Template->GetWaterBodyType() == GetWaterBodyType());

		WaterBodyComponentClass = Template->GetClass();
	}

	check(WaterBodyComponentClass != nullptr);

	// The WaterBodyComponent field is sometimes not set (or set incorrectly) by the time we call InitializeBody so attempt to retrieve 
	// the component pointer directly from the list of components
	if (WaterBodyComponent == nullptr || WaterBodyComponent->GetOwner() != this || WaterBodyComponent->GetWaterBodyType() != GetWaterBodyType())
	{
		WaterBodyComponent = Cast<UWaterBodyComponent>(FindComponentByClass(WaterBodyComponentClass));
	}

	if (!WaterBodyComponent)
	{
		const EObjectFlags Flags = GetMaskedFlags(RF_PropagateToSubObjects) | RF_DefaultSubObject;
		WaterBodyComponent = NewObject<UWaterBodyComponent>(this, WaterBodyComponentClass, GetWaterBodyComponentName(GetWaterBodyType()), Flags, Template);
		WaterBodyComponent->SetMobility(EComponentMobility::Static);
	}

	checkf(WaterBodyComponent, TEXT("Failed to create a water body component for a water body actor (%s)!"), *GetActorNameOrLabel());

	SetRootComponent(WaterBodyComponent);

	// We need to retrieve the list of WaterBodyComponents to delete any inherited components of the wrong type
	TInlineComponentArray<UWaterBodyComponent*> WaterBodyComponents;
	GetComponents(WaterBodyComponents);
	for (UWaterBodyComponent* Component : WaterBodyComponents)
	{
		if (Component && Component->GetWaterBodyType() != GetWaterBodyType())
		{
			Component->DestroyComponent();
		}
	}
}

void AWaterBody::DeprecateData()
{
	// Note: this function will be called multiple times so its important that the deprecated data

	// Deprecation of actor data must happen during serialize so that CDOs are updated correctly for delta-serialization.
	// Without this, WaterBodyComponents will serialize based on an incorrect archetype and all "default" properties will be reset
	// to their native defaults. The deprecation is still also required in ::PostLoad because non-cdo components will lose the 
	// data during their Serialize which occurs after this.
#if WITH_EDITORONLY_DATA
	if (SplineComp)
	{
		if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::MoveWaterMetadataToActor)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (SplineComp->SplineCurves.Metadata_DEPRECATED)
			{
				UWaterSplineMetadata* OldSplineMetadata = Cast<UWaterSplineMetadata>(SplineComp->SplineCurves.Metadata_DEPRECATED);
				SplineComp->SplineCurves.Metadata_DEPRECATED = nullptr;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				if (WaterSplineMetadata)
				{
					WaterSplineMetadata->Depth = OldSplineMetadata->Depth;
					WaterSplineMetadata->WaterVelocityScalar = OldSplineMetadata->WaterVelocityScalar;
					WaterSplineMetadata->RiverWidth = OldSplineMetadata->RiverWidth;
				}
			}
		}
	}

	if (WaterBodyType == EWaterBodyType::Lake && GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ReplaceLakeCollision)
	{
		TArray<UPrimitiveComponent*> LocalCollisionComponents = WaterBodyComponent->GetCollisionComponents();
		
		for (auto It = LocalCollisionComponents.CreateIterator(); It; ++It)
		{
			if (UBoxComponent* OldLakeCollision = Cast<UBoxComponent>(*It))
			{
				OldLakeCollision->ConditionalPostLoad();

				OldLakeCollision->DestroyComponent();
				// Rename it so we can use the name
				OldLakeCollision->Rename(TEXT("LakeCollision_Old"), this, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
				It.RemoveCurrent();
			}
		}
	}

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::FixUpUnderwaterPostProcessMaterial)
	{
		// Get back the underwater post process material from where it was before : 
		// UnderwaterPostProcessMaterial_DEPRECATED takes priority as it was used to override the material from WeightedBlendables that was set via the BP : 
		if (UnderwaterPostProcessSettings_DEPRECATED.UnderwaterPostProcessMaterial_DEPRECATED)
		{
			UnderwaterPostProcessMaterial_DEPRECATED = UnderwaterPostProcessSettings_DEPRECATED.UnderwaterPostProcessMaterial_DEPRECATED;
		}
		else if (UnderwaterPostProcessSettings_DEPRECATED.PostProcessSettings.WeightedBlendables.Array.Num() > 0)
		{
			UnderwaterPostProcessMaterial_DEPRECATED = Cast<UMaterialInterface>(UnderwaterPostProcessSettings_DEPRECATED.PostProcessSettings.WeightedBlendables.Array[0].Object);
			UnderwaterPostProcessSettings_DEPRECATED.PostProcessSettings.WeightedBlendables.Array.Empty();
		}
		// If the material was actually already a MID, use its parent, we will instantiate a transient MID out of it from code anyway : 
		if (UnderwaterPostProcessMaterial_DEPRECATED)
		{
			if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(UnderwaterPostProcessMaterial_DEPRECATED))
			{
				UnderwaterPostProcessMaterial_DEPRECATED = MID->GetMaterial();
			}
		}

		// don't call CreateOrUpdateUnderwaterPostProcessMID() just yet because we need the water mesh actor to be registerd
	}

	if (GetLinkerCustomVersion(FWaterCustomVersion::GUID) < FWaterCustomVersion::WaterBodyRefactor)
	{
		// Try to retrieve wave data from BP properties when it was defined in BP : 
		if (UClass* WaterBodyClass = GetClass())
		{
			if (WaterBodyClass->ClassGeneratedBy != nullptr)
			{
				FStructProperty* OldWaveStructProperty = nullptr;
				for (FProperty* BPProperty = WaterBodyClass->PropertyLink; BPProperty != nullptr; BPProperty = BPProperty->PropertyLinkNext)
				{
					const FString WaveSpectrumSettingsName(TEXT("Wave Spectrum Settings"));
					if (BPProperty->GetName() == WaveSpectrumSettingsName)
					{
						OldWaveStructProperty = CastField<FStructProperty>(BPProperty);
						break;
					}
				}

				if (OldWaveStructProperty != nullptr)
				{
					void* OldPropertyOnWaveSpectrumSettings = OldWaveStructProperty->ContainerPtrToValuePtr<void>(this);
					// We need to propagate object flags to the sub objects (if we deprecate an archetype's data, it is public and its sub-object need to be as well) :
					EObjectFlags NewFlags = GetMaskedFlags(RF_PropagateToSubObjects) | RF_Transactional;
					UGerstnerWaterWaves* GerstnerWaves = NewObject<UGerstnerWaterWaves>(this, MakeUniqueObjectName(this, UGerstnerWaterWaves::StaticClass(), TEXT("GestnerWaterWaves")), NewFlags);
					UClass* NewGerstnerClass = UGerstnerWaterWaveGeneratorSimple::StaticClass();
					UGerstnerWaterWaveGeneratorSimple* GerstnerWavesGenerator = NewObject<UGerstnerWaterWaveGeneratorSimple>(this, MakeUniqueObjectName(this, NewGerstnerClass, TEXT("GestnerWaterWavesGenerator")), NewFlags);
					GerstnerWaves->GerstnerWaveGenerator = GerstnerWavesGenerator;
					WaterWaves = GerstnerWaves; // we're in PostLoad, we don't want to send the water body changed event as it might re-enter into BP script

					for (FProperty* NewProperty = NewGerstnerClass->PropertyLink; NewProperty != nullptr; NewProperty = NewProperty->PropertyLinkNext)
					{
						void* NewPropertyOnGerstnerWavesGenerator = NewProperty->ContainerPtrToValuePtr<void>(GerstnerWavesGenerator);

						// Iterate through each property field in the lightmass settings struct that we are copying from...
						for (TFieldIterator<FProperty> OldIt(OldWaveStructProperty->Struct); OldIt; ++OldIt)
						{
							FProperty* OldProperty = *OldIt;
							void* OldPropertyToCopy = OldProperty->ContainerPtrToValuePtr<void>(OldPropertyOnWaveSpectrumSettings);
							if ((OldProperty->GetName().Contains(NewProperty->GetName()))
								|| (OldProperty->GetName().Contains(FString(TEXT("MaxWaves"))) && (NewProperty->GetName() == FString(TEXT("NumWaves")))))
							{
								OldProperty->CopySingleValue(NewPropertyOnGerstnerWavesGenerator, OldPropertyToCopy);
								break;
							}
							else if (OldProperty->GetName().Contains(FString(TEXT("DominantWaveDirection"))) && (NewProperty->GetName() == FString(TEXT("WindAngleDeg"))))
							{
								FVector2D Direction2D;
								OldProperty->CopySingleValue(&Direction2D, OldPropertyToCopy);
								FVector Direction(Direction2D, 0.0f);
								FRotator Rotator = Direction.Rotation();
								GerstnerWavesGenerator->WindAngleDeg = Rotator.Yaw;
								break;
							}
						}
					}
				}
			}
		}
	}

	if (GetLinkerCustomVersion(FWaterCustomVersion::GUID) < FWaterCustomVersion::FixupUnserializedGerstnerWaves)
	{
		// At one point, some attributes from UGerstnerWaterWaves were transient, recompute those here at load-time (nowadays, they are serialized properly so they should be properly recompute on property change)
		if (WaterBodyComponent->HasWaves())
		{
			check(WaterWaves != nullptr);
			if (UGerstnerWaterWaves* GerstnerWaterWaves = Cast<UGerstnerWaterWaves>(WaterWaves->GetWaterWaves()))
			{
				GerstnerWaterWaves->ConditionalPostLoad();
				GerstnerWaterWaves->RecomputeWaves(/*bAllowBPScript = */false); // We're in PostLoad, don't let BP script run, this is forbidden
			}
		}
	}

	if (GetLinkerCustomVersion(FWaterCustomVersion::GUID) < FWaterCustomVersion::MoveTerrainCarvingSettingsToWater)
	{
		// This horrific piece of code is for salvaging data from the old struct (TerrainCarvingSettings_DEPRECATED, of type FLandmassTerrainCarvingSettings), to the 
		//  "new" struct (WaterHeightmapSettings_DEPRECATED, of type FWaterBodyHeightmapSettings) (which is also deprecated, mind you, since it has moved into UWaterBodyComponent). 
		//  A straight-up Memcpy used to be enough, but now the Effects.Displacement.Texture has gone from TObjectPtr to TSoftObjectPtr, so we cannot do this anymore and we have to copy 
		//  each sub-struct one by one... The code is still there for perfect backwards-compatibility, but should not ever be exercised, since FLandmassTerrainCarvingSettings was only used 
		//  before Water became public... 
#define COPY_SIMILAR_SETTINGS(MemberName) \
		static_assert(sizeof(WaterHeightmapSettings_DEPRECATED.MemberName) == sizeof(TerrainCarvingSettings_DEPRECATED.MemberName), "Both old and new settings struct should be exactly similar"); \
		FMemory::Memcpy((void*)&WaterHeightmapSettings_DEPRECATED.MemberName, (void*)&TerrainCarvingSettings_DEPRECATED.MemberName, sizeof(WaterHeightmapSettings_DEPRECATED.MemberName));

		COPY_SIMILAR_SETTINGS(BlendMode);
		COPY_SIMILAR_SETTINGS(FalloffSettings);
		COPY_SIMILAR_SETTINGS(Effects.Blurring);
		COPY_SIMILAR_SETTINGS(Effects.CurlNoise);
		// Copy FWaterBrushEffects from FLandmassBrushEffectsList by hand because it is not binary-compatible : 
		WaterHeightmapSettings_DEPRECATED.Effects.Displacement.DisplacementHeight = TerrainCarvingSettings_DEPRECATED.Effects.Displacement.DisplacementHeight;
		WaterHeightmapSettings_DEPRECATED.Effects.Displacement.DisplacementTiling = TerrainCarvingSettings_DEPRECATED.Effects.Displacement.DisplacementTiling;
		WaterHeightmapSettings_DEPRECATED.Effects.Displacement.Texture = TerrainCarvingSettings_DEPRECATED.Effects.Displacement.Texture;
		WaterHeightmapSettings_DEPRECATED.Effects.Displacement.Midpoint = TerrainCarvingSettings_DEPRECATED.Effects.Displacement.Midpoint;
		WaterHeightmapSettings_DEPRECATED.Effects.Displacement.Channel = TerrainCarvingSettings_DEPRECATED.Effects.Displacement.Channel;
		WaterHeightmapSettings_DEPRECATED.Effects.Displacement.WeightmapInfluence = TerrainCarvingSettings_DEPRECATED.Effects.Displacement.WeightmapInfluence;
		COPY_SIMILAR_SETTINGS(Effects.SmoothBlending);
		COPY_SIMILAR_SETTINGS(Effects.Terracing);
		WaterHeightmapSettings_DEPRECATED.Priority_DEPRECATED = TerrainCarvingSettings_DEPRECATED.Priority;
	}

	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject | RF_DefaultSubObject))
	{
		if (WaterWaves && (!WaterWaves->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject | RF_DefaultSubObject)) && (WaterWaves->GetOuter() != this))
		{
			WaterWaves->ClearFlags(RF_Public);
			// At one point, WaterWaves's outer was the level. We need them to be outered by the water body : 
			WaterWaves->Rename(nullptr, this, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
		}
	}
	
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterBodyComponentRefactor)
	{
		check(WaterBodyComponent && WaterBodyComponent->GetWaterBodyType() == GetWaterBodyType());
		check(SplineComp);

		WaterBodyComponent->Mobility = SplineComp->Mobility;
		WaterBodyComponent->PhysicalMaterial_DEPRECATED = PhysicalMaterial_DEPRECATED;
		WaterBodyComponent->TargetWaveMaskDepth = TargetWaveMaskDepth_DEPRECATED;
		WaterBodyComponent->MaxWaveHeightOffset = MaxWaveHeightOffset_DEPRECATED;
		WaterBodyComponent->bFillCollisionUnderWaterBodiesForNavmesh_DEPRECATED = bFillCollisionUnderWaterBodiesForNavmesh_DEPRECATED;
		WaterBodyComponent->UnderwaterPostProcessSettings = UnderwaterPostProcessSettings_DEPRECATED;
		WaterBodyComponent->CurveSettings = CurveSettings_DEPRECATED;
		WaterBodyComponent->SetWaterMaterial(WaterMaterial_DEPRECATED);
		WaterBodyComponent->SetUnderwaterPostProcessMaterial(UnderwaterPostProcessMaterial_DEPRECATED);
		WaterBodyComponent->WaterHeightmapSettings = WaterHeightmapSettings_DEPRECATED;
		WaterBodyComponent->LayerWeightmapSettings = LayerWeightmapSettings_DEPRECATED;
		WaterBodyComponent->bAffectsLandscape = bAffectsLandscape_DEPRECATED;
		WaterBodyComponent->bGenerateCollisions_DEPRECATED = bGenerateCollisions_DEPRECATED;
		WaterBodyComponent->WaterMeshOverride = WaterMeshOverride_DEPRECATED;
		WaterBodyComponent->OverlapMaterialPriority = OverlapMaterialPriority_DEPRECATED;
		WaterBodyComponent->CollisionProfileName_DEPRECATED = CollisionProfileName_DEPRECATED;
		WaterBodyComponent->WaterMID = WaterMID_DEPRECATED;
		WaterBodyComponent->UnderwaterPostProcessMID = UnderwaterPostProcessMID_DEPRECATED;
		WaterBodyComponent->Islands_DEPRECATED = Islands_DEPRECATED;
		WaterBodyComponent->ExclusionVolumes_DEPRECATED = ExclusionVolumes_DEPRECATED;
		WaterBodyComponent->bCanAffectNavigation_DEPRECATED = bCanAffectNavigation_DEPRECATED;
		WaterBodyComponent->WaterNavAreaClass = WaterNavAreaClass_DEPRECATED;
		WaterBodyComponent->ShapeDilation = ShapeDilation_DEPRECATED;

		// Some deprecated data on the actor were also later deprecated on the component (e.g. bFillCollisionUnderWaterBodiesForNavmesh_DEPRECATED), make sure it runs its own deprecation operations on it : 
		WaterBodyComponent->DeprecateData();
	}
#endif // WITH_EDITORONLY_DATA
}

void AWaterBody::PostInitProperties()
{
	Super::PostInitProperties();

	InitializeBody();
}

void AWaterBody::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FWaterCustomVersion::GUID);

	if (Ar.IsLoading() && !Ar.IsTransacting())
	{
		InitializeBody();

#if WITH_EDITORONLY_DATA
		DeprecateData();
#endif // WITH_EDITORONLY_DATA
	}
}

void AWaterBody::PostLoad()
{
	Super::PostLoad();

	InitializeBody();

#if WITH_EDITORONLY_DATA
	if (SplineComp)
	{
		// Keep metadata in sync
		if (WaterSplineMetadata)
		{
			const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();
			WaterSplineMetadata->Fixup(NumPoints, SplineComp);
		}
	}
	
	DeprecateData();
	
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterBodyComponentRefactor)
	{
		WaterBodyComponent->SetRelativeTransform(SplineComp->GetRelativeTransform());
		SplineComp->ResetRelativeTransform();
	}
#endif
}

void AWaterBody::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	if (IsValid(WaterBodyComponent))
	{
		FOnWaterBodyChangedParams Params;
		Params.bShapeOrPositionChanged = true;
		Params.bWeightmapSettingsChanged = true;
		WaterBodyComponent->UpdateAll(Params);
		WaterBodyComponent->UpdateMaterialInstances();

#if WITH_EDITOR
		// Make sure existing collision components are marked as net-addressable (their names should already be deterministic) :
		TArray<UPrimitiveComponent*> LocalCollisionComponents = WaterBodyComponent->GetCollisionComponents();
		for (auto It = LocalCollisionComponents.CreateIterator(); It; ++It)
		{
			if (UActorComponent* CollisionComponent = Cast<UActorComponent>(*It))
			{
				CollisionComponent->SetNetAddressable();
			}
		}
#endif // WITH_EDITOR

		// We must check for WaterBodyIndex to see if we have already been registered because PostRegisterAllComponents can be called multiple times in a row (e.g. if the actor is a child 
		//  actor of another BP, the parent BP instance will register first, with all its child components, which will trigger registration of the child water body actor, and then 
		//  the water body actor will also get registered independently as a "standard" actor) :
		FWaterBodyManager* Manager = UWaterSubsystem::GetWaterBodyManager(GetWorld());
		if (Manager && !IsTemplate() && (WaterBodyComponent->WaterBodyIndex == INDEX_NONE))
		{
			WaterBodyComponent->WaterBodyIndex = Manager->AddWaterBodyComponent(WaterBodyComponent);
			WaterBodyIndex = WaterBodyComponent->WaterBodyIndex;
		}

		// At this point, the water mesh actor should be ready and we can setup the MID accordingly : 
		// Needs to be done at the end so that all data needed by the MIDs (e.g. WaterBodyIndex) is up to date :
		WaterBodyComponent->UpdateMaterialInstances();

#if WITH_EDITOR
		// We need to generate the water body render data for the first time here after the waterbody component is registered to the actor.
		if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterBodyStaticMeshComponents)
		{
			WaterBodyComponent->UpdateWaterBodyRenderData();
		}
#endif // WITH_EDITOR

		WaterBodyComponent->OnPostRegisterAllComponents();

		// Now that all components are registered and setup we can notify the navigation system.
		if (WaterBodyComponent->IsNavigationRelevant())
		{
			FNavigationSystem::UpdateActorAndComponentData(*this);
		}
	}
}

bool AWaterBody::IsHLODRelevant() const
{
	return true;
}

bool AWaterBody::IsComponentRelevantForNavigation(UActorComponent* Component) const
{
	// Wait for the whole setup to be completed before making child components relevant to navigation (i.e. PostRegisterAllComponents).
	// Then we update the actor and all its components (i.e. FNavigationSystem::UpdateActorAndComponentData).
	// This reduces the number of redundant processing in the navigation system while setting up the components.
	return HasActorRegisteredAllComponents();
}

#if WITH_EDITOR
void AWaterBody::PreDuplicateFromRoot(FObjectDuplicationParameters& DupParams)
{
	Super::PreDuplicateFromRoot(DupParams);

	if (DupParams.DuplicateMode == EDuplicateMode::PIE)
	{
		// Avoid copying expensive UStaticMesh when entering pie, instead use the duplication seed to have the PIE instance share the same mesh pointer.
		TArray<TObjectPtr<UWaterBodyMeshComponent>> MeshComponents = { WaterInfoMeshComponent, DilatedWaterInfoMeshComponent };
		MeshComponents.Append(WaterBodyStaticMeshComponents);

		for (const TObjectPtr<UWaterBodyMeshComponent>& MeshComponent : MeshComponents)
		{
			if (IsValid(MeshComponent) && IsValid(MeshComponent->GetStaticMesh()))
			{
				UStaticMesh* StaticMesh = MeshComponent->GetStaticMesh();
				DupParams.DuplicationSeed.Emplace(StaticMesh, StaticMesh);
			}
		}
	}
}

void AWaterBody::SetActorHiddenInGame(bool bNewHidden)
{
	// It's kinda sad that being hidden in game for the actor doesn't end up calling OnHiddenInGameChanged on the component but intercepting it on the actor, we can inform the component that it needs
	//  to update its (internal components') visibility :
	if (IsHidden() != bNewHidden)
	{
		Super::SetActorHiddenInGame(bNewHidden);

		if (WaterBodyComponent)
		{
			WaterBodyComponent->UpdateVisibility();
		}
	}
}

void AWaterBody::SetIsTemporarilyHiddenInEditor(bool bIsHidden)
{
	// It's kinda sad that there's no way for the component to override a virtual function for when it becomes hidden in editor but intercepting it on the actor, we can inform the component that it needs
	//  to update its (internal components') visibility :
	if (IsTemporarilyHiddenInEditor() != bIsHidden)
	{
		Super::SetIsTemporarilyHiddenInEditor(bIsHidden);

		if (WaterBodyComponent)
		{
			WaterBodyComponent->UpdateVisibility();
		}
	}
}

bool AWaterBody::SetIsHiddenEdLayer(bool bIsHiddenEdLayer)
{
	// It's kinda sad that there's no way for the component to override a virtual function for when its layer becomes hidden but intercepting it on the actor, we can inform the component that it needs
	//  to update its (internal components') visibility :
	if (Super::SetIsHiddenEdLayer(bIsHiddenEdLayer))
	{
		if (WaterBodyComponent)
		{
			WaterBodyComponent->UpdateVisibility();
		}
		return true;
	}
	return false;
}

void AWaterBody::GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const
{
	Super::GetActorDescProperties(PropertyPairsMap);
	if (AffectsLandscape())
	{
		PropertyPairsMap.AddProperty(ALandscape::AffectsLandscapeActorDescProperty);
	}
}

void AWaterBody::PostActorCreated()
{
	Super::PostActorCreated();

	FOnWaterBodyChangedParams Params;

	Params.PropertyChangedEvent.ChangeType = EPropertyChangeType::ValueSet;
	Params.bShapeOrPositionChanged = true;
	Params.bUserTriggered = false;
	
	WaterBodyComponent->OnWaterBodyChanged(Params);

	WaterBodyComponent->OnPostActorCreated();
}

#endif // WITH_EDITOR

void AWaterBody::UnregisterAllComponents(bool bForReregister)
{
	if (WaterBodyComponent)
	{
		// We must check for WaterBodyIndex because UnregisterAllComponents can be called multiple times in a row by PostEditChangeProperty, etc.
		FWaterBodyManager* Manager = UWaterSubsystem::GetWaterBodyManager(GetWorld());
		if (Manager && !IsTemplate() && (WaterBodyComponent->WaterBodyIndex != INDEX_NONE))
		{
			Manager->RemoveWaterBodyComponent(WaterBodyComponent);
		}
		WaterBodyComponent->WaterBodyIndex = INDEX_NONE;
		WaterBodyIndex = INDEX_NONE;
	}

	Super::UnregisterAllComponents(bForReregister);
}

// ----------------------------------------------------------------------------------

UDEPRECATED_WaterBodyGenerator::UDEPRECATED_WaterBodyGenerator(const FObjectInitializer& Initializer) : Super(Initializer) {}

#undef LOCTEXT_NAMESPACE 
