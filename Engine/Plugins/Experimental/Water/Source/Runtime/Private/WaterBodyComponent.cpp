// Copyright Epic Games, Inc. All Rights Reserved.


#include "WaterBodyComponent.h"
#include "DynamicMesh/InfoTypes.h"
#include "UObject/UObjectIterator.h"
#include "EngineUtils.h"
#include "LandscapeProxy.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "AI/NavigationSystemBase.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "Spatial/MeshAABBTree3.h"
#include "WaterModule.h"
#include "WaterBodyActor.h"
#include "WaterSubsystem.h"
#include "WaterBodyExclusionVolume.h"
#include "WaterBodyIslandActor.h"
#include "WaterEditorServices.h"
#include "WaterSplineComponent.h"
#include "WaterRuntimeSettings.h"
#include "WaterUtils.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Engine/Texture2D.h"
#include "WaterMeshComponent.h"
#include "WaterVersion.h"
#include "Misc/UObjectToken.h"
#include "Logging/MessageLog.h"
#include "Engine/StaticMesh.h"
#include "WaterBodyStaticMeshComponent.h"
#include "WaterBodyMeshComponent.h"
#include "WaterBodyInfoMeshComponent.h"
#include "LocalVertexFactory.h"
#include "UObject/ICookInfo.h"
#include "UObject/ObjectSaveContext.h"
#include "DataDrivenShaderPlatformInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyComponent)

#if WITH_EDITOR
#include "Components/BillboardComponent.h"
#include "Modules/ModuleManager.h"
#include "WaterModule.h"
#include "StaticMeshAttributes.h"
#include "WaterBodyHLODBuilder.h"
#include "MeshMergeModule.h"
#include "WaterBodyMeshBuilder.h"
#include "MeshDescription.h"
#include "Algo/RemoveIf.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"
#endif

#define LOCTEXT_NAMESPACE "Water"

// ----------------------------------------------------------------------------------

DECLARE_CYCLE_STAT(TEXT("WaterBody_ComputeWaterInfo"), STAT_WaterBody_ComputeWaterInfo, STATGROUP_Water);
DECLARE_CYCLE_STAT(TEXT("WaterBody_ComputeWaterDepth"), STAT_WaterBody_ComputeWaterDepth, STATGROUP_Water);
DECLARE_CYCLE_STAT(TEXT("WaterBody_ComputeLocation"), STAT_WaterBody_ComputeLocation, STATGROUP_Water);
DECLARE_CYCLE_STAT(TEXT("WaterBody_ComputeNormal"), STAT_WaterBody_ComputeNormal, STATGROUP_Water);
DECLARE_CYCLE_STAT(TEXT("WaterBody_ComputeLandscapeDepth"), STAT_WaterBody_ComputeLandscapeDepth, STATGROUP_Water);
DECLARE_CYCLE_STAT(TEXT("WaterBody_ComputeWaveHeight"), STAT_WaterBody_ComputeWaveHeight, STATGROUP_Water);

// ----------------------------------------------------------------------------------

TAutoConsoleVariable<float> CVarWaterOceanFallbackDepth(
	TEXT("r.Water.OceanFallbackDepth"),
	3000.0f,
	TEXT("Depth to report for the ocean when no terrain is found under the query location. Not used when <= 0."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarWaterBodyBuildConservativeRasterizationMesh(
	TEXT("r.Water.BuildConservativeRasterizationMesh"), 1,
	TEXT("Enables additional data in the UV channels of the water mesh, which is used for software conservative rasterization when creating the GPU water quadtree."),
	ECVF_ReadOnly);

const FName UWaterBodyComponent::WaterBodyIndexParamName(TEXT("WaterBodyIndex"));
const FName UWaterBodyComponent::WaterZoneIndexParamName(TEXT("WaterZoneIndex"));
const FName UWaterBodyComponent::WaterBodyZOffsetParamName(TEXT("WaterBodyZOffset"));
const FName UWaterBodyComponent::WaterVelocityAndHeightName(TEXT("WaterVelocityAndHeight"));
const FName UWaterBodyComponent::GlobalOceanHeightName(TEXT("GlobalOceanHeight"));
const FName UWaterBodyComponent::MaxFlowVelocityParamName(TEXT("MaxFlowVelocity"));

namespace UE::Water
{
	static bool ShouldUpdateWaterMeshForPropertyChange(const FPropertyChangedEvent& InPropertyChangedEvent)
	{
#if WITH_EDITOR
		if (InPropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
		{
			const IWaterModuleInterface& WaterModule = FModuleManager::GetModuleChecked<IWaterModuleInterface>("Water");
			if (const IWaterEditorServices* WaterEditorServices = WaterModule.GetWaterEditorServices())
			{
				return WaterEditorServices->GetShouldUpdateWaterMeshDuringInteractiveChanges();
			}
		}
#endif // WITH_EDITOR

		return true;
	}
}

UWaterBodyComponent::UWaterBodyComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAffectsLandscape = true;

	SetCollisionProfileName(GetDefault<UWaterRuntimeSettings>()->GetDefaultWaterCollisionProfileName());

	WaterMID = nullptr;
	WaterInfoMID = nullptr;

	TargetWaveMaskDepth = 2048.f;

	bFillCollisionUnderneathForNavmesh = false;
	bCanEverAffectNavigation = false;

#if WITH_EDITORONLY_DATA
	// Maintain the old default values for deprecated members so delta serialization is still correct when we deprecate them in PostLoad
	bCanAffectNavigation_DEPRECATED = false;
	bFillCollisionUnderWaterBodiesForNavmesh_DEPRECATED = false;
	CollisionProfileName_DEPRECATED = GetDefault<UWaterRuntimeSettings>()->GetDefaultWaterCollisionProfileName();
#endif // WITH_EDITORONLY_DATA
}

bool UWaterBodyComponent::IsHLODRelevant() const
{
	if (!CanBeHLODRelevant(this))
	{
		return false;
	}

	return bEnableAutoLODGeneration;
}

void UWaterBodyComponent::OnVisibilityChanged()
{
	Super::OnVisibilityChanged();

	UpdateVisibility();
}

void UWaterBodyComponent::OnHiddenInGameChanged()
{
	Super::OnHiddenInGameChanged();

	UpdateVisibility();
}

bool UWaterBodyComponent::IsFlatSurface() const
{
	// Lakes and oceans have surfaces aligned with the XY plane
	return (GetWaterBodyType() == EWaterBodyType::Lake || GetWaterBodyType() == EWaterBodyType::Ocean);
}

bool UWaterBodyComponent::IsWaveSupported() const
{
	return (GetWaterBodyType() == EWaterBodyType::Lake || GetWaterBodyType() == EWaterBodyType::Ocean || GetWaterBodyType() == EWaterBodyType::Transition);
}

bool UWaterBodyComponent::HasWaves() const
{ 
	if (!IsWaveSupported())
	{
		return false;
	}
	return GetWaterWaves() ? (GetWaterWaves()->GetWaterWaves() != nullptr) : false;
}

FBox UWaterBodyComponent::GetCollisionComponentBounds() const
{
	FBox Box(ForceInit);
	for (UPrimitiveComponent* CollisionComponent : GetCollisionComponents())
	{
		if (CollisionComponent && CollisionComponent->IsRegistered())
		{
			Box += CollisionComponent->Bounds.GetBox();
		}
	}
	return Box;
}

UPhysicalMaterial* UWaterBodyComponent::GetPhysicalMaterial() const
{
	return BodyInstance.GetPhysMaterialOverride();
}

void UWaterBodyComponent::SetWaterMeshOverride(UStaticMesh* InMesh)
{
	WaterMeshOverride = InMesh;

	FOnWaterBodyChangedParams Params;
	Params.bShapeOrPositionChanged = true;
	Params.bUserTriggered = true;
	Params.bWeightmapSettingsChanged = false;
	OnWaterBodyChanged(Params);
}

AWaterBody* UWaterBodyComponent::GetWaterBodyActor() const
{
	// If we have an Owner, it must be an AWaterBody
	return GetOwner() ? Cast<AWaterBody>(GetOwner()) : nullptr;
}

UWaterSplineComponent* UWaterBodyComponent::GetWaterSpline() const
{
	if (const AWaterBody* OwningWaterBody = GetWaterBodyActor())
	{
		return OwningWaterBody->GetWaterSpline();
	}
	return nullptr;
}

UWaterBodyInfoMeshComponent* UWaterBodyComponent::GetWaterInfoMeshComponent() const
{
	if (const AWaterBody* OwningWaterBody = GetWaterBodyActor())
	{
		return OwningWaterBody->WaterInfoMeshComponent.Get();
	}
	return nullptr;
}

UWaterBodyInfoMeshComponent* UWaterBodyComponent::GetDilatedWaterInfoMeshComponent() const
{
	if (const AWaterBody* OwningWaterBody = GetWaterBodyActor())
	{
		return OwningWaterBody->DilatedWaterInfoMeshComponent.Get();
	}
	return nullptr;
}

bool UWaterBodyComponent::IsWaterSplineClosedLoop() const
{
	return (GetWaterBodyType() == EWaterBodyType::Lake) || (GetWaterBodyType() == EWaterBodyType::Ocean);
}

bool UWaterBodyComponent::IsHeightOffsetSupported() const
{
	return GetWaterBodyType() == EWaterBodyType::Ocean;
}

bool UWaterBodyComponent::AffectsLandscape() const
{
	return bAffectsLandscape && (GetWaterBodyType() != EWaterBodyType::Transition);
}

bool UWaterBodyComponent::AffectsWaterMesh() const
{ 
	return ShouldGenerateWaterMeshTile();
}

bool UWaterBodyComponent::AffectsWaterInfo() const
{
	// Currently only water bodies which are rendered by the water mesh can render into the water info texture
	return ShouldGenerateWaterMeshTile();
}

#if WITH_EDITOR
ETextureRenderTargetFormat UWaterBodyComponent::GetBrushRenderTargetFormat() const
{
	return (GetWaterBodyType() == EWaterBodyType::River) ? ETextureRenderTargetFormat::RTF_RGBA32f : ETextureRenderTargetFormat::RTF_RGBA16f;
}

void UWaterBodyComponent::GetBrushRenderDependencies(TSet<UObject*>& OutDependencies) const
{
	// Lazy-load the referenced textures : this allows the settings to not be editor-only but to only actually load data if GetBrushRenderDependencies is called (which is : only in the editor)
	for (const TPair<FName, FWaterBodyWeightmapSettings>& Pair : LayerWeightmapSettings)
	{
		if (UTexture2D* Texture = Pair.Value.ModulationTexture.LoadSynchronous())
		{
			OutDependencies.Add(Texture);
		}
	}

	if (UTexture2D* Texture = WaterHeightmapSettings.Effects.Displacement.Texture.LoadSynchronous())
	{
		OutDependencies.Add(Texture);
	}
}
#endif //WITH_EDITOR

void UWaterBodyComponent::SetWaterMaterial(UMaterialInterface* InMaterial)
{
	if (WaterMaterial != InMaterial)
	{
		WaterMaterial = InMaterial;
		UpdateMaterialInstances();
	}
}

void UWaterBodyComponent::SetWaterInfoMaterial(UMaterialInterface* InMaterial)
{
	if (WaterInfoMaterial != InMaterial)
	{
		WaterInfoMaterial = InMaterial;
		UpdateMaterialInstances();
	}
}

void UWaterBodyComponent::SetWaterStaticMeshMaterial(UMaterialInterface* InMaterial)
{
	if (WaterStaticMeshMaterial != InMaterial)
	{
		WaterStaticMeshMaterial = InMaterial;
		UpdateMaterialInstances();
	}
}

UMaterialInstanceDynamic* UWaterBodyComponent::GetWaterMaterialInstance()
{
	CreateOrUpdateWaterMID(); 
	return WaterMID;
}

UMaterialInstanceDynamic* UWaterBodyComponent::GetUnderwaterPostProcessMaterialInstance()
{
	CreateOrUpdateUnderwaterPostProcessMID(); 
	return UnderwaterPostProcessMID;
}

UMaterialInstanceDynamic* UWaterBodyComponent::GetWaterInfoMaterialInstance()
{
	CreateOrUpdateWaterInfoMID();
	return WaterInfoMID;
}

UMaterialInstanceDynamic* UWaterBodyComponent::GetWaterStaticMeshMaterialInstance()
{
	CreateOrUpdateWaterStaticMeshMID();
	return WaterStaticMeshMID;
}

void UWaterBodyComponent::SetUnderwaterPostProcessMaterial(UMaterialInterface* InMaterial)
{
	UnderwaterPostProcessMaterial = InMaterial;
	UpdateMaterialInstances();
}

void UWaterBodyComponent::SetWaterAndUnderWaterPostProcessMaterial(UMaterialInterface* InWaterMaterial, UMaterialInterface* InUnderWaterPostProcessMaterial)
{
	bool bUpdateInstances = WaterMaterial != InWaterMaterial || UnderwaterPostProcessMaterial != InUnderWaterPostProcessMaterial;

	WaterMaterial = InWaterMaterial;
	UnderwaterPostProcessMaterial = InUnderWaterPostProcessMaterial;

	if (bUpdateInstances)
	{
		UpdateMaterialInstances();
	}
}

bool UWaterBodyComponent::ShouldGenerateWaterMeshTile() const
{
	return bAlwaysGenerateWaterMeshTiles
		|| ((GetWaterBodyType() != EWaterBodyType::Transition)
		&& (GetWaterMeshOverride() == nullptr)
		&& (GetWaterMaterial() != nullptr));
}

void UWaterBodyComponent::AddIsland(AWaterBodyIsland* Island)
{
	WaterBodyIslands.AddUnique(Island);
}

void UWaterBodyComponent::RemoveIsland(AWaterBodyIsland* Island)
{
	WaterBodyIslands.RemoveSwap(Island);
}

void UWaterBodyComponent::UpdateIslands()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWaterBodyComponent::UpdateIslands);

	// For now, islands are not detected dynamically
#if WITH_EDITOR
	if (GetWorld())
	{
		WaterBodyIslands.Empty();
		for (AWaterBodyIsland* Island : TActorRange<AWaterBodyIsland>(GetWorld()))
		{
			Island->UpdateOverlappingWaterBodyComponents();
		}
	}
#endif // WITH_EDITOR
}

void UWaterBodyComponent::AddExclusionVolume(AWaterBodyExclusionVolume* InExclusionVolume)
{
	WaterBodyExclusionVolumes.AddUnique(InExclusionVolume);

	// Update only the collision meshes for the water body
	UpdateWaterBody(/*bWithExclusionVolumes=*/true);
}

void UWaterBodyComponent::RemoveExclusionVolume(AWaterBodyExclusionVolume* InExclusionVolume)
{
	WaterBodyExclusionVolumes.RemoveSwap(InExclusionVolume);

	// Update only the collision meshes for the water body
	UpdateWaterBody(/*bWithExclusionVolumes=*/true);
}

void UWaterBodyComponent::UpdateExclusionVolumes()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWaterBodyComponent::UpdateExclusionVolumes);
	if (GetWorld())
	{
		for (AWaterBodyExclusionVolume* ExclusionVolume : TActorRange<AWaterBodyExclusionVolume>(GetWorld()))
		{
			FWaterExclusionVolumeChangedParams Params;
			Params.bUserTriggered = false;
			ExclusionVolume->UpdateOverlappingWaterBodies(Params);
		}
	}
}

void UWaterBodyComponent::UpdateWaterZones(bool bAllowChangesDuringCook /* = false */)
{
	if (UWorld* World = GetWorld())
	{
		TSoftObjectPtr<AWaterZone> FoundZone = nullptr;
		if (!WaterZoneOverride.IsNull())
		{
			FoundZone = WaterZoneOverride.Get();
		}
		else
		{
			// Don't attempt to find a water zone while cooking and just rely on the serialized pointer from the editor.
			if (IsRunningCookCommandlet() && !bAllowChangesDuringCook)
			{
				return;
			}

			const FBox Bounds3D = CalcBounds(GetComponentToWorld()).GetBox();

			const AActor* ActorOwner = GetTypedOuter<AActor>();
			const ULevel* PreferredLevel = ActorOwner ? ActorOwner->GetLevel() : nullptr;
			FoundZone = UWaterSubsystem::FindWaterZone(World, FBox2D(FVector2D(Bounds3D.Min), FVector2D(Bounds3D.Max)), PreferredLevel);
		}

		if (OwningWaterZone != FoundZone)
		{
			if (AWaterZone* OldOwningZonePtr = OwningWaterZone.Get())
			{
				OldOwningZonePtr->RemoveWaterBodyComponent(this);
			}

			if (AWaterZone* FoundZonePtr = FoundZone.Get())
			{
				FoundZonePtr->AddWaterBodyComponent(this);
			}

			OwningWaterZone = FoundZone;
			
			UpdateMaterialInstances();
		}
	}
}

AWaterZone* UWaterBodyComponent::GetWaterZone() const
{
	return OwningWaterZone.Get();
}

void UWaterBodyComponent::MarkOwningWaterZoneForRebuild(EWaterZoneRebuildFlags InRebuildFlags, bool bInOnlyWithinWaterBodyBounds) const
{
	if (AWaterZone* WaterZone = GetWaterZone())
	{
		EWaterZoneRebuildFlags RebuildFlags = InRebuildFlags;

		// Avoid rebuilding things which this water body should not affect:
		if (!AffectsWaterInfo())
		{
			EnumRemoveFlags(RebuildFlags, EWaterZoneRebuildFlags::UpdateWaterInfoTexture);
		}
		if (!AffectsWaterMesh())
		{
			EnumRemoveFlags(RebuildFlags, EWaterZoneRebuildFlags::UpdateWaterMesh);
		}

		if (bInOnlyWithinWaterBodyBounds)
		{
			const FBox WaterBodyBounds = Bounds.GetBox();
			WaterZone->MarkForRebuild(RebuildFlags, FBox2D(FVector2D(WaterBodyBounds.Min), FVector2D(WaterBodyBounds.Max)), /* DebugRequestingObject = */ GetOwner());
		}
		else
		{
			WaterZone->MarkForRebuild(RebuildFlags, /* DebugRequestingObject = */ GetOwner());
		}
	}
}

void UWaterBodyComponent::SetWaterZoneOverride(const TSoftObjectPtr<AWaterZone>& InWaterZoneOverride)
{
	WaterZoneOverride = InWaterZoneOverride;
	UpdateWaterZones();
}

FPostProcessVolumeProperties UWaterBodyComponent::GetPostProcessProperties() const
{
	FPostProcessVolumeProperties Ret;
	Ret.bIsEnabled = UnderwaterPostProcessSettings.bEnabled;
	Ret.bIsUnbound = false;
	Ret.BlendRadius = UnderwaterPostProcessSettings.BlendRadius;
	Ret.BlendWeight = UnderwaterPostProcessSettings.BlendWeight;
	Ret.Priority = UnderwaterPostProcessSettings.Priority;
	Ret.Settings = &CurrentPostProcessSettings;
	Ret.Size = Bounds.GetBox().GetVolume();

	// We can give this a singleton Guid, because only one water body is activated at a time, depending on player location.
	Ret.VolumeGuid = FGuid(0xe0809956, 0xa2eb49f2, 0xb4f0aa6c, 0x143c89b1);
	return Ret;
}

void UWaterBodyComponent::PostDuplicate(bool bDuplicateForPie)
{
	Super::PostDuplicate(bDuplicateForPie);

#if WITH_EDITOR
	if (!bDuplicateForPie && GIsEditor)
	{
		// Sometimes during PostDuplicate the water body component can be created in isolation without an owner. In this case there won't be a water spline.
		if (UWaterSplineComponent* WaterSpline = GetWaterSpline())
		{
			// After duplication due to copy-pasting, UWaterSplineMetadata might have been edited without the spline component being made aware of that (for some reason, USplineComponent::PostDuplicate isn't called)::
			GetWaterSpline()->SynchronizeWaterProperties();
		}

		FOnWaterBodyChangedParams Params;
		Params.bShapeOrPositionChanged = true;
		Params.bWeightmapSettingsChanged = true;
		OnWaterBodyChanged(Params);
	}

	RegisterOnUpdateWavesData(GetWaterWaves(), /* bRegister = */true);
#endif // WITH_EDITOR
}

float UWaterBodyComponent::FindInputKeyClosestToWorldLocation(const FVector& WorldLocation) const
{
	const UWaterSplineComponent* const WaterSpline = GetWaterSpline();

	return WaterSpline ? WaterSpline->FindInputKeyClosestToWorldLocation(WorldLocation) : 0;
}

float UWaterBodyComponent::GetConstantSurfaceZ() const
{
	const UWaterSplineComponent* const WaterSpline = GetWaterSpline();

	// A single Z doesn't really make sense for non-flat water bodies, but it can be useful for when using FixedZ post process for example. Take the first spline key in that case :
	float WaterSurfaceZ = (IsFlatSurface() || WaterSpline == nullptr) ? GetComponentLocation().Z : WaterSpline->GetLocationAtSplineInputKey(0.0f, ESplineCoordinateSpace::World).Z;
	
	// Apply body height offset if applicable (ocean)
	if (IsHeightOffsetSupported())
	{
		WaterSurfaceZ += GetHeightOffset();
	}

	return WaterSurfaceZ;
}

float UWaterBodyComponent::GetConstantDepth() const
{
	// Only makes sense when you consider the water depth to be constant for the whole water body, in which case we just use the first spline key's : 
	const UWaterSplineComponent* const WaterSpline = GetWaterSpline();
	float SplinePointDepth = WaterSpline ? WaterSpline->GetFloatPropertyAtSplineInputKey(0.0f, GET_MEMBER_NAME_CHECKED(UWaterSplineMetadata, Depth)) : 0.0f;

	// Ensure that the existing spline depth is used if it is non-zero, otherwise use the new FixedWaterDepth parameter.
	return SplinePointDepth != 0.f ? SplinePointDepth : FixedWaterDepth;
}

FVector UWaterBodyComponent::GetConstantVelocity() const
{
	// Only makes sense when you consider the water velocity to be constant for the whole water body, in which case we just use the first spline key's : 
	return GetWaterVelocityVectorAtSplineInputKey(0.0f);
}

void UWaterBodyComponent::GetSurfaceMinMaxZ(float& OutMinZ, float& OutMaxZ) const
{
	const float SurfaceZ = GetConstantSurfaceZ();
	const float MaxWaveHeight = GetMaxWaveHeight();
	OutMaxZ = SurfaceZ + MaxWaveHeight;
	OutMinZ = SurfaceZ - MaxWaveHeight;
}

EWaterBodyQueryFlags UWaterBodyComponent::CheckAndAjustQueryFlags(EWaterBodyQueryFlags InQueryFlags) const
{
	EWaterBodyQueryFlags Result = InQueryFlags;

	// Waves only make sense for the following queries : 
	check(!EnumHasAnyFlags(Result, EWaterBodyQueryFlags::IncludeWaves)
		|| EnumHasAnyFlags(Result, EWaterBodyQueryFlags::ComputeLocation | EWaterBodyQueryFlags::ComputeNormal | EWaterBodyQueryFlags::ComputeDepth | EWaterBodyQueryFlags::ComputeImmersionDepth));

	// Simple waves only make sense when computing waves : 
	check(!EnumHasAnyFlags(Result, EWaterBodyQueryFlags::SimpleWaves)
		|| EnumHasAnyFlags(Result, EWaterBodyQueryFlags::IncludeWaves));

	if (EnumHasAnyFlags(InQueryFlags, EWaterBodyQueryFlags::ComputeDepth | EWaterBodyQueryFlags::ComputeImmersionDepth))
	{
		// We need location when querying depth : 
		Result |= EWaterBodyQueryFlags::ComputeLocation;
	}

	if (EnumHasAnyFlags(InQueryFlags, EWaterBodyQueryFlags::IncludeWaves) && HasWaves())
	{
		// We need location and water depth when computing waves :
		Result |= EWaterBodyQueryFlags::ComputeLocation | EWaterBodyQueryFlags::ComputeDepth;
	}

	return Result;
}

bool UWaterBodyComponent::IsWorldLocationInExclusionVolume(const FVector& InWorldLocation) const
{
	for (const TSoftObjectPtr<AWaterBodyExclusionVolume>& ExclusionVolume : WaterBodyExclusionVolumes)
	{
		if (ExclusionVolume.IsValid() && ExclusionVolume->EncompassesPoint(InWorldLocation))
		{
			return true;
		}
	}

	return false;
}

TValueOrError<FWaterBodyQueryResult, EWaterBodyQueryError> UWaterBodyComponent::TryQueryWaterInfoClosestToWorldLocation(const FVector& InWorldLocation, EWaterBodyQueryFlags InQueryFlags, const TOptional<float>& InSplineInputKey) const
{
	if (!IsRegistered())
	{
		return MakeError(EWaterBodyQueryError::WaterBodyNotRegistered);
	}

	if (WaterSplineMetadata == nullptr)
	{
		return MakeError(EWaterBodyQueryError::NullWaterSplineMetadata);
	}

	// TryQuery is implemented in such a way that it calls into the old virtual function so existing implementations
	// are used until the old impl is removed and we can move the implementation into TryQuery.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return MakeValue(QueryWaterInfoClosestToWorldLocation(InWorldLocation, InQueryFlags, InSplineInputKey));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FWaterBodyQueryResult UWaterBodyComponent::QueryWaterInfoClosestToWorldLocation(const FVector& InWorldLocation, EWaterBodyQueryFlags InQueryFlags, const TOptional<float>& InSplineInputKey) const
{
	SCOPE_CYCLE_COUNTER(STAT_WaterBody_ComputeWaterInfo);

	// Use the (optional) input spline input key if it has already been computed: 
	FWaterBodyQueryResult Result(InSplineInputKey);
	Result.SetQueryFlags(CheckAndAjustQueryFlags(InQueryFlags));

	if (!EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::IgnoreExclusionVolumes))
	{
		// No early-out, so that the requested information is still set. It is expected for the caller to check for IsInExclusionVolume() because technically, the returned information will be invalid :
		Result.SetIsInExclusionVolume(IsWorldLocationInExclusionVolume(InWorldLocation));
	}

	// Lakes and oceans have surfaces aligned with the XY plane
	const bool bFlatSurface = IsFlatSurface();
	const UWaterSplineComponent* const WaterSpline = GetWaterSpline();

	FVector ShallowWaterVelocity = { 0, 0, 0 };
	float ShallowWaterHeight = 0;
	float ShallowWaterDepth = 0;
	const bool bUseShallowWaterValues = UseBakedSimulationForQueriesAndPhysics();

	if (bUseShallowWaterValues)
	{		
		BakedShallowWaterSim->BakedSimulationData->SampleShallowWaterSimulationAtPosition(InWorldLocation,
			ShallowWaterVelocity, ShallowWaterHeight, ShallowWaterDepth);
	}

	// Compute water plane location :
	if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeLocation))
	{
		SCOPE_CYCLE_COUNTER(STAT_WaterBody_ComputeLocation);
		FVector WaterPlaneLocation = InWorldLocation;
		// If in exclusion volume, force the water plane location at the query location. It is technically invalid, but it's up to the caller to check whether we're in an exclusion volume. 
		//  If the user fails to do so, at least it allows immersion depth to be 0.0f, which means the query location is NOT in water :
		if (!Result.IsInExclusionVolume())
		{
			float WaterPlaneHeight;

			if (bFlatSurface || WaterSpline == nullptr)
			{
				WaterPlaneHeight = GetComponentLocation().Z;
			}
			else if (bUseShallowWaterValues)
			{
				WaterPlaneHeight = ShallowWaterHeight;
			}
			else
			{
				WaterPlaneHeight = WaterSpline->GetLocationAtSplineInputKey(Result.LazilyComputeSplineKey(*this, InWorldLocation), ESplineCoordinateSpace::World).Z;
			}

			WaterPlaneLocation.Z = WaterPlaneHeight;

			// Apply body height offset if applicable (ocean)
			if (IsHeightOffsetSupported())
			{
				WaterPlaneLocation.Z += GetHeightOffset();
			}
		}

		Result.SetWaterPlaneLocation(WaterPlaneLocation);
		// When not including waves, water surface == water plane : 
		Result.SetWaterSurfaceLocation(WaterPlaneLocation);
	}

	// Compute water plane normal :
	FVector WaterPlaneNormal = FVector::UpVector;
	if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeNormal))
	{
		SCOPE_CYCLE_COUNTER(STAT_WaterBody_ComputeNormal);
		// Default to Z up for the normal
		if (!bFlatSurface && WaterSpline != nullptr)
		{
			if (bUseShallowWaterValues)
			{
				WaterPlaneNormal = BakedShallowWaterSim->BakedSimulationData->ComputeShallowWaterSimulationNormalAtPosition(InWorldLocation);
			}
			else
			{
				// For rivers default to using spline up vector to account for sloping rivers
				WaterPlaneNormal = WaterSpline->GetUpVectorAtSplineInputKey(Result.LazilyComputeSplineKey(*this, InWorldLocation), ESplineCoordinateSpace::World);
			}			
		}

		Result.SetWaterPlaneNormal(WaterPlaneNormal);
		// When not including waves, water surface == water plane : 
		Result.SetWaterSurfaceNormal(WaterPlaneNormal);
	}

	// Compute water plane depth : 
	float WaveAttenuationFactor = 1.0f;
	if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeDepth))
	{
		SCOPE_CYCLE_COUNTER(STAT_WaterBody_ComputeWaterDepth);

		check(EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeLocation));
		float WaterPlaneDepth = 0.0f;

		// The better option for computing water depth for ocean and lake is landscape : 
		const bool bTryUseLandscape = (GetWaterBodyType() == EWaterBodyType::Ocean || GetWaterBodyType() == EWaterBodyType::Lake);

		if (bUseShallowWaterValues)
		{
			WaterPlaneDepth = ShallowWaterDepth;
		}
		else if (bTryUseLandscape)
		{
			TOptional<float> LandscapeHeightOptional;
			if (ALandscapeProxy* LandscapePtr = FindLandscape())
			{
				SCOPE_CYCLE_COUNTER(STAT_WaterBody_ComputeLandscapeDepth);
				LandscapeHeightOptional = LandscapePtr->GetHeightAtLocation(InWorldLocation);
			}

			bool bValidLandscapeData = LandscapeHeightOptional.IsSet();
			if (bValidLandscapeData)
			{
				WaterPlaneDepth = Result.GetWaterPlaneLocation().Z - LandscapeHeightOptional.GetValue();
				// Special case : cancel out waves for under-landscape ocean
				if ((WaterPlaneDepth < 0.0f) && (GetWaterBodyType() == EWaterBodyType::Ocean))
				{
					WaveAttenuationFactor = 0.0f;
				}
			}

			// If the height is invalid, we either have invalid landscape data or we're under the 
			if (!bValidLandscapeData || (WaterPlaneDepth < 0.0f))
			{
				if (GetWaterBodyType() == EWaterBodyType::Ocean)
				{
					// Fallback value when landscape is not found under the ocean water.
					WaterPlaneDepth = CVarWaterOceanFallbackDepth.GetValueOnAnyThread();
				}
				else
				{
					check(GetWaterBodyType() == EWaterBodyType::Lake);
					// For an underwater lake, consider an uniform depth across the projection segment on the lake spline :
					WaterPlaneDepth = WaterSplineMetadata->Depth.Eval(Result.LazilyComputeSplineKey(*this, InWorldLocation), 0.f);
				}
			}
		}
		else
		{
			// For rivers and transitions, depth always come from the spline :
			WaterPlaneDepth = WaterSplineMetadata->Depth.Eval(Result.LazilyComputeSplineKey(*this, InWorldLocation), 0.f);
		}

		WaterPlaneDepth = FMath::Max(WaterPlaneDepth, 0.0f);
		Result.SetWaterPlaneDepth(WaterPlaneDepth);

		// When not including waves, water surface == water plane : 
		Result.SetWaterSurfaceDepth(WaterPlaneDepth);
	}

	// Optionally compute water surface location/normal/depth for waves : 
	if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::IncludeWaves) && HasWaves())
	{
		SCOPE_CYCLE_COUNTER(STAT_WaterBody_ComputeWaveHeight);
		FWaveInfo WaveInfo;

		if (!Result.IsInExclusionVolume())
		{
			WaveInfo.AttenuationFactor = WaveAttenuationFactor;
			WaveInfo.Normal = WaterPlaneNormal;
			const bool bSimpleWaves = EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::SimpleWaves);
			GetWaveInfoAtPosition(Result.GetWaterPlaneLocation(), Result.GetWaterSurfaceDepth(), bSimpleWaves, WaveInfo);
		}

		Result.SetWaveInfo(WaveInfo);

		if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeLocation))
		{
			FVector WaterSurfaceLocation = Result.GetWaterSurfaceLocation();
			WaterSurfaceLocation.Z += WaveInfo.Height;
			Result.SetWaterSurfaceLocation(WaterSurfaceLocation);
		}

		if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeNormal))
		{
			Result.SetWaterSurfaceNormal(WaveInfo.Normal);
		}

		if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeDepth))
		{
			Result.SetWaterSurfaceDepth(Result.GetWaterSurfaceDepth() + WaveInfo.Height);
		}
	}

	if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeImmersionDepth))
	{
		check(EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeLocation));

		// Immersion depth indicates how much under the water surface is the world location. 
		//  therefore, it takes into account the waves if IncludeWaves is passed :
		Result.SetImmersionDepth(Result.GetWaterSurfaceLocation().Z - InWorldLocation.Z);
		// When in an exclusion volume, the queried location is considered out of water (immersion depth == 0.0f)
		check(!Result.IsInExclusionVolume() || (Result.GetImmersionDepth() == 0.0f));
	}

	// Compute velocity : 
	if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeVelocity))
	{
		FVector Velocity = FVector::ZeroVector;
		if (!Result.IsInExclusionVolume())
		{
			if (bUseShallowWaterValues)
			{
				Velocity = ShallowWaterVelocity;
			}
			else
			{
				Velocity = GetWaterVelocityVectorAtSplineInputKey(Result.LazilyComputeSplineKey(*this, InWorldLocation));
			}
		}

		Result.SetVelocity(Velocity);
	}

	return Result;

}

bool UWaterBodyComponent::GetWaterSurfaceInfoAtLocation(const FVector& InLocation, FVector& OutWaterSurfaceLocation, FVector& OutWaterSurfaceNormal, FVector& OutWaterVelocity, float& OutWaterDepth, bool bIncludeDepth /* = false */) const
{
	EWaterBodyQueryFlags QueryFlags =
		EWaterBodyQueryFlags::ComputeLocation
		| EWaterBodyQueryFlags::ComputeNormal
		| EWaterBodyQueryFlags::ComputeVelocity;

	if (bIncludeDepth)
	{
		QueryFlags |= EWaterBodyQueryFlags::ComputeDepth;
	}

	const TValueOrError<FWaterBodyQueryResult, EWaterBodyQueryError> QueryResult = TryQueryWaterInfoClosestToWorldLocation(InLocation, QueryFlags);
	if (!QueryResult.HasValue())
	{
		return false;
	}

	const FWaterBodyQueryResult& Query = QueryResult.GetValue();
	OutWaterSurfaceLocation = Query.GetWaterSurfaceLocation();
	OutWaterSurfaceNormal = Query.GetWaterSurfaceNormal();
	OutWaterVelocity = Query.GetVelocity();

	if (bIncludeDepth)
	{
		OutWaterDepth = Query.GetWaterSurfaceDepth();
	}

	return true;
}

float UWaterBodyComponent::GetWaterVelocityAtSplineInputKey(float InKey) const
{
	return WaterSplineMetadata ? WaterSplineMetadata->WaterVelocityScalar.Eval(InKey, 0.f) : 0.0f;
}

FVector UWaterBodyComponent::GetWaterVelocityVectorAtSplineInputKey(float InKey) const
{
	UWaterSplineComponent* WaterSpline = GetWaterSpline();
	const float WaterVelocityScalar = GetWaterVelocityAtSplineInputKey(InKey);
	const FVector SplineDirection = WaterSpline ? WaterSpline->GetDirectionAtSplineInputKey(InKey, ESplineCoordinateSpace::World) : FVector::ZeroVector;
	return SplineDirection * WaterVelocityScalar;
}

float UWaterBodyComponent::GetAudioIntensityAtSplineInputKey(float InKey) const
{
	return WaterSplineMetadata ? WaterSplineMetadata->AudioIntensity.Eval(InKey, 0.f) : 0.0f;
}

void UWaterBodyComponent::SetWaterVelocityAtSplineInputKey(float InKey, float InVelocity)
{
	if (WaterSplineMetadata)
	{
		const int32 PointIndexForKey = WaterSplineMetadata->WaterVelocityScalar.GetPointIndexForInputValue(InKey);
		WaterSplineMetadata->WaterVelocityScalar.Points[PointIndexForKey].OutVal = InVelocity;
	}
}

void UWaterBodyComponent::SetAudioIntensityAtSplineInputKey(float InKey, float InAudioIntensity)
{
	if (WaterSplineMetadata)
	{
		const int32 PointIndexForKey = WaterSplineMetadata->AudioIntensity.GetPointIndexForInputValue(InKey);
		WaterSplineMetadata->AudioIntensity.Points[PointIndexForKey].OutVal = InAudioIntensity;
	}
}

void UWaterBodyComponent::OnRegister()
{
#if WITH_EDITORONLY_DATA
	// Prevents USceneComponent from creating the SpriteComponent in OnRegister because we want to provide a different texture
	bVisualizeComponent = false;
#endif // WITH_EDITORONLY_DATA

	Super::OnRegister();

	AWaterBody* OwningWaterBodyActor = GetWaterBodyActor();
	if (OwningWaterBodyActor == nullptr)
	{
		return;
	}

	WaterSplineMetadata = OwningWaterBodyActor->GetWaterSplineMetadata();

	check(WaterSplineMetadata);

#if WITH_EDITOR
	RegisterOnChangeWaterSplineData(/*bRegister = */true); 

	CreateWaterSpriteComponent();
#endif // WITH_EDITOR

	if (AWaterZone* WaterZone = GetWaterZone())
	{
		WaterZone->AddWaterBodyComponent(this);
	}
}

void UWaterBodyComponent::OnUnregister()
{
#if WITH_EDITOR
	RegisterOnChangeWaterSplineData(/*bRegister = */false);
#endif // WITH_EDITOR

	if (AWaterZone* WaterZone = GetWaterZone())
	{
		WaterZone->RemoveWaterBodyComponent(this);
	}

	Super::OnUnregister();
}

TArray<AWaterBodyIsland*> UWaterBodyComponent::GetIslands() const
{
	TArray<AWaterBodyIsland*> IslandActors;
	IslandActors.Reserve(WaterBodyIslands.Num());

	for (const TSoftObjectPtr<AWaterBodyIsland>& IslandPtr : WaterBodyIslands)
	{
		if (AWaterBodyIsland* Island = IslandPtr.Get())
		{
			IslandActors.Add(Island);
		}
	}

	return IslandActors;
}

TArray<AWaterBodyExclusionVolume*> UWaterBodyComponent::GetExclusionVolumes() const
{
	TArray<AWaterBodyExclusionVolume*> Result;
	Result.Reserve(WaterBodyExclusionVolumes.Num());

	for (const TSoftObjectPtr<AWaterBodyExclusionVolume>& VolumePtr : WaterBodyExclusionVolumes)
	{
		if (AWaterBodyExclusionVolume* Volume = VolumePtr.Get())
		{
			Result.Add(Volume);
		}
	}

	return Result;
}

// Our transient MIDs are per-object and shall not survive duplicating nor be exported to text when copy-pasting : 
EObjectFlags UWaterBodyComponent::GetTransientMIDFlags() const
{
	return RF_Transient | RF_NonPIEDuplicateTransient | RF_TextExportTransient;
}

void UWaterBodyComponent::UpdateMaterialInstances()
{
	CreateOrUpdateWaterMID();
	CreateOrUpdateWaterInfoMID();
	CreateOrUpdateWaterStaticMeshMID();
	CreateOrUpdateUnderwaterPostProcessMID();

	// Update the water mesh since it will not contain the up-to-date MIDs:
	MarkOwningWaterZoneForRebuild(EWaterZoneRebuildFlags::UpdateWaterMesh);
}

bool UWaterBodyComponent::UpdateWaterHeight()
{
	bool bWaterBodyChanged = false;
	const AActor* Owner = GetOwner();
	USplineComponent* WaterSpline = GetWaterSpline();
	if (IsFlatSurface() && WaterSpline && Owner)
	{
		const int32 NumSplinePoints = WaterSpline->GetNumberOfSplinePoints();

		const float ActorZ = Owner->GetActorLocation().Z;

		for (int32 PointIndex = 0; PointIndex < NumSplinePoints; ++PointIndex)
		{
			FVector WorldLoc = WaterSpline->GetLocationAtSplinePoint(PointIndex, ESplineCoordinateSpace::World);

			if (WorldLoc.Z != ActorZ)
			{
				bWaterBodyChanged = true;
				WorldLoc.Z = ActorZ;
				WaterSpline->SetLocationAtSplinePoint(PointIndex, WorldLoc, ESplineCoordinateSpace::World);
			}
		}
	}

	return bWaterBodyChanged;
}

void UWaterBodyComponent::CreateOrUpdateWaterMID()
{
	// If GetWorld fails we may be in a blueprint
	if (GetWorld())
	{
		WaterMID = FWaterUtils::GetOrCreateTransientMID(WaterMID, TEXT("WaterMID"), WaterMaterial, GetTransientMIDFlags());

		SetDynamicParametersOnMID(WaterMID);
	}
}

void UWaterBodyComponent::CreateOrUpdateUnderwaterPostProcessMID()
{
	// If GetWorld fails we may be in a blueprint
	if (GetWorld())
	{
		UnderwaterPostProcessMID = FWaterUtils::GetOrCreateTransientMID(UnderwaterPostProcessMID, TEXT("UnderwaterPostProcessMID"), UnderwaterPostProcessMaterial, GetTransientMIDFlags());

		SetDynamicParametersOnUnderwaterPostProcessMID(UnderwaterPostProcessMID);

		// update the transient post process settings accordingly : 
		PrepareCurrentPostProcessSettings();
	}
}

void UWaterBodyComponent::CreateOrUpdateWaterInfoMID()
{
	if (GetWorld())
	{
		WaterInfoMID = FWaterUtils::GetOrCreateTransientMID(WaterInfoMID, TEXT("WaterInfoMID"), WaterInfoMaterial, GetTransientMIDFlags());
		if (UWaterBodyInfoMeshComponent* WaterInfoMesh = GetWaterInfoMeshComponent())
		{
			WaterInfoMesh->SetMaterial(0, WaterInfoMID);
		}
		if (UWaterBodyInfoMeshComponent* DilatedWaterMesh = GetDilatedWaterInfoMeshComponent())
		{
			DilatedWaterMesh->SetMaterial(0, WaterInfoMID);
		}

		SetDynamicParametersOnWaterInfoMID(WaterInfoMID);
	}
}

void UWaterBodyComponent::CreateOrUpdateWaterStaticMeshMID()
{
	if (GetWorld())
	{
		WaterStaticMeshMID = FWaterUtils::GetOrCreateTransientMID(WaterStaticMeshMID, TEXT("WaterLODMID"), WaterStaticMeshMaterial, GetTransientMIDFlags());
		if (AWaterBody* WaterBodyActor = GetWaterBodyActor())
		{
			for (UWaterBodyStaticMeshComponent* WaterBodyStaticMeshComponent : WaterBodyActor->GetWaterBodyStaticMeshComponents())
			{
				if (IsValid(WaterBodyStaticMeshComponent))
				{
					WaterBodyStaticMeshComponent->SetMaterial(0, WaterStaticMeshMID);
				}
			}
		}

		SetDynamicParametersOnMID(WaterStaticMeshMID);
	}
}

void UWaterBodyComponent::PrepareCurrentPostProcessSettings()
{
	// Prepare the transient settings that are actually used by the post-process system : 
	// - Copy all the non-transient settings :
	CurrentPostProcessSettings = UnderwaterPostProcessSettings.PostProcessSettings;

	// - Control the WeightedBlendables with the transient underwater post process MID : 
	if (UnderwaterPostProcessMID != nullptr)
	{
		if (CurrentPostProcessSettings.WeightedBlendables.Array.Num() == 0)
		{
			CurrentPostProcessSettings.WeightedBlendables.Array.Emplace();
		}
		FWeightedBlendable& Blendable = CurrentPostProcessSettings.WeightedBlendables.Array[0];
		Blendable.Object = UnderwaterPostProcessMID;
		Blendable.Weight = 1.0f;
	}
	else
	{
		CurrentPostProcessSettings.WeightedBlendables.Array.Empty();
	}
}

ALandscapeProxy* UWaterBodyComponent::FindLandscape() const
{
	if (!Landscape.IsValid())
	{
		const FBox ComponentBounds = Bounds.GetBox();
		for (TObjectIterator<ALandscapeProxy> It; It; ++It)
		{
			if (It->GetWorld() == GetWorld())
			{
				const FBox Box = It->GetComponentsBoundingBox();
				if (Box.IntersectXY(ComponentBounds))
				{
					Landscape = *It;
					return Landscape.Get();
				}
			}
		}
	}
	return Landscape.Get();
}

void UWaterBodyComponent::UpdateVisibility()
{
	if (UWorld* World = GetWorld())
	{
	 	const bool bIsWaterRenderingEnabled = FWaterUtils::IsWaterEnabled(/*bIsRenderThread = */false);
	 
		bool bIsRenderedByWaterMesh = ShouldGenerateWaterMeshTile();

		// Handle the standard renderable components (i.e. when the water body is not rendered by the water mesh) : 
		{
			bool bLocalVisible = bIsWaterRenderingEnabled && !bIsRenderedByWaterMesh && GetVisibleFlag();
			bool bLocalHiddenInGame = !bIsWaterRenderingEnabled || bIsRenderedByWaterMesh || bHiddenInGame;

			for (UPrimitiveComponent* Component : GetStandardRenderableComponents())
			{
				Component->SetVisibility(bLocalVisible);
				Component->SetHiddenInGame(bLocalHiddenInGame);
			}
		}

		// Keep track of the visibility state when the water body is being rendered by the water mesh, to avoid calling MarkOwningWaterZoneForRebuild every time UpdateVisibility is called :
		const bool bPreviousIsRenderedByWaterMeshAndVisible = bIsRenderedByWaterMeshAndVisible;
		bIsRenderedByWaterMeshAndVisible = bIsRenderedByWaterMesh && ShouldRender();
		const bool bHasVisibilityChanged = (bIsRenderedByWaterMeshAndVisible != bPreviousIsRenderedByWaterMeshAndVisible);

		if (bHasVisibilityChanged)
		{
			// If the component is being or can be rendered by the water mesh or renders into the water info texture, rebuild it in case its visibility has changed : 
			MarkOwningWaterZoneForRebuild(EWaterZoneRebuildFlags::All);
		}
	}
}

#if WITH_EDITOR
void UWaterBodyComponent::PreEditUndo()
{
	Super::PreEditUndo();

	// On undo, when PreEditChange is called, PropertyAboutToChange is nullptr so we need to unregister from the previous object here :
	RegisterOnUpdateWavesData(GetWaterWaves(), /*bRegister = */false);
}

void UWaterBodyComponent::PostEditUndo()
{
	if (AWaterBody* WaterBodyActor = GetWaterBodyActor())
	{
		WaterBodyActor->CleanupInvalidStaticMeshComponents();
	}

	Super::PostEditUndo();

	// Since this component may become unregistered/deleted if we are undoing the creation of a water body.
	// Ensure we only trigger updates if this component is registered.
	if (IsRegistered())
	{
		FOnWaterBodyChangedParams Params;
		Params.bShapeOrPositionChanged = true;
		Params.bWeightmapSettingsChanged = true;
		OnWaterBodyChanged(Params);

		// On undo, when PostEditChangeProperty is called, PropertyChangedEvent is fake so we need to register to the new object here :
		RegisterOnUpdateWavesData(GetWaterWaves(), /*bRegister = */true);

		RequestGPUWaveDataUpdate();
	}
}

void UWaterBodyComponent::PostEditImport()
{
	Super::PostEditImport();

	FOnWaterBodyChangedParams Params;
	Params.bShapeOrPositionChanged = true;
	Params.bWeightmapSettingsChanged = true;
	Params.bUserTriggered = true;
	OnWaterBodyChanged(Params);

	RequestGPUWaveDataUpdate();
}

void UWaterBodyComponent::OnPostEditChangeProperty(FOnWaterBodyChangedParams& InOutOnWaterBodyChangedParams)
{
	const FPropertyChangedEvent& PropertyChangedEvent = InOutOnWaterBodyChangedParams.PropertyChangedEvent;
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UWaterBodyComponent, LayerWeightmapSettings))
	{
		InOutOnWaterBodyChangedParams.bWeightmapSettingsChanged = true;
	}
	else if ((PropertyName == GET_MEMBER_NAME_CHECKED(UWaterBodyComponent, WaterMaterial)) ||
			(PropertyName == GET_MEMBER_NAME_CHECKED(UWaterBodyComponent, UnderwaterPostProcessMaterial)) ||
			(PropertyName == GET_MEMBER_NAME_CHECKED(UWaterBodyComponent, WaterInfoMaterial)))
	{
		UpdateMaterialInstances();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaterBodyComponent, WaterStaticMeshMaterial))
	{
		UpdateMaterialInstances();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaterBodyComponent, TargetWaveMaskDepth))
	{
		RequestGPUWaveDataUpdate();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaterBodyComponent, MaxWaveHeightOffset))
	{
		// Waves data affect the navigation :
		InOutOnWaterBodyChangedParams.bShapeOrPositionChanged = true;
	}
	else if (PropertyChangedEvent.MemberProperty && (PropertyChangedEvent.MemberProperty->GetFName() == FName(TEXT("RelativeScale3D"))
												|| PropertyChangedEvent.MemberProperty->GetFName() == FName(TEXT("RelativeRotation"))))
	{
		FixupEditorTransform();
		InOutOnWaterBodyChangedParams.bShapeOrPositionChanged = true;
	}
	else if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UWaterBodyComponent, StaticMeshSettings))
	{
		UpdateWaterBodyStaticMeshComponents();
	}
	else if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UWaterBodyComponent, WaterZoneOverride))
	{
		UpdateWaterZones();
	}
}

TArray<TSharedRef<FTokenizedMessage>> UWaterBodyComponent::CheckWaterBodyStatus()
{
	TArray<TSharedRef<FTokenizedMessage>> Result;

	const UWorld* World = GetWorld();
	if (!IsTemplate() && World && World->WorldType != EWorldType::EditorPreview)
	{
		if (GetWaterBodyActor() == nullptr)
		{
			Result.Add(FTokenizedMessage::Create(EMessageSeverity::Error)
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(FText::Format(
					LOCTEXT("MapCheck_Message_MissingWaterBodyActor", "WaterBodyComponent is attached to an actor which is not an AWaterBody ({0})! WaterBodyComponents required parent water body actors to function correctly"),
					FText::FromString(GetOwner() ? GetOwner()->GetActorLabel() : TEXT(""))))));
		}
		else
		{
			if (AffectsWaterMesh())
			{
				AWaterZone* WaterZone = GetWaterZone();
				if (WaterZone == nullptr)
				{
					if (!StaticMeshSettings.bEnableWaterBodyStaticMesh)
					{
						Result.Add(FTokenizedMessage::Create(EMessageSeverity::Error)
							->AddToken(FUObjectToken::Create(this))
							->AddToken(FTextToken::Create(FText::Format(
								LOCTEXT("MapCheck_Message_MissingWaterZone", "Water body {0} without a static mesh fallback requires a WaterZone actor to be rendered. Please add one to the map or enable the static mesh fallback. "),
								FText::FromString(GetWaterBodyActor()->GetActorLabel())))));
					}
				}
				else
				{
					if (WaterZone->IsLocalOnlyTessellationEnabled() && !StaticMeshSettings.bEnableWaterBodyStaticMesh)
					{
						Result.Add(FTokenizedMessage::Create(EMessageSeverity::Warning)
							->AddToken(FUObjectToken::Create(this))
							->AddToken(FTextToken::Create(FText::Format(
								LOCTEXT("MapCheck_Message_LocalTessellationWithoutStaticMesh", "Water body {0} is rendered by a water zone with local tessellation enabled but does not have its own static mesh enabled to fallback on!"),
								FText::FromString(GetWaterBodyActor()->GetActorLabel()))))
							->AddToken(FActionToken::Create(LOCTEXT("MapCheck_MessageAction_EnableWaterBodyStaticMesh", "Click here to enable the static mesh"), FText(),
								FOnActionTokenExecuted::CreateUObject(this, &UWaterBodyComponent::SetWaterBodyStaticMeshEnabled, true), true))
								);
					}
					
					if (GetWaterInfoMeshComponent() == nullptr || GetWaterInfoMeshComponent()->GetStaticMesh() == nullptr)
					{
						Result.Add(FTokenizedMessage::Create(EMessageSeverity::Warning)
							->AddToken(FUObjectToken::Create(this))
							->AddToken(FTextToken::Create(FText::Format(
								LOCTEXT("MapCheck_Message_MissingWaterBodyMesh", "Water body {0} has no mesh information. This is caused by the water spline forming invalid geometry or not having enough spline points to be a valid shape. Adjust the water spline to resolve this issue."),
								FText::FromString(GetWaterBodyActor()->GetActorLabel()))))
							);
					}
				}
			}

			if (AffectsLandscape() && (FindLandscape() == nullptr))
			{
				Result.Add(FTokenizedMessage::Create(EMessageSeverity::Error)
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(FText::Format(
						LOCTEXT("MapCheck_Message_MissingLandscape", "Water body {0} requires a Landscape to be rendered. Please add one to the map. "),
						FText::FromString(GetWaterBodyActor()->GetActorLabel())))));
			}
		}
	}
	return Result;
}

void UWaterBodyComponent::CheckForErrors()
{
	Super::CheckForErrors();

	TArray<TSharedRef<FTokenizedMessage>> StatusMessages = CheckWaterBodyStatus();
	for (const TSharedRef<FTokenizedMessage>& StatusMessage : StatusMessages)
	{
		FMessageLog("MapCheck").AddMessage(StatusMessage);
	}
}

void UWaterBodyComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FOnWaterBodyChangedParams Params(PropertyChangedEvent);
	Params.bUserTriggered = true;
	OnPostEditChangeProperty(Params);
	
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (!IsTemplate())
	{
		OnWaterBodyChanged(Params);
	}
}

void UWaterBodyComponent::OnWaterSplineDataChanged(const FOnWaterSplineDataChangedParams& InParams)
{
	// Transfer the FOnWaterSplineDataChangedParams parameters to FOnWaterBodyChangedParams :
	FOnWaterBodyChangedParams Params(InParams.PropertyChangedEvent);
	Params.bShapeOrPositionChanged = true;
	Params.bUserTriggered = InParams.bUserTriggered;
	OnWaterBodyChanged(Params);
}

void UWaterBodyComponent::RegisterOnUpdateWavesData(UWaterWavesBase* InWaterWaves, bool bRegister)
{
	if (InWaterWaves != nullptr)
	{
		if (bRegister)
		{
			InWaterWaves->OnUpdateWavesData.AddUObject(this, &UWaterBodyComponent::OnWavesDataUpdated);
		}
		else
		{
			InWaterWaves->OnUpdateWavesData.RemoveAll(this);
		}
	}
}

void UWaterBodyComponent::OnWavesDataUpdated(UWaterWavesBase* InWaterWaves, EPropertyChangeType::Type InChangeType)
{
	RequestGPUWaveDataUpdate();

	FOnWaterBodyChangedParams Params;
	// Waves data affect the navigation :
	Params.PropertyChangedEvent.ChangeType = InChangeType;
	Params.bShapeOrPositionChanged = true;
	Params.bUserTriggered = true;
	OnWaterBodyChanged(Params);
}

void UWaterBodyComponent::OnWaterSplineMetadataChanged(const FOnWaterSplineMetadataChangedParams& InParams)
{
	bool bShapeOrPositionChanged = false;

	FName ChangedProperty = InParams.PropertyChangedEvent.GetPropertyName();
	if ((ChangedProperty == NAME_None)
		|| (ChangedProperty == GET_MEMBER_NAME_CHECKED(UWaterSplineMetadata, Depth))
		|| (ChangedProperty == GET_MEMBER_NAME_CHECKED(UWaterSplineMetadata, RiverWidth))
		|| (ChangedProperty == GET_MEMBER_NAME_CHECKED(UWaterSplineMetadata, WaterVelocityScalar)))
	{
		// Those changes require an update of the water brush (except in interactive mode, where we only apply the change once the value is actually set): 
		bShapeOrPositionChanged = true;
	}

	if ((ChangedProperty == NAME_None)
		|| (ChangedProperty == GET_MEMBER_NAME_CHECKED(UWaterSplineMetadata, RiverWidth)))
	{ 
		// River Width is driving the spline shape, make sure the spline component is aware of the change : 
		GetWaterSpline()->SynchronizeWaterProperties();
	}

	// Waves data affect the navigation :
	FOnWaterBodyChangedParams Params;
	Params.bShapeOrPositionChanged = bShapeOrPositionChanged;
	Params.bWeightmapSettingsChanged = false;
	Params.bUserTriggered = InParams.bUserTriggered;
	OnWaterBodyChanged(Params); 
}

void UWaterBodyComponent::RegisterOnChangeWaterSplineData(bool bRegister)
{
	if (UWaterSplineComponent* WaterSpline = GetWaterSpline())
	{
		if (bRegister)
		{
			WaterSpline->OnWaterSplineDataChanged().AddUObject(this, &UWaterBodyComponent::OnWaterSplineDataChanged);
		}
		else
		{
			WaterSpline->OnWaterSplineDataChanged().RemoveAll(this);
		}
	}

	if (WaterSplineMetadata != nullptr)
	{
		if (bRegister)
		{
			WaterSplineMetadata->OnChangeMetadata.AddUObject(this, &UWaterBodyComponent::OnWaterSplineMetadataChanged);
		}
		else
		{
			WaterSplineMetadata->OnChangeMetadata.RemoveAll(this);
		}
	}
}

#endif // WITH_EDITOR

void UWaterBodyComponent::GetNavigationData(struct FNavigationRelevantData& Data) const
{
	if (IsNavigationRelevant())
	{
		TArray<UPrimitiveComponent*> LocalCollisionComponents = GetCollisionComponents(/*bInOnlyEnabledComponents = */true);
		for (int32 CompIdx = 0; CompIdx < LocalCollisionComponents.Num(); CompIdx++)
		{
			UPrimitiveComponent* PrimComp = LocalCollisionComponents[CompIdx];
			if (PrimComp == nullptr)
			{
				UE_LOGF(LogNavigation, Warning, "%ls: skipping null collision component at index %d in %ls", ANSI_TO_TCHAR(__FUNCTION__), CompIdx, *GetFullNameSafe(this));
				continue;
			}


			FCompositeNavModifier CompositeNavModifier;
			CompositeNavModifier.CreateAreaModifiers(PrimComp, WaterNavAreaClass);
			for (FAreaNavModifier& AreaNavModifier : CompositeNavModifier.GetMutableAreas())
			{
				AreaNavModifier.SetExpandTopByCellHeight(true);
			}

			Data.Modifiers.Add(CompositeNavModifier);
			// skip recursion on this component
			if (PrimComp != this)
			{
				PrimComp->GetNavigationData(Data);
			}
		}
	}
}

FBox UWaterBodyComponent::GetNavigationBounds() const
{
	return GetCollisionComponentBounds();
}

bool UWaterBodyComponent::IsNavigationRelevant() const
{
	return Super::IsNavigationRelevant() && (GetCollisionComponents().Num() > 0);
}

void UWaterBodyComponent::ApplyCollisionSettings()
{
	// Transfer the collision settings of the water body component to all of its child collision components 
	TArray<UPrimitiveComponent*> CollisionComponents = GetCollisionComponents(/*bInOnlyEnabledComponents = */false);
	for (UPrimitiveComponent* CollisionComponent : CollisionComponents)
	{
		CopySharedCollisionSettingsToComponent(CollisionComponent);
	}
}

void UWaterBodyComponent::ApplyNavigationSettings()
{
	// Transfer the navigation settings of the water body component to all of its child collision components 
	const TArray<UPrimitiveComponent*> CollisionComponents = GetCollisionComponents(/*bInOnlyEnabledComponents = */false);
	for (UPrimitiveComponent* CollisionComponent : CollisionComponents)
	{
		CopySharedNavigationSettingsToComponent(CollisionComponent);
	}
}

void UWaterBodyComponent::RequestGPUWaveDataUpdate()
{
	if (FWaterBodyManager* Manager = UWaterSubsystem::GetWaterBodyManager(GetWorld()))
	{
		Manager->RequestWaveDataRebuild();
	}
}

void UWaterBodyComponent::BeginUpdateWaterBody()
{
	UpdateSplineComponent();
}

void UWaterBodyComponent::UpdateWaterBody(bool bWithExclusionVolumes)
{
	// The first update is without exclusion volumes : perform it.
	// The second update is with exclusion volumes but there's no need to perform it again if we don't have exclusion volumes anyway, because the result will be the same.
	if (!bWithExclusionVolumes || GetExclusionVolumes().Num() > 0)
	{
		OnUpdateBody(bWithExclusionVolumes);
	}

	// Update our cached bounds now that our shape may have changed:
	UpdateBounds();
}

void UWaterBodyComponent::UpdateAll(const FOnWaterBodyChangedParams& InParams)
{
	BeginUpdateWaterBody();

	AWaterBody* WaterBodyOwner = GetWaterBodyActor();
	if (WaterBodyOwner == nullptr)
	{
		return;
	}

	const bool bUserTriggered = InParams.bUserTriggered;
	bool bShapeOrPositionChanged = InParams.bShapeOrPositionChanged;
	
	// TODO [jonathan.bard] : Replace GIsEditor by World->IsEditorWorld
	if (GIsEditor || IsBodyDynamic())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UWaterBodyComponent::UpdateAll);

		bShapeOrPositionChanged |= UpdateWaterHeight();

		if (bShapeOrPositionChanged)
		{
			// We might be affected to a different landscape now that our shape has changed : 
			Landscape.Reset();
		}

		// First, update the water body without taking into account exclusion volumes, as those rely on the collision to detect overlapping water bodies
		UpdateWaterBody(/* bWithExclusionVolumes*/false);

		// Then, update the list of exclusion volumes after this adjustment
		if (bShapeOrPositionChanged)
		{
			UpdateIslands();

			UpdateExclusionVolumes();

			// Only update the water zones if this was a user triggered change. Rely on the serialized water zone pointer normally.
			if (bUserTriggered)
			{
				UpdateWaterZones();
			}
		}

		// Finally, generate the body once again, this time with the updated list of exclusion volumes
		UpdateWaterBody(/*bWithExclusionVolumes*/true);

		ApplyCollisionSettings();

		ApplyNavigationSettings();

		if (bShapeOrPositionChanged)
		{
			EWaterZoneRebuildFlags RebuildFlags = EWaterZoneRebuildFlags::All;
			if (!UE::Water::ShouldUpdateWaterMeshForPropertyChange(InParams.PropertyChangedEvent))
			{
				EnumRemoveFlags(RebuildFlags, EWaterZoneRebuildFlags::UpdateWaterMesh);
			}

			MarkOwningWaterZoneForRebuild(RebuildFlags);
		}

		// 'UpdateWaterBody' creates/updates collision components so we need to update the navigation data
		if (IsRegistered() && IsNavigationRelevant())
		{
			FNavigationSystem::UpdateActorAndComponentData(*WaterBodyOwner);
		}

#if WITH_EDITOR
		UpdateWaterSpriteComponent();
#endif
	}

	// Always update the visibility, since it might change based on CVars and there's a transient boolean (bIsComponent
	UpdateVisibility();
}

void UWaterBodyComponent::OnPostRegisterAllComponents()
{
#if WITH_EDITOR
	// Before this version, we always updated the water zone pointer on load. This no longer occurs so any water body which doesn't have a serialized pointer needs to set it now
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterOwningZonePointerFixup)
	{
		if (OwningWaterZone.IsNull())
		{
			// This needs to happen during a cook commandlet since the asset wasn't ever saved with a valid pointer in the editor.
			UpdateWaterZones(/** bAllowChangesDuringCook = */ true);
		}
	}

	
	bool bIsConservativeRasterMeshInvalid = false;
	if (UWaterBodyInfoMeshComponent* WaterInfoMeshComponent = GetWaterInfoMeshComponent(); IsValid(WaterInfoMeshComponent) && IsValid(WaterInfoMeshComponent->GetStaticMesh()))
	{
		const bool bHasConservativeRasterMesh = WaterInfoMeshComponent->bIsConservativeRasterCompatible;
		const bool bShouldHaveConservativeRastermesh = CVarWaterBodyBuildConservativeRasterizationMesh.GetValueOnGameThread() != 0;
		bIsConservativeRasterMeshInvalid = (bHasConservativeRasterMesh != bShouldHaveConservativeRastermesh);
	}

	if (AffectsWaterInfo() && (bIsConservativeRasterMeshInvalid || GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterBodyStaticMeshFixup))
	{
		const IWaterModuleInterface& WaterModule = FModuleManager::GetModuleChecked<IWaterModuleInterface>("Water");
		if (IWaterEditorServices* WaterEditorServices = WaterModule.GetWaterEditorServices())
		{
			if (GetWorld() && GetWorld()->WorldType == EWorldType::Editor)
			{
				WaterEditorServices->TryMarkPackageAsModified(GetPackage());
			}
		}
		UpdateWaterBodyRenderData();
	}
	
	// Ensure that the sprite component is updated once the water body is fully setup after PostRegister.
	UpdateWaterSpriteComponent();
#endif // WITH_EDITOR
}

void UWaterBodyComponent::UpdateSplineComponent()
{
	if (UWaterSplineComponent* WaterSpline = GetWaterSpline())
	{
#if WITH_EDITOR
		// #todo_water: should we expose this at runtime? Might be necessary for dynamically changing water bodies.
		WaterSpline->SynchronizeWaterProperties();
#endif // WITH_EDITOR
		WaterSpline->SetClosedLoop(IsWaterSplineClosedLoop());
	}
}

void UWaterBodyComponent::OnWaterBodyChanged(const FOnWaterBodyChangedParams& InParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWaterBodyComponent::OnWaterBodyChanged)
	// It's possible to get called without a water spline after the Redo of a water body deletion (i.e. the water body actor gets deleted again, hence its SplineComp is restored to nullptr)
	//  This is a very-edgy case that needs to be checked everywhere that UpdateAll might hook into so it's simpler to just skip it all. The actor is in limbo by then anyway (it only survives because
	//  of the editor transaction) :
	if (GetWaterSpline())
	{
		UpdateAll(InParams);

		// Some of the spline parameters need to be transferred to the underwater post process MID, if any : 
		if (InParams.bShapeOrPositionChanged)
		{
			SetDynamicParametersOnUnderwaterPostProcessMID(UnderwaterPostProcessMID);
		}
	}

#if WITH_EDITOR
	if (UE::Water::ShouldUpdateWaterMeshForPropertyChange(InParams.PropertyChangedEvent))
	{
		UpdateWaterBodyRenderData();
	}

	AWaterBody* const WaterBodyActor = GetWaterBodyActor();
	if (WaterBodyActor == nullptr)
	{
		return;
	}

	// Transfer the FOnWaterBodyChangedParams parameters to FWaterBrushActorChangedEventParams :
	IWaterBrushActorInterface::FWaterBrushActorChangedEventParams Params(WaterBodyActor, InParams.PropertyChangedEvent);
	Params.bUserTriggered = InParams.bUserTriggered;
	Params.bShapeOrPositionChanged = InParams.bShapeOrPositionChanged;
	Params.bWeightmapSettingsChanged = InParams.bWeightmapSettingsChanged;
	WaterBodyActor->BroadcastWaterBrushActorChangedEvent(Params);
#endif
}

void UWaterBodyComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FWaterCustomVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

#if WITH_EDITOR
	if (Ar.IsObjectReferenceCollector() && Ar.IsSaving())
	{
		FSoftObjectPathSerializationScope EditorOnlyScope(ESoftObjectPathCollectType::EditorOnlyCollect);
		FSoftObjectPath WaterSpriteTexture(GetWaterSpriteTextureName());
		Ar << WaterSpriteTexture;
	}

	if (Ar.IsLoading())
	{
		// The WaterInfoMaterial fixup must happen in Serialize to ensure that blueprint hierarchy implements the fixup in the correct order
		// such that parent BPs set the value before children are serialized.
		if (GetLinkerCustomVersion(FWaterCustomVersion::GUID) < FWaterCustomVersion::DefaultWaterInfoMaterialAssignmentMoved)
		{
			// the defaults before this version no longer carry the default provided WaterInfoMaterial from their constructor
			// so we can detect old data that hasn't been overriden by users by checking if the WaterInfoMaterial is null. 
			// If a user _had_ overriden the material, it would have been set in UObject::Serialize.
			if (WaterInfoMaterial == nullptr)
			{
				FCookLoadScope UsedInGameScope(ECookLoadType::UsedInGame);
				WaterInfoMaterial = GetDefault<UWaterRuntimeSettings>()->GetDefaultWaterInfoMaterial();
			}
		}
	}
#endif
}

void UWaterBodyComponent::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// WaterMeshOverride is now enough to override the water mesh (bOverrideWaterMesh_DEPRECATED was superfluous), so make sure to discard WaterMeshOverride (except on custom water bodies) when the boolean wasn't set :
	if (!bOverrideWaterMesh_DEPRECATED && (WaterMeshOverride != nullptr) && (GetWaterBodyType() != EWaterBodyType::Transition))
	{
		WaterMeshOverride = nullptr;
	}

	// If available, use far mesh material as the HLOD material for water bodies created before HLOD support was added.
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterHLODSupportAdded)
	{
		if (const AWaterZone* WaterZone = GetWaterZone())
		{
			const UWaterMeshComponent* WaterMeshComponent = WaterZone->GetWaterMeshComponent();
			check(WaterMeshComponent);

			WaterHLODMaterial = WaterMeshComponent->FarDistanceMaterial;
		}
	}
#endif // WITH_EDITORONLY_DATA

	DeprecateData();

#if WITH_EDITORONLY_DATA
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterBodyPhysicalMaterialPropertyRemoval)
	{
		SetPhysMaterialOverride(PhysicalMaterial_DEPRECATED);
	}
#endif // WITH_EDITORONLY_DATA

	if ((IsComponentPSOPrecachingEnabled() && RHISupportsManualVertexFetch(GMaxRHIShaderPlatform)))
	{
		FPSOPrecacheParams PrecachePSOParams;
		SetupPrecachePSOParams(PrecachePSOParams);

		const FVertexFactoryType* VFType = &FLocalVertexFactory::StaticType;
		FPSOPrecacheVertexFactoryDataList VFDataList;
		VFDataList.Add(FPSOPrecacheVertexFactoryData(VFType));

		TArray<TObjectPtr<UMaterialInterface>> Materials = { WaterMaterial, UnderwaterPostProcessMaterial, WaterInfoMaterial };

		for (TObjectPtr<UMaterialInterface>& Mat : Materials)
		{
			if(Mat)
			{ 
				Mat->ConditionalPostLoad();
				Mat->PrecachePSOs(VFType, PrecachePSOParams);
			}
		}
	}

#if WITH_EDITOR
	RegisterOnUpdateWavesData(GetWaterWaves(), /* bRegister = */true);

	if (IsTemplate() && ((OwningWaterZone != nullptr) || (!OwningWaterZone.GetAssetName().IsEmpty())))
	{
		TWeakObjectPtr<UWaterBodyComponent> WaterBodyComponent = this;

		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(
				LOCTEXT("MapCheck_Warning_WaterZoneReferenceInCDO", "WaterBodyComponent contains a reference to {0} but is a template object. Please use the following action to mark the package dirty then save it."),
				FText::FromString(OwningWaterZone.GetLongPackageName()))))
			->AddToken(FActionToken::Create(LOCTEXT("MapCheck_MarkWaterBodyComponentPackageDirty", "Mark WaterBodyComponent package Dirty"), LOCTEXT("MapCheck_MarkWaterBodyComponentPackageDirty_Desc", "Marks the WaterBodyComponent's package dirty."),
				FOnActionTokenExecuted::CreateLambda([WaterBodyComponent]()
			{
				if (WaterBodyComponent.IsValid())
				{
					WaterBodyComponent->MarkPackageDirty();
				}
			}), /*bInSingleUse = */true));

		FMessageLog("MapCheck").Open(EMessageSeverity::Warning);
		
		OwningWaterZone.Reset();
	}

	
	bool bNotifyModifiedPackage = false;
	// If the detail mode of the water body component or the water spline component are ever not DM_Low,
	// depending on per platform project settings they may be culled out in a cooked build for certain platforms
	// breaking our assumptions that the waterbodycomponent/watersplinecomponent should always be present in game.
	// This is a tricky issue to debug if it comes up, since there will be very little indication as to why the component
	// went missing. There is not a valid use case for having DetailMode != DM_Low on either of these components so we
	// are conservative here and simply prevent error if it's ever not the case while encouraging users to fixup their data.

	if (DetailMode != DM_Low || (GetWaterSpline() && GetWaterSpline()->DetailMode != DM_Low))
	{
		DetailMode = DM_Low;
		if (UWaterSplineComponent* SplineComp = GetWaterSpline())
		{
			SplineComp->DetailMode = DM_Low;
		}
		bNotifyModifiedPackage = true;
	}

	if (GetLinkerCustomVersion(FWaterCustomVersion::GUID) < FWaterCustomVersion::RebuildWaterMeshDataTSetChange)
	{
		UpdateWaterBodyRenderData();

		bNotifyModifiedPackage = true;
	}

	// Push the request to the user that they should mark this modified water package as dirty and resave it to persist the DetailMode change.
	if (bNotifyModifiedPackage)
	{
		const IWaterModuleInterface& WaterModule = FModuleManager::GetModuleChecked<IWaterModuleInterface>("Water");
		if (IWaterEditorServices* WaterEditorServices = WaterModule.GetWaterEditorServices())
		{
			if (GetWorld() && GetWorld()->WorldType == EWorldType::Editor)
			{
				WaterEditorServices->TryMarkPackageAsModified(GetPackage());
			}
		}
	}
#endif // WITH_EDITOR
}

void UWaterBodyComponent::DeprecateData()
{
#if WITH_EDITORONLY_DATA
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterBodyComponentCollisionSettingsRefactor)
	{
		// Deprecate the old collision / navigation data and update it on all sub-components :
		SetCollisionProfileName(CollisionProfileName_DEPRECATED);
		SetGenerateOverlapEvents(bGenerateCollisions_DEPRECATED);
		// Transfer info to sub-components :
		ApplyCollisionSettings();

		bool bCanAffectNav = bGenerateCollisions_DEPRECATED && bCanAffectNavigation_DEPRECATED;
		SetCustomNavigableGeometry(bCanAffectNav ? EHasCustomNavigableGeometry::EvenIfNotCollidable : EHasCustomNavigableGeometry::No);
		SetCanEverAffectNavigation(bCanAffectNav);
		bFillCollisionUnderneathForNavmesh = bFillCollisionUnderWaterBodiesForNavmesh_DEPRECATED;
		// Transfer info to sub-components :
		ApplyNavigationSettings();
	}

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterBodyStaticMeshComponents)
	{
		WaterStaticMeshMaterial = WaterLODMaterial_DEPRECATED;
	}
#endif // WITH_EDITORONLY_DATA
}

void UWaterBodyComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
#if WITH_EDITOR
	RegisterOnChangeWaterSplineData(/*bRegister = */false);
	RegisterOnUpdateWavesData(GetWaterWaves(), /*bRegister = */false);
#endif // WITH_EDITOR

	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

void UWaterBodyComponent::OnComponentCollisionSettingsChanged(bool bUpdateOverlaps)
{
	if (IsRegistered() && !IsTemplate())			// not for CDOs
	{
		Super::OnComponentCollisionSettingsChanged(bUpdateOverlaps);

		// Transfer all settings leading to OnComponentCollisionSettingsChanged to be called to the sub-components handling collisions:
		ApplyCollisionSettings();
	}
}

void UWaterBodyComponent::OnGenerateOverlapEventsChanged()
{
	if (IsRegistered() && !IsTemplate())			// not for CDOs
	{
		Super::OnGenerateOverlapEventsChanged();

		ApplyCollisionSettings();
	}
}

void UWaterBodyComponent::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
}

bool UWaterBodyComponent::SetDynamicParametersOnMID(UMaterialInstanceDynamic* InMID)
{
	UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(GetWorld());
	if ((InMID == nullptr) || (WaterSubsystem == nullptr))
	{
		return false;
	}

	const float GlobalOceanHeight = WaterSubsystem->GetOceanTotalHeight();
	InMID->SetScalarParameterValue(WaterBodyIndexParamName, WaterBodyIndex);
	InMID->SetScalarParameterValue(GlobalOceanHeightName, GlobalOceanHeight);

	if (const AWaterZone* WaterZone = GetWaterZone())
	{
		InMID->SetScalarParameterValue(WaterZoneIndexParamName, WaterZone->GetWaterZoneIndex());
		InMID->SetTextureParameterValue(WaterVelocityAndHeightName, WaterZone->WaterInfoTextureArray);
	}

	return true;
}

bool UWaterBodyComponent::SetDynamicParametersOnUnderwaterPostProcessMID(UMaterialInstanceDynamic* InMID)
{
	UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(GetWorld());
	if ((InMID == nullptr) || (WaterSubsystem == nullptr))
	{
		return false;
	}

	// The post process MID needs the same base parameters as the water materials : 
	SetDynamicParametersOnMID(InMID);

	// Add here the list of parameters that the underwater material needs (for not nothing more than the standard material) :

	return true;
}

bool UWaterBodyComponent::SetDynamicParametersOnWaterInfoMID(UMaterialInstanceDynamic* InMID)
{
	UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(GetWorld());
	if ((InMID == nullptr) || (WaterSubsystem == nullptr))
	{
		return false;
	}

	InMID->SetScalarParameterValue(WaterBodyZOffsetParamName, WaterHeightmapSettings.FalloffSettings.ZOffset);
	InMID->SetScalarParameterValue(MaxFlowVelocityParamName, FWaterUtils::GetWaterMaxFlowVelocity(false));

	return true;
}

float UWaterBodyComponent::GetWaveReferenceTime() const
{
	if (HasWaves())
	{
		if (UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(GetWorld()))
		{
			return WaterSubsystem->GetWaterTimeSeconds();
		}
	}
	return 0.0f;
}

/** Returns wave-related information at the given world position and for this water depth.
 Pass bSimpleWaves = true for the simple version (faster computation, lesser accuracy, doesn't perturb the normal) */
bool UWaterBodyComponent::GetWaveInfoAtPosition(const FVector& InPosition, float InWaterDepth, bool bInSimpleWaves, FWaveInfo& InOutWaveInfo) const
{
	if (!HasWaves())
	{
		return false; //Collision needs to be fixed for rivers
	}

	float MaxWaveHeight = GetMaxWaveHeight();

	InOutWaveInfo.ReferenceTime = GetWaveReferenceTime();
	InOutWaveInfo.AttenuationFactor *= GetWaveAttenuationFactor(InPosition, InWaterDepth);

	// No need to perform computation if we're going to cancel it out afterwards :
	if (InOutWaveInfo.AttenuationFactor > 0.0f)
	{
		// Maximum amplitude that the wave can reach at this location : 
		InOutWaveInfo.MaxHeight = MaxWaveHeight * InOutWaveInfo.AttenuationFactor;

		float WaveHeight;
		if (bInSimpleWaves)
		{
			WaveHeight = GetSimpleWaveHeightAtPosition(InPosition, InWaterDepth, InOutWaveInfo.ReferenceTime);
		}
		else
		{
			FVector ComputedNormal;
			WaveHeight = GetWaveHeightAtPosition(InPosition, InWaterDepth, InOutWaveInfo.ReferenceTime, ComputedNormal);
			// Attenuate the normal :
			ComputedNormal = FMath::Lerp(InOutWaveInfo.Normal, ComputedNormal, InOutWaveInfo.AttenuationFactor);
			if (!ComputedNormal.IsZero())
			{
				InOutWaveInfo.Normal = ComputedNormal;
			}
		}

		// Attenuate the wave amplitude :
		InOutWaveInfo.Height = WaveHeight * InOutWaveInfo.AttenuationFactor;
	}

	return true;
}

float UWaterBodyComponent::GetMaxWaveHeight() const
{
	return (HasWaves() ? GetWaterWaves()->GetMaxWaveHeight() : 0.0f) + MaxWaveHeightOffset;
}

void UWaterBodyComponent::CopySharedCollisionSettingsToComponent(UPrimitiveComponent* InComponent)
{
	InComponent->SetCollisionEnabled(GetCollisionEnabled());
	InComponent->SetNotifyRigidBodyCollision(BodyInstance.bNotifyRigidBodyCollision);
	InComponent->SetCollisionResponseToChannels(BodyInstance.GetResponseToChannels());
	InComponent->SetCollisionProfileName(GetCollisionProfileName(), /*bUpdateOverlaps=*/ true);
	InComponent->SetGenerateOverlapEvents(GetGenerateOverlapEvents());
	InComponent->SetPhysMaterialOverride(BodyInstance.GetPhysMaterialOverride());
}

void UWaterBodyComponent::CopySharedNavigationSettingsToComponent(UPrimitiveComponent* InComponent)
{
	InComponent->SetCanEverAffectNavigation(CanEverAffectNavigation());
	InComponent->SetCustomNavigableGeometry(HasCustomNavigableGeometry());
	InComponent->bFillCollisionUnderneathForNavmesh = GetCollisionEnabled() != ECollisionEnabled::NoCollision && bFillCollisionUnderneathForNavmesh;
}

float UWaterBodyComponent::GetWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime, FVector& OutNormal) const
{
	check(HasWaves());

	return GetWaterWaves()->GetWaveHeightAtPosition(InPosition, InWaterDepth, InTime, OutNormal);
}

float UWaterBodyComponent::GetSimpleWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime) const
{
	check(HasWaves());

	return GetWaterWaves()->GetSimpleWaveHeightAtPosition(InPosition, InWaterDepth, InTime);
}

float UWaterBodyComponent::GetWaveAttenuationFactor(const FVector& InPosition, float InWaterDepth) const
{
	check(HasWaves());

	return GetWaterWaves()->GetWaveAttenuationFactor(InPosition, InWaterDepth, TargetWaveMaskDepth);
}

UWaterWavesBase* UWaterBodyComponent::GetWaterWaves() const
{
	if (AWaterBody* OwningWaterBody = GetWaterBodyActor())
	{
		return OwningWaterBody->GetWaterWaves();
	}
	return nullptr;
}

bool UWaterBodyComponent::GenerateWaterBodyMesh(UE::Geometry::FDynamicMesh3& OutMesh, UE::Geometry::FDynamicMesh3* OutDilatedMesh /* = nullptr */) const
{
	 checkf(!AffectsWaterInfo(), TEXT("WaterBodyComponent affects water info but does not implement GenerateWaterBodyMesh!")); 
	 return false;
}

#if WITH_EDITOR
void UWaterBodyComponent::SetWaterBodyStaticMeshEnabled(bool bEnabled)
{
	bool bChanged = StaticMeshSettings.bEnableWaterBodyStaticMesh != bEnabled;
	StaticMeshSettings.bEnableWaterBodyStaticMesh = bEnabled;

	if (bChanged)
	{
		UpdateWaterBodyRenderData();
	}
}

void UWaterBodyComponent::UpdateWaterBodyRenderData()
{
	// Don't need to update render data if we can't ever render, unless we're cooking in which case we do need to regenerate the water mesh data for the game
	if (FApp::CanEverRender() || IsRunningCookCommandlet())
	{
		if (const UWorld* World = GetWorld())
		{
			// Avoid updating any mesh data if we are in a PIE world if dynamic data changes aren't allowed.
			if (!World->HasBegunPlay() || AreDynamicDataChangesAllowed())
			{
				UpdateWaterInfoMeshComponents();

				UpdateWaterBodyStaticMeshComponents();

				OnWaterBodyRenderDataUpdated();
			}
		}
	}
}

void UWaterBodyComponent::OnWaterBodyRenderDataUpdated()
{
	const bool bAffectsWaterInfo = AffectsWaterInfo();
	const bool bAffectsWaterMesh = AffectsWaterMesh();

	MarkOwningWaterZoneForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture, /* bOnlyWithinWaterBodyBounds = */ false);

	if (bAffectsWaterInfo || bAffectsWaterMesh)
	{
		MarkRenderStateDirty();
	}
}

void UWaterBodyComponent::UpdateWaterInfoMeshComponents()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWaterBodyComponent::UpdateWaterInfoMeshComponents);

	// Not needed for CDOs or template actors
	if (IsTemplate() || (GetWaterBodyActor() && GetWaterBodyActor()->bIsEditorPreviewActor))
	{
		return;
	}

	if (!AffectsWaterInfo())
	{
		return;
	}

	const bool bShouldBuildConservativeRasterMesh = CVarWaterBodyBuildConservativeRasterizationMesh.GetValueOnGameThread() != 0;
	FWaterBodyMeshBuilder MeshBuilder;
	MeshBuilder.BuildWaterInfoMeshes( this, GetWaterInfoMeshComponent(), GetDilatedWaterInfoMeshComponent(), bShouldBuildConservativeRasterMesh);
}

void UWaterBodyComponent::UpdateWaterBodyStaticMeshComponents()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWaterBodyComponent::UpdateWaterBodyStaticMeshComponents);

	// Not needed for CDOs or template actors
	if (IsTemplate() || (GetOwner() && GetOwner()->bIsEditorPreviewActor))
	{
		return;
	}

	AWaterBody* WaterBodyActor = GetWaterBodyActor();
	if (WaterBodyActor == nullptr)
	{
		return;
	}

	WaterBodyActor->CleanupInvalidStaticMeshComponents();
	const TArray<TObjectPtr<UWaterBodyStaticMeshComponent>>& WaterBodyStaticMeshComponents = WaterBodyActor->GetWaterBodyStaticMeshComponents();

	const bool bShouldUseStaticMesh = StaticMeshSettings.bEnableWaterBodyStaticMesh && (GetWaterMeshOverride() == nullptr) && (GetWaterBodyType() != EWaterBodyType::Transition);

	if (bShouldUseStaticMesh)
	{
		FWaterBodyMeshBuilder LODMeshBuilder;
		TArray<TObjectPtr<UWaterBodyStaticMeshComponent>> NewStaticMeshComponents = LODMeshBuilder.BuildWaterBodyStaticMesh(this, StaticMeshSettings, WaterBodyStaticMeshComponents);
		
		const int32 RemoveStartIndex = NewStaticMeshComponents.Num();
		const int32 NumToRemove = FMath::Max(WaterBodyStaticMeshComponents.Num() - NewStaticMeshComponents.Num(), 0);
		WaterBodyActor->SetWaterBodyStaticMeshComponents(NewStaticMeshComponents, TConstArrayView<TObjectPtr<UWaterBodyStaticMeshComponent>>(WaterBodyStaticMeshComponents.GetData() + RemoveStartIndex, NumToRemove));
	}
	else
	{
		WaterBodyActor->SetWaterBodyStaticMeshComponents({}, WaterBodyStaticMeshComponents);
	}
}

void UWaterBodyComponent::FixupEditorTransform()
{
	// Water bodies should not have a scale of 0 on any component or they can generate NaN/Infs.
	// Any water body that is rendered into the water mesh should only have a z scale of 1.
	FVector CurrentScale = GetRelativeScale3D();
	if (CanEverAffectWaterMesh() || FMath::IsNearlyZero(CurrentScale.Z))
	{
		CurrentScale.Z = 1.;
	}
	if (FMath::IsNearlyZero(CurrentScale.X))
	{
		CurrentScale.X = 1.;
	}
	if (FMath::IsNearlyZero(CurrentScale.Y))
	{
		CurrentScale.Y = 1.;
	}
	SetRelativeScale3D(CurrentScale);
	
	// All water bodies which can ever be rendered by the water mesh should only have yaw rotation.
	if (CanEverAffectWaterMesh())
	{
		FRotator CorrectedRotation = GetRelativeRotation();
		CorrectedRotation.Pitch = 0.f;
		CorrectedRotation.Roll = 0.f;
		SetRelativeRotation(CorrectedRotation);
	}

}

void UWaterBodyComponent::CreateWaterSpriteComponent()
{
	UTexture2D* Texture = nullptr;
	{
		FCookLoadScope EditorOnlyScope(ECookLoadType::EditorOnly);
		Texture = LoadObject<UTexture2D>(nullptr, GetWaterSpriteTextureName());
	}

	IWaterModuleInterface& WaterModule = FModuleManager::GetModuleChecked<IWaterModuleInterface>(TEXT("Water"));
	if (IWaterEditorServices* WaterEditorServices = WaterModule.GetWaterEditorServices())
	{
		WaterEditorServices->RegisterWaterActorSprite(GetClass(), Texture);
	}

	bVisualizeComponent = true;
	CreateSpriteComponent(Texture);

	// CreateSpriteComponent will not create a component if we are in a game world such as PIE.
	if (SpriteComponent != nullptr)
	{
		SpriteComponent->SetRelativeScale3D(FVector(1.f, 1.f, 1.f));
		SpriteComponent->SetRelativeLocation(FVector(0.f, 0.f,  GetDefault<UWaterRuntimeSettings>()->WaterBodyIconWorldZOffset));
	}
}

void UWaterBodyComponent::UpdateWaterSpriteComponent()
{
	if (SpriteComponent)
	{
		SpriteComponent->SetVisibility(IsIconVisible());

		UTexture2D* IconTexture = SpriteComponent->Sprite;
		IWaterModuleInterface& WaterModule = FModuleManager::GetModuleChecked<IWaterModuleInterface>("Water");
		if (const IWaterEditorServices* WaterEditorServices = WaterModule.GetWaterEditorServices())
		{
			bool bHasError = false;
			TArray<TSharedRef<FTokenizedMessage>> StatusMessages = CheckWaterBodyStatus();
			for (const TSharedRef<FTokenizedMessage>& StatusMessage : StatusMessages)
			{
				// Message severities are ordered from most severe to least severe.
				if (StatusMessage->GetSeverity() <= EMessageSeverity::Error)
				{
					bHasError = true;
					break;
				}
			}

			if (bHasError)
			{
				IconTexture = WaterEditorServices->GetErrorSprite();
			}
			else
			{
				IconTexture = WaterEditorServices->GetWaterActorSprite(GetClass());
			}
		}


		const FVector ZOffset(0.0f, 0.0f, GetDefault<UWaterRuntimeSettings>()->WaterBodyIconWorldZOffset);
		SpriteComponent->SetWorldLocation(GetWaterSpriteLocation() + ZOffset);
		SpriteComponent->Sprite = IconTexture;

		SpriteComponent->MarkRenderStateDirty();
	}
}

TSubclassOf<UHLODBuilder> UWaterBodyComponent::GetCustomHLODBuilderClass() const
{
	return UWaterBodyHLODBuilder::StaticClass();
}

FMeshDescription UWaterBodyComponent::GetHLODMeshDescription() const
{
	FMeshDescription MeshDescription;

	if (WaterMeshOverride)
	{
		const int32 LastLOD = WaterMeshOverride->GetNumLODs() - 1;

		// If source model is valid, clone the mesh description, otherwise recreate it from the render data
		bool bIsMeshDescriptionValid = WaterMeshOverride->CloneMeshDescription(LastLOD, MeshDescription);
		if (!bIsMeshDescriptionValid)
		{
			const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
			MeshMergeUtilities.RetrieveMeshDescription(WaterMeshOverride, LastLOD, MeshDescription);
		}

		return MeshDescription;
	}
	
	return FWaterBodyMeshBuilder().BuildMeshDescription(this);
}

UMaterialInterface* UWaterBodyComponent::GetHLODMaterial() const
{
	return WaterHLODMaterial;
}

void UWaterBodyComponent::SetHLODMaterial(UMaterialInterface* InMaterial)
{
	WaterHLODMaterial = InMaterial;
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

