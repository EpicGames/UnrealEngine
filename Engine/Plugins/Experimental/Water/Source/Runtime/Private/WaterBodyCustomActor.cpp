// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyCustomActor.h"

#include "WaterBodyCustomComponent.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyCustomActor)

#if WITH_EDITOR
#endif

// ----------------------------------------------------------------------------------

AWaterBodyCustom::AWaterBodyCustom(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WaterBodyType = EWaterBodyType::Transition;

#if WITH_EDITORONLY_DATA
	bAffectsLandscape_DEPRECATED = false;
#endif
}

void AWaterBodyCustom::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterBodyComponentRefactor)
	{
		UWaterBodyCustomComponent* CustomComponent = CastChecked<UWaterBodyCustomComponent>(WaterBodyComponent);
		if (CustomGenerator_DEPRECATED)
		{
			CustomComponent->MeshComp =	CustomGenerator_DEPRECATED->MeshComp;
		}
	}
#endif // WITH_EDITORONLY_DATA
}

// ----------------------------------------------------------------------------------

UDEPRECATED_CustomMeshGenerator::UDEPRECATED_CustomMeshGenerator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
