// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyCustomComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/UObjectToken.h"
#include "WaterBodyActor.h"
#include "WaterBodyMeshComponent.h"
#include "WaterSplineComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyCustomComponent)

#if WITH_EDITOR
#endif

#define LOCTEXT_NAMESPACE "Water"

// ----------------------------------------------------------------------------------

UWaterBodyCustomComponent::UWaterBodyCustomComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAffectsLandscape = false;

	// @todo_water : Remove these checks (Once AWaterBody is no more Blueprintable, these methods should become PURE_VIRTUAL and this class should overload them)
	check(!IsFlatSurface());
	check(!IsWaterSplineClosedLoop());
	check(!IsHeightOffsetSupported());
}

TArray<UPrimitiveComponent*> UWaterBodyCustomComponent::GetCollisionComponents(bool bInOnlyEnabledComponents) const
{
	TArray<UPrimitiveComponent*> Result;
	if ((MeshComp != nullptr) && (!bInOnlyEnabledComponents || (MeshComp->GetCollisionEnabled() != ECollisionEnabled::NoCollision)))
	{
		Result.Add(MeshComp);
	}
	return Result;
}

TArray<UPrimitiveComponent*> UWaterBodyCustomComponent::GetStandardRenderableComponents() const 
{
	TArray<UPrimitiveComponent*> Result;
	if (MeshComp != nullptr)
	{
		Result.Add(MeshComp);
	}
	return Result;
}

FBoxSphereBounds UWaterBodyCustomComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if (MeshComp)
	{
		return MeshComp->CalcBounds(MeshComp->GetRelativeTransform()).TransformBy(LocalToWorld);
	}
	return Super::CalcBounds(LocalToWorld);
}

void UWaterBodyCustomComponent::Reset()
{
	AActor* Owner = GetOwner();
	check(Owner);

	TArray<UStaticMeshComponent*> MeshComponents;
	Owner->GetComponents(MeshComponents);

	MeshComp = nullptr;
	for (UStaticMeshComponent* MeshComponent : MeshComponents)
	{
		MeshComponent->DestroyComponent();
	}
}

void UWaterBodyCustomComponent::OnUpdateBody(bool bWithExclusionVolumes)
{
	AActor* OwnerActor = GetOwner();
	check(OwnerActor);

	if (!MeshComp)
	{
		MeshComp = NewObject<UStaticMeshComponent>(OwnerActor, TEXT("CustomMeshComponent"), RF_Transactional);
		MeshComp->SetNetAddressable(); // it's deterministically named so it's addressable over network (needed for collision)
		MeshComp->SetupAttachment(this);
		MeshComp->bEnableAutoLODGeneration = false; // The water body component is already responsible for generating HLOD of the WaterBody actor.

		if(IsRegistered())
		{
			MeshComp->RegisterComponent();
		}
	}

	TInlineComponentArray<UPrimitiveComponent*> PrimitiveComponents;
	OwnerActor->GetComponents(PrimitiveComponents);

	// Make no assumptions for custom water bodies: all (non-visualization) primitive components will be included in navigation except the water info components
	for (UPrimitiveComponent* Comp : PrimitiveComponents)
	{
#if WITH_EDITORONLY_DATA
		if (Comp->IsVisualizationComponent())
		{
			continue;
		}
#endif // WITH_EDITORONLY_DATA
		if (Comp->IsA<UWaterBodyMeshComponent>())
		{
			continue;
		}

		// Do not copy Nav setting over Water Spline Component, its just a spline component and will report Navigation warning if only one spline point because of empty bounds box
		if (!Comp->IsA<UWaterSplineComponent>())
		{
			CopySharedNavigationSettingsToComponent(Comp);
		}

		Comp->SetMobility(Mobility);
	}

	CreateOrUpdateWaterMID();
	MeshComp->SetStaticMesh(GetWaterMeshOverride());
	MeshComp->SetCastShadow(false);
	CopySharedCollisionSettingsToComponent(MeshComp);
	CopySharedNavigationSettingsToComponent(MeshComp);
	MeshComp->MarkRenderStateDirty();
}

void UWaterBodyCustomComponent::CreateOrUpdateWaterMID()
{
	Super::CreateOrUpdateWaterMID();
	if (MeshComp != nullptr)
	{
		MeshComp->SetMaterial(0, WaterMID);
	}
}

FPrimitiveSceneProxy* UWaterBodyCustomComponent::CreateSceneProxy()
{
	// Don't create a scene proxy for custom water body components since they don't render into the water info texture (yet)
	return nullptr;
}

void UWaterBodyCustomComponent::BeginUpdateWaterBody()
{
	Super::BeginUpdateWaterBody();

	UMaterialInstanceDynamic* WaterMaterialInstance = GetWaterMaterialInstance();
	if (WaterMaterialInstance && MeshComp)
	{
		// We need to get(or create) the water MID at runtime and apply it to the static mesh component 
		// The MID is transient so it will not make it through serialization, apply it here (at runtime)
		MeshComp->SetMaterial(0, WaterMaterialInstance);
	}
}

#if WITH_EDITOR
TArray<TSharedRef<FTokenizedMessage>> UWaterBodyCustomComponent::CheckWaterBodyStatus() 
{
	TArray<TSharedRef<FTokenizedMessage>> StatusMessages = Super::CheckWaterBodyStatus();

	if (!IsTemplate())
	{
		if (WaterMeshOverride == nullptr && GetWaterBodyActor() != nullptr)
		{
			StatusMessages.Add(FTokenizedMessage::Create(EMessageSeverity::Error)
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(FText::Format(
					LOCTEXT("MapCheck_Message_MissingCustomWaterMesh", "Custom water body {0} requires a static mesh to be rendered. Please set WaterMeshOverride to a valid static mesh. "),
					FText::FromString(GetWaterBodyActor()->GetActorLabel())))));
		}
	}

	return StatusMessages;
}

const TCHAR* UWaterBodyCustomComponent::GetWaterSpriteTextureName() const
{
	return TEXT("/Water/Icons/WaterBodyCustomSprite");
}

bool UWaterBodyCustomComponent::IsIconVisible() const
{
	return (GetWaterMeshOverride() == nullptr);
}

void UWaterBodyCustomComponent::PostLoad()
{
	Super::PostLoad();

	// Make sure the custom mesh component is not HLOD relevant
	// The water body component is already responsible for generating 
	// the HLOD of the WaterBody actor.
	if (MeshComp && MeshComp->IsHLODRelevant())
	{
		MeshComp->bEnableAutoLODGeneration = false;
	}
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

