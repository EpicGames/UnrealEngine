// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyExclusionVolume.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/FortniteValkyrieBranchObjectVersion.h"
#include "Engine/OverlapResult.h"
#include "Engine/World.h"
#include "UObject/UObjectIterator.h"
#include "WaterBodyActor.h"
#include "WaterBodyManager.h"
#include "WaterEditorServices.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyExclusionVolume)

#if WITH_EDITOR
#include "Components/BillboardComponent.h"
#include "WaterIconHelper.h"
#include "Modules/ModuleManager.h"
#include "WaterModule.h"
#endif

AWaterBodyExclusionVolume::AWaterBodyExclusionVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	ActorIcon = FWaterIconHelper::EnsureSpriteComponentCreated(this, TEXT("/Water/Icons/WaterBodyExclusionVolumeSprite"));
	// By default, exclusion volume are not spatially loaded to avoid recomputing water body collision at runtime when the actor is streamed in,
	//  which is allowed, but tends to be expensive
	bIsSpatiallyLoaded = false;
#endif
}

void AWaterBodyExclusionVolume::UpdateOverlappingWaterBodies(const FWaterExclusionVolumeChangedParams& Params)
{
	TArray<FOverlapResult> Overlaps;

	UWorld* World = GetWorld();
	if (World == nullptr)
	{
		return;
	}

	FBoxSphereBounds Bounds = GetBounds();
	World->OverlapMultiByObjectType(Overlaps, Bounds.Origin, FQuat::Identity, FCollisionObjectQueryParams::AllObjects, FCollisionShape::MakeBox(Bounds.BoxExtent));

	// Find any new overlapping bodies and notify them that this exclusion volume influences them
	TSet<UWaterBodyComponent*> ExistingOverlappingBodies;
	TSet<TWeakObjectPtr<UWaterBodyComponent>> NewOverlappingBodies;

	TSoftObjectPtr<AWaterBodyExclusionVolume> SoftThis(this);

	// Fixup overlapping bodies (iterating on actors on post-load will fail, but this is fine as this exclusion volume should not yet be referenced by an existing water body upon loading) : 
	FWaterBodyManager::ForEachWaterBodyComponent(World, [SoftThis, &ExistingOverlappingBodies](UWaterBodyComponent* WaterBodyComponent)
	{
		if (WaterBodyComponent->ContainsExclusionVolume(SoftThis))
		{
			ExistingOverlappingBodies.Add(WaterBodyComponent);
		}
		return true;
	});

	for (const FOverlapResult& Result : Overlaps)
	{
		AWaterBody* WaterBody = Result.OverlapObjectHandle.FetchActor<AWaterBody>();
		if (WaterBody)
		{
			bool bWaterBodyIsInList = WaterBodies.Contains(WaterBody);
			bool bAddToExclusion = (ExclusionMode == EWaterExclusionMode::RemoveWaterBodiesListFromExclusion && !bWaterBodyIsInList)
				|| (ExclusionMode == EWaterExclusionMode::AddWaterBodiesListToExclusion && bWaterBodyIsInList);
			
			if (bAddToExclusion)
			{
				UWaterBodyComponent* WaterBodyComponent = WaterBody->GetWaterBodyComponent();
				NewOverlappingBodies.Add(WaterBodyComponent);
				// If the water body is not already overlapping then notify
				if (!ExistingOverlappingBodies.Contains(WaterBodyComponent))
				{
					WaterBodyComponent->AddExclusionVolume(this);
				}
			}
		}
	}

	// Find existing bodies that are no longer overlapping and remove them, otherwise trigger them to rebuild with the new location
	FOnWaterBodyChangedParams WaterBodyChangedParams(Params.PropertyChangedEvent);
	WaterBodyChangedParams.bUserTriggered = Params.bUserTriggered;
	WaterBodyChangedParams.bShapeOrPositionChanged = false;
	WaterBodyChangedParams.bWeightmapSettingsChanged = false;
	for (UWaterBodyComponent* ExistingBody : ExistingOverlappingBodies)
	{
		if (ExistingBody)
		{
			if (!NewOverlappingBodies.Contains(ExistingBody))
			{
				ExistingBody->RemoveExclusionVolume(this);
			}
		}
	}
}

#if WITH_EDITOR
void AWaterBodyExclusionVolume::UpdateActorIcon()
{
	UTexture2D* IconTexture = ActorIcon->Sprite;
	IWaterModuleInterface& WaterModule = FModuleManager::GetModuleChecked<IWaterModuleInterface>("Water");
	if (const IWaterEditorServices* WaterEditorServices = WaterModule.GetWaterEditorServices())
	{
		IconTexture = WaterEditorServices->GetWaterActorSprite(GetClass());
	}
	FWaterIconHelper::UpdateSpriteComponent(this, IconTexture);
}
#endif // WITH_EDITOR

void AWaterBodyExclusionVolume::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	FWaterExclusionVolumeChangedParams Params;
	Params.bUserTriggered = false;
	UpdateOverlappingWaterBodies(Params);

	UpdateAffectedWaterBodyCollisions(Params);
}

void AWaterBodyExclusionVolume::UpdateAffectedWaterBodyCollisions(const FWaterExclusionVolumeChangedParams& Params)
{
	TSoftObjectPtr<AWaterBodyExclusionVolume> SoftThis(this);

	FOnWaterBodyChangedParams WaterBodyChangedParams(Params.PropertyChangedEvent);
	WaterBodyChangedParams.bUserTriggered = Params.bUserTriggered;
	WaterBodyChangedParams.bShapeOrPositionChanged = false;
	WaterBodyChangedParams.bWeightmapSettingsChanged = false;
	FWaterBodyManager::ForEachWaterBodyComponent(GetWorld(), [SoftThis, &WaterBodyChangedParams](UWaterBodyComponent* WaterBodyComponent)
	{
		if (WaterBodyComponent->ContainsExclusionVolume(SoftThis))
		{
			WaterBodyComponent->OnWaterBodyChanged(WaterBodyChangedParams);
		}
		return true;
	});
}

void AWaterBodyExclusionVolume::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// Perform data deprecation: 
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::SupportMultipleWaterBodiesPerExclusionVolume)
	{
		if (WaterBodyToIgnore_DEPRECATED != nullptr)
		{
			WaterBodiesToIgnore_DEPRECATED.Add(WaterBodyToIgnore_DEPRECATED);
			WaterBodyToIgnore_DEPRECATED = nullptr;
		}
	}

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterExclusionVolumeExcludeAllDefault)
	{
		WaterBodiesToExclude_DEPRECATED = MoveTemp(WaterBodiesToIgnore_DEPRECATED);

		// If the existing actor had selected specific water bodies to exclude then we set the new value to false by default.
		if (!bIgnoreAllOverlappingWaterBodies_DEPRECATED && WaterBodiesToExclude_DEPRECATED.Num() > 0)
		{
			bExcludeAllOverlappingWaterBodies_DEPRECATED = false;
		}
	}
	if (GetLinkerCustomVersion(FFortniteValkyrieBranchObjectVersion::GUID) < FFortniteValkyrieBranchObjectVersion::WaterBodyExclusionVolumeMode)
	{
		ExclusionMode = bExcludeAllOverlappingWaterBodies_DEPRECATED ? EWaterExclusionMode::RemoveWaterBodiesListFromExclusion : EWaterExclusionMode::AddWaterBodiesListToExclusion;
		for (AWaterBody* WaterBody : WaterBodiesToExclude_DEPRECATED)
		{
			WaterBodies.Add(WaterBody);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void AWaterBodyExclusionVolume::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteValkyrieBranchObjectVersion::GUID);
}

void AWaterBodyExclusionVolume::Destroyed()
{
	Super::Destroyed();

	// No need for water bodies to keep a pointer to ourselves, even if a lazy one :
	// Use a TObjectRange here instead of the Manager for each because it may not be valid
	UWorld* World = GetWorld();
	for (UWaterBodyComponent* WaterBodyComponent : TObjectRange<UWaterBodyComponent>())
	{
		if (WaterBodyComponent && WaterBodyComponent->GetWorld() == World)
		{
			WaterBodyComponent->RemoveExclusionVolume(this);
		}
	}
}

#if WITH_EDITOR
void AWaterBodyExclusionVolume::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);

	FWaterExclusionVolumeChangedParams Params;
	Params.PropertyChangedEvent.ChangeType = bFinished ? EPropertyChangeType::ValueSet : EPropertyChangeType::Interactive;
	Params.bUserTriggered = true;
	UpdateOverlappingWaterBodies(Params);

	UpdateAffectedWaterBodyCollisions(Params);
}

void AWaterBodyExclusionVolume::PostEditUndo()
{
	Super::PostEditUndo();

	FWaterExclusionVolumeChangedParams Params;
	Params.bUserTriggered = true;
	UpdateOverlappingWaterBodies(Params);

	UpdateAffectedWaterBodyCollisions(Params);
}

void AWaterBodyExclusionVolume::PostEditImport()
{
	Super::PostEditImport();

	FWaterExclusionVolumeChangedParams Params;
	Params.bUserTriggered = true;
	UpdateOverlappingWaterBodies(Params);

	UpdateAffectedWaterBodyCollisions(Params);
}

void AWaterBodyExclusionVolume::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FWaterExclusionVolumeChangedParams Params(PropertyChangedEvent);
	Params.bUserTriggered = true;
	UpdateOverlappingWaterBodies(Params);

	UpdateAffectedWaterBodyCollisions(Params);
}

FName AWaterBodyExclusionVolume::GetCustomIconName() const
{
	// override this to not inherit from ABrush::GetCustomIconName's behavior and use the class icon instead 
	return NAME_None;
}

#endif // WITH_EDITOR

