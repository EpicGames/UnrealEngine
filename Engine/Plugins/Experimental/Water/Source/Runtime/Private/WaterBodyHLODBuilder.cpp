// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyHLODBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyHLODBuilder)

#if WITH_EDITOR

#include "Components/StaticMeshComponent.h"
#include "UObject/Package.h"
#include "WaterBodyComponent.h"

#include "MeshDescription.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"

#include "MeshSimplification.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "Serialization/ArchiveCrc32.h"
#include "Engine/HLODProxy.h"
#include "SceneView.h"

#include "WorldPartition/HLOD/HLODHashBuilder.h"

#endif

UWaterBodyHLODBuilder::UWaterBodyHLODBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

static double GetPixelToWorldUnit(const double InDistanceZ)
{
	// Generate a projection matrix.
	static const FIntPoint ScreenSize(1920, 1080);
	static const float HalfFOVRad = FMath::DegreesToRadians(45.0f);
	static const FMatrix InvProjectionMatrix = FPerspectiveMatrix(HalfFOVRad, ScreenSize.X, ScreenSize.Y, 0.01f).Inverse();

	const FVector2D ScreenPixel((ScreenSize / 2) + FIntPoint(1, 0));
	FVector Origin, Direction;
	FSceneView::DeprojectScreenToWorld(ScreenPixel, FIntRect(FIntPoint(0, 0), ScreenSize), InvProjectionMatrix, Origin, Direction);
	const FVector4d WorldPos = -Direction * InDistanceZ;
	return WorldPos.X;
}

bool UWaterBodyHLODBuilder::ComputeHLODHash(FHLODHashBuilder& HashBuilder, const UActorComponent* InSourceComponent) const
{
	const UWaterBodyComponent* WaterBodyComponent = Cast<UWaterBodyComponent>(InSourceComponent);
	if (!WaterBodyComponent)
	{
		return false;
	}
	
	HashBuilder.HashField(WaterBodyComponent->GetComponentTransform(), TEXT("Transform"));
	HashBuilder.HashField(WaterBodyComponent->GetHLODMeshDescription(), TEXT("HLODMesh"));

	HashBuilder << WaterBodyComponent->GetHLODMaterial();

	return true;
}

TArray<UActorComponent*> UWaterBodyHLODBuilder::Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const
{
	TArray<UWaterBodyComponent*> SourceWaterBodyComponents = FilterComponents<UWaterBodyComponent>(InSourceComponents);
	
	TArray<UActorComponent*> HLODComponents;
	TArray<UStaticMesh*> StaticMeshes;

	// Compute the maximum error to keep displacement under a single pixel for the requested HLOD draw distance.
	const double MaxDisplacement = GetPixelToWorldUnit(InHLODBuildContext.MinVisibleDistance);
	const double MaxDisplacementSqr = FMath::Square(MaxDisplacement);

	for (UWaterBodyComponent* WaterBodyComponent : SourceWaterBodyComponents)
	{
		FName StaticMeshName = MakeUniqueObjectName(nullptr, UStaticMesh::StaticClass(), TEXT("WaterBodyHLODMesh"));
		UStaticMesh* StaticMesh = NewObject<UStaticMesh>(InHLODBuildContext.AssetsOuter, StaticMeshName);
		StaticMesh->SetFlags(InHLODBuildContext.AssetsObjectFlags);

		// Mesh
		{
			FMeshDescription HLODMesh = WaterBodyComponent->GetHLODMeshDescription();

			// Simplify mesh
			{
				// MeshDescription -> DynamicMesh
				UE::Geometry::FDynamicMesh3 DynamicMesh;
				FMeshDescriptionToDynamicMesh MeshDescriptionToDynamicMesh;
				MeshDescriptionToDynamicMesh.bDisableAttributes = true;
				MeshDescriptionToDynamicMesh.Convert(&HLODMesh, DynamicMesh, false);

				// Simplify
				UE::Geometry::FQEMSimplification Simplifier(&DynamicMesh);
				Simplifier.bPreserveBoundaryShape = false;
				Simplifier.SimplifyToMaxError(MaxDisplacementSqr);
				
				// DynamicMesh -> MeshDescription
				FMeshDescription SimplifiedWaterMesh = HLODMesh;
				FDynamicMeshToMeshDescription DynamicMeshToMeshDescription;
				DynamicMeshToMeshDescription.Convert(&DynamicMesh, SimplifiedWaterMesh);
								
				HLODMesh = SimplifiedWaterMesh;
			}

			StaticMesh->AddSourceModel();
			StaticMesh->CreateMeshDescription(0, HLODMesh);
			StaticMesh->CommitMeshDescription(0);
			StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);
		}
		
		// Material
		{
			UMaterialInterface* HLODMaterial = WaterBodyComponent->GetHLODMaterial();
			StaticMesh->AddMaterial(HLODMaterial);
		}
		
		StaticMeshes.Add(StaticMesh);

		FName ComponentName = MakeUniqueObjectName(nullptr, UStaticMeshComponent::StaticClass(), TEXT("WaterHLODMeshComponent"));
		UStaticMeshComponent* StaticMeshComponent = NewObject<UStaticMeshComponent>(GetTransientPackage(), ComponentName);
		StaticMeshComponent->SetRelativeTransform(WaterBodyComponent->GetComponentTransform());
		StaticMeshComponent->SetStaticMesh(StaticMesh);
		HLODComponents.Add(StaticMeshComponent);
	}

	UStaticMesh::BatchBuild(StaticMeshes);

	return HLODComponents;
}

#endif // #if WITH_EDITOR

