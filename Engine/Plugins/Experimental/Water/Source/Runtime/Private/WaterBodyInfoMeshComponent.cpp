// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyInfoMeshComponent.h"
#include "SceneView.h"
#include "WaterInfoRendering.h"
#include "StaticMeshSceneProxy.h"
#include "UObject/UObjectIterator.h"
#include "Rendering/CustomRenderPass.h"
#include "WaterBodyActor.h"
#include "WaterBodyComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyInfoMeshComponent)

void OnCVarWaterInfoSceneProxiesValueChanged(IConsoleVariable*)
{
	for (UWaterBodyInfoMeshComponent* WaterBodyInfoMeshComponent : TObjectRange<UWaterBodyInfoMeshComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
	{
		WaterBodyInfoMeshComponent->MarkRenderStateDirty();
	}
}

static TAutoConsoleVariable<int32> CVarShowWaterInfoSceneProxies(
	TEXT("r.Water.WaterInfo.ShowSceneProxies"),
	0,
	TEXT("When enabled, shows the water scene proxies in the main viewport. Useful for debugging only.\n")
	TEXT("0: Disabled\n")
	TEXT("1: Only show water info mesh\n")
	TEXT("2: Only show water info dilated mesh\n")
	TEXT("3: Show both water info meshes"),
	FConsoleVariableDelegate::CreateStatic(OnCVarWaterInfoSceneProxiesValueChanged),
	ECVF_RenderThreadSafe
);

extern TAutoConsoleVariable<int32> CVarWaterMeshGPUQuadTree;

int32 GetWaterInfoRenderingMethod()
{
	static IConsoleVariable* CVarWaterInfoRenderMethod = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Water.WaterInfo.RenderMethod"));
	return CVarWaterInfoRenderMethod ? CVarWaterInfoRenderMethod->GetInt() : 0;
}

UWaterBodyInfoMeshComponent::UWaterBodyInfoMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAffectDistanceFieldLighting = false;
	bSelectable = false;
	
	// Skip computing the bounds for this component since it should always be attached to the water body component and those bounds are the proper bounds for the info meshes.
	bUseAttachParentBound = true;
}

FPrimitiveSceneProxy* UWaterBodyInfoMeshComponent::CreateSceneProxy()
{
	if (!CanCreateSceneProxy())
	{
		return nullptr;
	}
	return new FWaterBodyInfoMeshSceneProxy(this, bIsDilatedMesh);
}

FWaterBodyInfoMeshSceneProxy::FWaterBodyInfoMeshSceneProxy(UWaterBodyInfoMeshComponent* Component, bool InbIsDilatedMesh)
	: FStaticMeshSceneProxy(Component, true),
	bIsDilatedMesh(InbIsDilatedMesh)
{
	// Disable Notify on WorldAddRemove. This prevents the component from being unhidden in FPrimitiveSceneProxy::OnLevelAddedToWorld_RenderThread if it was part of a streamed level.
	// WaterInfo proxies should only be unhidden during WaterInfo passes.
	bShouldNotifyOnWorldAddRemove = false;

	// When water info mesh is rendered with custom render passes, do not disable the mesh
	if (GetWaterInfoRenderingMethod() != 2)
	{
		SetEnabled(false);
	}

	check(Component != nullptr);

	if (AWaterBody* WaterBodyActor = Component->GetOwner<AWaterBody>())
	{
		if (UWaterBodyComponent* WaterBodyComponent = WaterBodyActor->GetWaterBodyComponent())
		{
			EWaterZoneRebuildFlags Flags = EWaterZoneRebuildFlags::UpdateWaterInfoTexture;
			// The GPU quad tree is built by rasterizing the water info meshes into a texture.
			// Just like the water info texture, the quadtree must also be updated if the proxy is rebuilt.
			if (CVarWaterMeshGPUQuadTree.GetValueOnGameThread() == 1)
			{
				Flags |= EWaterZoneRebuildFlags::UpdateWaterMesh;
			}
			WaterBodyComponent->MarkOwningWaterZoneForRebuild(Flags);
		}
	}
}

bool FWaterBodyInfoMeshSceneProxy::GetMeshElement(int32 LODIndex, int32 BatchIndex, int32 ElementIndex, uint8 InDepthPriorityGroup, bool bUseSelectionOutline, bool bAllowPreCulledIndices, FMeshBatch& OutMeshBatch) const
{
	bool bResult = FStaticMeshSceneProxy::GetMeshElement(LODIndex, BatchIndex, ElementIndex, InDepthPriorityGroup, bUseSelectionOutline, bAllowPreCulledIndices, OutMeshBatch);
	if (bResult)
	{
		OutMeshBatch.bUseForWaterInfoTextureDepth = bIsDilatedMesh; // The dilated mesh is drawn in the water info texture depth-only pass
	}
	return bResult;
}

static bool ShouldShowOutsideWaterInfoPass(bool bIsDilatedMesh)
{
#if UE_BUILD_SHIPPING
	return false;
#else
	const int32 ShowWaterInfoSceneProxiesValue = CVarShowWaterInfoSceneProxies.GetValueOnAnyThread();
	return (ShowWaterInfoSceneProxiesValue == 1 && !bIsDilatedMesh) || (ShowWaterInfoSceneProxiesValue == 2 && bIsDilatedMesh) || (ShowWaterInfoSceneProxiesValue > 2);
#endif
}

void FWaterBodyInfoMeshSceneProxy::SetEnabled(bool bInEnabled)
{
	SetForceHidden(!(bInEnabled || ShouldShowOutsideWaterInfoPass(bIsDilatedMesh)));
}

FPrimitiveViewRelevance FWaterBodyInfoMeshSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	using namespace UE::WaterInfo;

	FPrimitiveViewRelevance Result = FStaticMeshSceneProxy::GetViewRelevance(View);

	// When water info mesh is rendered with custom render passes, enable the mesh for drawing
	if (GetWaterInfoRenderingMethod() == 2)
	{
		Result.bDrawRelevance = false;
		if (View->CustomRenderPass != nullptr)
		{
			const FName& PassName = const_cast<FSceneView*>(View)->CustomRenderPass->GetTypeName();
			Result.bDrawRelevance = (PassName == GetWaterInfoDepthPassName())
				|| (PassName == GetWaterInfoColorPassName())
				|| (PassName == GetWaterInfoDilationPassName());

		}
		Result.bShadowRelevance = false;
	}

	Result.bDrawRelevance |= ShouldShowOutsideWaterInfoPass(bIsDilatedMesh);

	return Result;
}