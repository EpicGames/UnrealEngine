// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterBodyMeshComponent.h"
#include "StaticMeshSceneProxy.h"

#include "WaterBodyInfoMeshComponent.generated.h"

class UWaterBodyComponent;

/**
 * WaterBodyMeshComponent used to render the water body into the water info texture.
 * Utilizes a custom scene proxy to allow hiding the mesh outside of all other passes besides the water info passes.
 */
UCLASS(MinimalAPI)
class UWaterBodyInfoMeshComponent : public UWaterBodyMeshComponent
{
	GENERATED_BODY()
public:
	UWaterBodyInfoMeshComponent(const FObjectInitializer& ObjectInitializer);

	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	
	virtual bool IsHLODRelevant() const { return false; }

	// Whether the water body mesh has been enlarged/dilated past the shoreline of the actual water body.
	// For every water body, there is a dilated and non-dilated mesh. Both are needed for generating the water info texture.
	UPROPERTY()
	bool bIsDilatedMesh = false;

	// Whether the water body mesh is compatible with software conservative rasterization used for generating the GPU water quadtree.
	// This requires three UV channels with 32bit floats storing the vertex position of the previous and next vertex in the triangle.
	UPROPERTY()
	bool bIsConservativeRasterCompatible = false;

protected:
	// Even though we may be still precaching PSOs, we don't want to delay render proxy creation
	// or fallback to the default material. We're always returning false here so that we decline
	// those optimizations.
	virtual bool UsePSOPrecacheRenderProxyDelay() const override { return false; }
};

struct FWaterBodyInfoMeshSceneProxy : public FStaticMeshSceneProxy
{
public:
	FWaterBodyInfoMeshSceneProxy(UWaterBodyInfoMeshComponent* Component, bool InbIsDilatedMesh);

	virtual bool GetMeshElement(
		int32 LODIndex,
		int32 BatchIndex,
		int32 ElementIndex,
		uint8 InDepthPriorityGroup,
		bool bUseSelectionOutline,
		bool bAllowPreCulledIndices,
		FMeshBatch& OutMeshBatch) const override;

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;

	SIZE_T GetTypeHash() const
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	void SetEnabled(bool bInEnabled);

private:
	bool bIsDilatedMesh = false;
};

