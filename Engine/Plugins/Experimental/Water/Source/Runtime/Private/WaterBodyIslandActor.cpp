// Copyright Epic Games, Inc. All Rights Reserved.


#include "WaterBodyIslandActor.h"
#include "Engine/OverlapResult.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "WaterSplineComponent.h"
#include "PropertyPairsMap.h"
#include "WaterBodyActor.h"
#include "WaterBodyManager.h"
#include "WaterRuntimeSettings.h"
#include "WaterEditorServices.h"
#include "WaterVersion.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyIslandActor)

// ----------------------------------------------------------------------------------

#if WITH_EDITOR
#include "Components/BillboardComponent.h"
#include "Modules/ModuleManager.h"
#include "WaterIconHelper.h"
#include "WaterModule.h"
#include "Landscape.h"
#endif // WITH_EDITOR

// ----------------------------------------------------------------------------------

AWaterBodyIsland::AWaterBodyIsland(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SplineComp = CreateDefaultSubobject<UWaterSplineComponent>(TEXT("WaterSpline"));
	SplineComp->SetMobility(EComponentMobility::Static);
	SplineComp->SetClosedLoop(true);
	
#if WITH_EDITOR
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		SplineComp->OnWaterSplineDataChanged().AddUObject(this, &AWaterBodyIsland::OnWaterSplineDataChanged);
	}

	ActorIcon = FWaterIconHelper::EnsureSpriteComponentCreated(this, TEXT("/Water/Icons/WaterBodyIslandSprite"));
#endif

	RootComponent = SplineComp;
}

#if WITH_EDITOR
ETextureRenderTargetFormat AWaterBodyIsland::GetBrushRenderTargetFormat() const
{
	return ETextureRenderTargetFormat::RTF_RG16f;
}

void AWaterBodyIsland::GetBrushRenderDependencies(TSet<UObject*>& OutDependencies) const 
{
	// Lazy-load the referenced textures : this allows the settings to not be editor-only but to only actually load data if GetBrushRenderDependencies is called (which is : only in the editor)
	for (const TPair<FName, FWaterBodyWeightmapSettings>& Pair : WaterWeightmapSettings)
	{
		if (UTexture2D* Texture = Pair.Value.ModulationTexture.LoadSynchronous())
		{
			OutDependencies.Add(Texture);
		}
	}

	if (UTexture2D* Texture = WaterHeightmapSettings.Effects.Displacement.Texture.LoadSynchronous())
	{
		OutDependencies.Add(Texture);
	}
}
#endif //WITH_EDITOR

void AWaterBodyIsland::UpdateHeight()
{
	if (SplineComp)
	{
		const int32 NumSplinePoints = SplineComp->GetNumberOfSplinePoints();

		const float ActorZ = GetActorLocation().Z;

		for (int32 PointIndex = 0; PointIndex < NumSplinePoints; ++PointIndex)
		{
			FVector WorldLoc = SplineComp->GetLocationAtSplinePoint(PointIndex, ESplineCoordinateSpace::World);

			WorldLoc.Z = ActorZ;
			SplineComp->SetLocationAtSplinePoint(PointIndex, WorldLoc, ESplineCoordinateSpace::World);
		}
	}
}

void AWaterBodyIsland::Destroyed()
{
	Super::Destroyed();

	// No need for water bodies to keep a pointer to ourselves, even if a lazy one :
	// Use a TObjectRange here instead of the Manager for each because it may not be valid
	UWorld* World = GetWorld();
	for (UWaterBodyComponent* WaterBodyComponent : TObjectRange<UWaterBodyComponent>())
	{
		if (WaterBodyComponent && WaterBodyComponent->GetWorld() == World)
		{
			WaterBodyComponent->RemoveIsland(this);
		}
	}
}

void AWaterBodyIsland::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FWaterCustomVersion::GUID);
}

void AWaterBodyIsland::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (GetLinkerCustomVersion(FWaterCustomVersion::GUID) < FWaterCustomVersion::MoveTerrainCarvingSettingsToWater)
	{
		// Try to retrieve wave data from BP properties when it was defined in BP : 
		if (UClass* WaterBodyClass = GetClass())
		{
			if (WaterBodyClass->ClassGeneratedBy != nullptr)
			{
				for (FProperty* BPProperty = WaterBodyClass->PropertyLink; BPProperty != nullptr; BPProperty = BPProperty->PropertyLinkNext)
				{
					const FString CurveSettingsName(TEXT("Curve Settings"));
					if (BPProperty->GetName() == CurveSettingsName)
					{
						if (FStructProperty* OldCurveSettingsStructProperty = CastField<FStructProperty>(BPProperty))
						{
							FWaterCurveSettings* OldCurveSettings = static_cast<FWaterCurveSettings*>(OldCurveSettingsStructProperty->ContainerPtrToValuePtr<void>(this));
							WaterCurveSettings = *OldCurveSettings;
						}
					}

					const FString LayerWeightmapSettingsName(TEXT("Layer Weightmap Settings"));
					if (BPProperty->GetName() == LayerWeightmapSettingsName)
					{
						if (FMapProperty* OldLayerWeightmapSettingsMapProperty = CastField<FMapProperty>(BPProperty))
						{
							FScriptMapHelper MapHelper(OldLayerWeightmapSettingsMapProperty, OldLayerWeightmapSettingsMapProperty->ContainerPtrToValuePtr<void>(this));
							for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
							{
								uint8* PairPtr = MapHelper.GetPairPtr(It);
								const FName* Key = MapHelper.GetKeyProperty()->ContainerPtrToValuePtr<FName>(PairPtr);
								const FWaterBodyWeightmapSettings* Value = MapHelper.GetValueProperty()->ContainerPtrToValuePtr<FWaterBodyWeightmapSettings>(PairPtr);
								WaterWeightmapSettings.FindOrAdd(*Key) = *Value;
							}
						}
					}

					const FString TerrainEffectsName(TEXT("Terrain Effects"));
					if (BPProperty->GetName() == TerrainEffectsName)
					{
						if (FStructProperty* OldTerrainEffectsStructProperty = CastField<FStructProperty>(BPProperty))
						{
							FLandmassBrushEffectsList* OldTerrainEffectsSettings = static_cast<FLandmassBrushEffectsList*>(OldTerrainEffectsStructProperty->ContainerPtrToValuePtr<void>(this));
							check(sizeof(*OldTerrainEffectsSettings) == sizeof(FWaterBrushEffects));
							WaterHeightmapSettings.Effects = *reinterpret_cast<FWaterBrushEffects*>(OldTerrainEffectsSettings);
						}
					}
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
void AWaterBodyIsland::GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const
{
	Super::GetActorDescProperties(PropertyPairsMap);
	if (AffectsLandscape())
	{
		PropertyPairsMap.AddProperty(ALandscape::AffectsLandscapeActorDescProperty);
	}
}

void AWaterBodyIsland::UpdateOverlappingWaterBodyComponents()
{
	TArray<FOverlapResult> Overlaps;

	UWorld* World = GetWorld();
	if (World == nullptr)
	{
		return;
	}

	if (SplineComp)
	{
		FCollisionShape OverlapShape;
		// Expand shape in Z to ensure we get overlaps for islands slighty above or below water level
		OverlapShape.SetBox((FVector3f)SplineComp->Bounds.BoxExtent+FVector3f(0,0,10000));
		World->OverlapMultiByObjectType(Overlaps, SplineComp->Bounds.Origin, FQuat::Identity, FCollisionObjectQueryParams::AllObjects, OverlapShape);
	}

	// Find any new overlapping bodies and notify them that this island influences them
	TSet<UWaterBodyComponent*> ExistingOverlappingBodies;
	TSet<TWeakObjectPtr<UWaterBodyComponent>> NewOverlappingBodies;

	TSoftObjectPtr<AWaterBodyIsland> SoftThis(this);

	// Fixup overlapping bodies
	FWaterBodyManager::ForEachWaterBodyComponent(World, [SoftThis, &ExistingOverlappingBodies](UWaterBodyComponent* WaterBodyComponent)
	{
		if (WaterBodyComponent->ContainsIsland(SoftThis))
		{
			ExistingOverlappingBodies.Add(WaterBodyComponent);
		}
		return true;
	});

	for (const FOverlapResult& Result : Overlaps)
	{
		if (AWaterBody* WaterBody = Result.OverlapObjectHandle.FetchActor<AWaterBody>())
		{
			UWaterBodyComponent* WaterBodyComponent = WaterBody->GetWaterBodyComponent();
			NewOverlappingBodies.Add(WaterBodyComponent);
			// If the water body is not already overlapping then notify
			if (!ExistingOverlappingBodies.Contains(WaterBodyComponent))
			{
				WaterBodyComponent->AddIsland(this);
			}
		}
	}

	// Find existing bodies that are no longer overlapping and remove them
	for (UWaterBodyComponent* ExistingBodyComponent : ExistingOverlappingBodies)
	{
		if (ExistingBodyComponent && !NewOverlappingBodies.Contains(ExistingBodyComponent))
		{
			ExistingBodyComponent->RemoveIsland(this);
		}
	}
}

void AWaterBodyIsland::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);

	FOnWaterBodyIslandChangedParams Params;
	Params.PropertyChangedEvent.ChangeType = bFinished ? EPropertyChangeType::ValueSet : EPropertyChangeType::Interactive;
	Params.bShapeOrPositionChanged = true;
	Params.bUserTriggered = true;
	UpdateAll(Params);
}

void AWaterBodyIsland::PostEditUndo()
{
	Super::PostEditUndo();

	FOnWaterBodyIslandChangedParams Params;
	Params.bShapeOrPositionChanged = true;
	Params.bWeightmapSettingsChanged = true;
	UpdateAll(Params);
}

void AWaterBodyIsland::PostEditImport()
{
	Super::PostEditImport();

	FOnWaterBodyIslandChangedParams Params;
	Params.bShapeOrPositionChanged = true;
	Params.bWeightmapSettingsChanged = true;
	Params.bUserTriggered = true;
	UpdateAll(Params);
}

void AWaterBodyIsland::UpdateAll(const FOnWaterBodyIslandChangedParams& InParams)
{
	UpdateHeight();

	if (InParams.bShapeOrPositionChanged)
	{
		UpdateOverlappingWaterBodyComponents();
	}

	OnWaterBodyIslandChanged(InParams);

	UpdateActorIcon();
}

void AWaterBodyIsland::UpdateActorIcon()
{
	if (ActorIcon && SplineComp && !bIsEditorPreviewActor)
	{
		UTexture2D* IconTexture = ActorIcon->Sprite;
		IWaterModuleInterface& WaterModule = FModuleManager::GetModuleChecked<IWaterModuleInterface>("Water");
		if (const IWaterEditorServices* WaterEditorServices = WaterModule.GetWaterEditorServices())
		{
			IconTexture = WaterEditorServices->GetWaterActorSprite(GetClass());
		}
		FWaterIconHelper::UpdateSpriteComponent(this, IconTexture);

		// Move the actor icon to the center of the island
		FVector ZOffset(0.0f, 0.0f, GetDefault<UWaterRuntimeSettings>()->WaterBodyIconWorldZOffset);
		ActorIcon->SetWorldLocation(SplineComp->Bounds.Origin + ZOffset);
	}
}

void AWaterBodyIsland::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FOnWaterBodyIslandChangedParams Params(PropertyChangedEvent);
	Params.bUserTriggered = true;
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UWaterBodyComponent, LayerWeightmapSettings))
	{
		Params.bWeightmapSettingsChanged = true;
	}

	OnWaterBodyIslandChanged(Params);

	UpdateActorIcon();
}

void AWaterBodyIsland::OnWaterSplineDataChanged(const FOnWaterSplineDataChangedParams& InParams)
{
	UpdateOverlappingWaterBodyComponents();

	// Transfer the FOnWaterSplineDataChangedParams parameters to FOnWaterBodyIslandChangedParams :
	FOnWaterBodyIslandChangedParams Params(InParams.PropertyChangedEvent);
	Params.bShapeOrPositionChanged = true;
	Params.bUserTriggered = true;
	OnWaterBodyIslandChanged(Params);
}

void AWaterBodyIsland::OnWaterBodyIslandChanged(const FOnWaterBodyIslandChangedParams& InParams)
{
#if WITH_EDITOR
	// Transfer the FOnWaterBodyIslandChangedParams parameters to FWaterBrushActorChangedEventParams :
	FWaterBrushActorChangedEventParams Params(this, InParams.PropertyChangedEvent);
	Params.bShapeOrPositionChanged = InParams.bShapeOrPositionChanged;
	Params.bWeightmapSettingsChanged = InParams.bWeightmapSettingsChanged;
	Params.bUserTriggered = InParams.bUserTriggered;
	BroadcastWaterBrushActorChangedEvent(Params);
#endif
}

#endif
