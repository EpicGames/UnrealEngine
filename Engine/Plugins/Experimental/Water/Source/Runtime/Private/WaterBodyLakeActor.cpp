// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyLakeActor.h"
#include "Components/StaticMeshComponent.h"
#include "WaterBodyLakeComponent.h"
#include "LakeCollisionComponent.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyLakeActor)

#if WITH_EDITOR
#endif

// ----------------------------------------------------------------------------------

AWaterBodyLake::AWaterBodyLake(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WaterBodyType = EWaterBodyType::Lake;
}

void AWaterBodyLake::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterBodyComponentRefactor)
	{
		UWaterBodyLakeComponent* LakeComponent = CastChecked<UWaterBodyLakeComponent>(WaterBodyComponent);
		if (LakeGenerator_DEPRECATED)
		{
			LakeComponent->LakeMeshComp = LakeGenerator_DEPRECATED->LakeMeshComp;
			if (LakeComponent->LakeMeshComp)
			{
				LakeComponent->LakeMeshComp->SetupAttachment(LakeComponent);
			}
			LakeComponent->LakeCollision = LakeGenerator_DEPRECATED->LakeCollision;
			if (LakeComponent->LakeCollision)
			{
				LakeComponent->LakeCollision->SetupAttachment(LakeComponent);
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

// ----------------------------------------------------------------------------------

UDEPRECATED_LakeGenerator::UDEPRECATED_LakeGenerator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

