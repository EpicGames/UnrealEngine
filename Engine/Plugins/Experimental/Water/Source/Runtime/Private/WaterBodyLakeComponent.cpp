// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyLakeComponent.h"
#include "Components/StaticMeshComponent.h"
#include "WaterModule.h"
#include "DynamicMesh/InfoTypes.h"
#include "WaterSplineComponent.h"
#include "WaterSubsystem.h"
#include "Engine/StaticMesh.h"
#include "LakeCollisionComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Polygon2.h"
#include "ConstrainedDelaunay2.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Curve/PolygonOffsetUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyLakeComponent)

// ----------------------------------------------------------------------------------

UWaterBodyLakeComponent::UWaterBodyLakeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// @todo_water : Remove these checks (Once AWaterBody is no more Blueprintable, these methods should become PURE_VIRTUAL and this class should overload them)
	check(IsFlatSurface());
	check(IsWaterSplineClosedLoop());
	check(!IsHeightOffsetSupported());
}

TArray<UPrimitiveComponent*> UWaterBodyLakeComponent::GetCollisionComponents(bool bInOnlyEnabledComponents) const
{
	TArray<UPrimitiveComponent*> Result;
	if ((LakeCollision != nullptr) && (!bInOnlyEnabledComponents || (LakeCollision->GetCollisionEnabled() != ECollisionEnabled::NoCollision)))
	{
		Result.Add(LakeCollision);
	}
	return Result;
}

TArray<UPrimitiveComponent*> UWaterBodyLakeComponent::GetStandardRenderableComponents() const 
{
	TArray<UPrimitiveComponent*> Result = Super::GetStandardRenderableComponents();
	if (LakeMeshComp != nullptr)
	{
		Result.Add(LakeMeshComp);
	}
	return Result;
}

bool UWaterBodyLakeComponent::GenerateWaterBodyMesh(UE::Geometry::FDynamicMesh3& OutMesh, UE::Geometry::FDynamicMesh3* OutDilatedMesh) const
{
	using namespace UE::Geometry;

	TRACE_CPUPROFILER_EVENT_SCOPE(GenerateLakeMesh);

	const UWaterSplineComponent* SplineComp = GetWaterSpline();
	if (SplineComp == nullptr || SplineComp->GetNumberOfSplineSegments() < 3)
	{
		return false;
	}

	FPolygon2d LakePoly;
	{
		TArray<FVector> PolyLineVertices;
		SplineComp->ConvertSplineToPolyLine(ESplineCoordinateSpace::Local, FMath::Square(10.f), PolyLineVertices);
		
		for (int32 i = 0; i < PolyLineVertices.Num() - 1; ++i) // skip the last vertex since it's the same as the first vertex
		{
			LakePoly.AppendVertex(FVector2D(PolyLineVertices[i]));
		}
	}
	
	FConstrainedDelaunay2d Triangulation;
	Triangulation.FillRule = FConstrainedDelaunay2d::EFillRule::Positive;
	Triangulation.Add(LakePoly);
	if (!Triangulation.Triangulate())
	{
		UE_LOGF(LogWater, Warning, "Failed to triangulate Lake mesh for %ls.  Ensure that the Lake's spline does not form any loops (%ls)", *GetOwner()->GetActorNameOrLabel(), *GetFullNameSafe(GetOwner()));
	}

	if (Triangulation.Triangles.Num() == 0)
	{
		return false;
	}


	check(OutMesh.Attributes());
	FDynamicMeshColorOverlay* ColorOverlay = OutMesh.Attributes()->PrimaryColors();
	FDynamicMeshNormalOverlay* NormalOverlay = OutMesh.Attributes()->PrimaryNormals();

	for (const FVector2d& Vertex : Triangulation.Vertices)
	{
		FVertexInfo MeshVertex(FVector3d(Vertex.X, Vertex.Y, 0.f));

		OutMesh.AppendVertex(MeshVertex);
		ColorOverlay->AppendElement(FVector4f(0.f));
		NormalOverlay->AppendElement(FVector3f(0., 0., 1.));
	}

	for (const FIndex3i& Triangle : Triangulation.Triangles)
	{
		const int TriangleID = OutMesh.AppendTriangle(Triangle);
		ColorOverlay->SetTriangle(TriangleID, Triangle);
		NormalOverlay->SetTriangle(TriangleID, Triangle);
	}

	if (ShapeDilation > 0.f && OutDilatedMesh)
	{
		TArray<FGeneralPolygon2d> OffsetPolys;
		UE::Geometry::PolygonsOffset(
			ShapeDilation / 2.f,
			{ LakePoly },
			OffsetPolys,
			false,
			2.0);


		FConstrainedDelaunay2d DilationTriangulation;
		DilationTriangulation.FillRule = FConstrainedDelaunay2d::EFillRule::Positive;

		for (const FGeneralPolygon2d& Poly : OffsetPolys)
		{
			if (Poly.SignedArea() <= 0.)
			{
				UE_LOGF(LogWater, Warning, "Failed to apply offset for shape dilation (%ls", *GetOwner()->GetActorNameOrLabel());
				continue;
			}

			DilationTriangulation.Add(Poly);
		}

		if (!DilationTriangulation.Triangulate())
		{
			UE_LOGF(LogWater, Warning, "Failed to triangulate dilated lake mesh (%ls", *GetOwner()->GetActorNameOrLabel());
			return false;
		}

		if (DilationTriangulation.Triangles.Num() == 0)
		{
			return false;
		}

		FDynamicMeshColorOverlay* DilatedColorOverlay = OutDilatedMesh->Attributes()->PrimaryColors();
		FDynamicMeshNormalOverlay* DilatedNormalOverlay = OutDilatedMesh->Attributes()->PrimaryNormals();
		for (const FVector2d& Vertex : DilationTriangulation.Vertices)
		{
			FVertexInfo MeshVertex(FVector3d(Vertex.X, Vertex.Y, 0.));

			OutDilatedMesh->AppendVertex(MeshVertex);
			DilatedColorOverlay->AppendElement(FVector4f(0.f));
			DilatedNormalOverlay->AppendElement(FVector3f(0., 0., 1.));
		}

		for (const FIndex3i& Triangle : DilationTriangulation.Triangles)
		{
			const int TriangleID = OutDilatedMesh->AppendTriangle(Triangle);
			DilatedColorOverlay->SetTriangle(TriangleID, Triangle);
			DilatedNormalOverlay->SetTriangle(TriangleID, Triangle);
		}
	}

	return true;
}

FBoxSphereBounds UWaterBodyLakeComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if (UWaterSplineComponent* WaterSpline = GetWaterSpline())
	{
		FBox BoxExtent = WaterSpline->GetLocalBounds().GetBox();
		BoxExtent.Max.Z += MaxWaveHeightOffset;
		BoxExtent.Min.Z -= GetChannelDepth();
		return FBoxSphereBounds(BoxExtent).TransformBy(LocalToWorld);
	}
	
	return FBoxSphereBounds(ForceInit);
}

void UWaterBodyLakeComponent::Reset()
{
	AActor* Owner = GetOwner();
	check(Owner);

	TArray<UStaticMeshComponent*> MeshComponents;
	Owner->GetComponents(MeshComponents);

	for (UStaticMeshComponent* MeshComponent : MeshComponents)
	{
		MeshComponent->DestroyComponent();
	}

	if (LakeCollision)
	{
		LakeCollision->DestroyComponent();
	}

	LakeCollision = nullptr;
	LakeMeshComp = nullptr;
}

void UWaterBodyLakeComponent::OnUpdateBody(bool bWithExclusionVolumes)
{
	AActor* OwnerActor = GetOwner();
	check(OwnerActor);
	
	if (!LakeMeshComp)
	{
		LakeMeshComp = NewObject<UStaticMeshComponent>(OwnerActor, TEXT("LakeMeshComponent"), RF_Transactional);
		LakeMeshComp->SetupAttachment(this);
		LakeMeshComp->RegisterComponent();
	}

	if (GetCollisionEnabled() != ECollisionEnabled::NoCollision)
	{
		if (!LakeCollision)
		{
			LakeCollision = NewObject<ULakeCollisionComponent>(OwnerActor, TEXT("LakeCollisionComponent"), RF_Transactional);
			LakeCollision->SetNetAddressable(); // it's deterministically named so it's addressable over network (needed for collision)
			LakeCollision->SetupAttachment(this);
			LakeCollision->RegisterComponent();
		}
	}
	else
	{
		if (LakeCollision)
		{
			LakeCollision->DestroyComponent();
		}
		LakeCollision = nullptr;
	}

	if (UWaterSplineComponent* WaterSpline = GetWaterSpline())
	{
		UStaticMesh* WaterMesh = GetWaterMeshOverride() ? GetWaterMeshOverride() : ToRawPtr(UWaterSubsystem::StaticClass()->GetDefaultObject<UWaterSubsystem>()->DefaultLakeMesh);

		const FVector SplineExtent = WaterSpline->Bounds.BoxExtent;

		FVector WorldLoc(WaterSpline->Bounds.Origin);
		WorldLoc.Z = GetComponentLocation().Z;

		if (WaterMesh)
		{
			FTransform MeshCompToWorld = WaterSpline->GetComponentToWorld();
			// Scale the water mesh so that it is the size of the bounds
			FVector MeshExtent = WaterMesh->GetBounds().BoxExtent;
			MeshExtent.Z = 1.0f;

			const FVector LocalSplineExtent = WaterSpline->Bounds.TransformBy(MeshCompToWorld.Inverse()).BoxExtent;

			const FVector ScaleRatio = SplineExtent / MeshExtent;
			LakeMeshComp->SetWorldScale3D(FVector(ScaleRatio.X, ScaleRatio.Y, 1));
			LakeMeshComp->SetWorldLocation(WorldLoc);
			LakeMeshComp->SetWorldRotation(FQuat::Identity);
			LakeMeshComp->SetAbsolute(false, false, true);
			LakeMeshComp->SetStaticMesh(WaterMesh);
			LakeMeshComp->SetMaterial(0, GetWaterMaterialInstance());
			LakeMeshComp->SetCastShadow(false);
			LakeMeshComp->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		}

		LakeMeshComp->SetMobility(Mobility);

		if (LakeCollision)
		{
			check(GetCollisionEnabled () != ECollisionEnabled::NoCollision);
			LakeCollision->SetMobility(Mobility);
			CopySharedCollisionSettingsToComponent(LakeCollision);
			CopySharedNavigationSettingsToComponent(LakeCollision);

			const float Depth = GetChannelDepth() / 2;
			FVector LakeCollisionExtent = FVector(SplineExtent.X, SplineExtent.Y, 0.f) / GetComponentScale();
			LakeCollisionExtent.Z = Depth + CollisionHeightOffset / 2;
			LakeCollision->SetWorldLocation(WorldLoc + FVector(0, 0, -Depth + CollisionHeightOffset / 2));
			LakeCollision->UpdateCollision(LakeCollisionExtent, true);
		}
	}
}

#if WITH_EDITOR

const TCHAR* UWaterBodyLakeComponent::GetWaterSpriteTextureName() const
{
	return TEXT("/Water/Icons/WaterBodyLakeSprite");
}

FVector UWaterBodyLakeComponent::GetWaterSpriteLocation() const
{
	return GetWaterSpline()->Bounds.Origin;
}

#endif // WITH_EDITOR

