// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyManager.h"
#include "WaterBodyComponent.h"
#include "WaterSubsystem.h"
#include "WaterViewExtension.h"

void FWaterBodyManager::Initialize(UWorld* World)
{
	if (World != nullptr)
	{
		WaterViewExtension = FSceneViewExtensions::NewExtension<FWaterViewExtension>(World);
		WaterViewExtension->Initialize();
	}
}

void FWaterBodyManager::Deinitialize()
{
	WaterViewExtension->Deinitialize();
	WaterViewExtension.Reset();
}

int32 FWaterBodyManager::AddWaterBodyComponent(UWaterBodyComponent* InWaterBodyComponent)
{
	RequestWaveDataRebuild();
	int32 LowestFreeIndex = 0;
	const int32 WaterIndex = WaterBodyComponents.EmplaceAtLowestFreeIndex(LowestFreeIndex, InWaterBodyComponent);
	OnWaterBodyAdded.Broadcast(InWaterBodyComponent);
	return WaterIndex;
}

void FWaterBodyManager::RemoveWaterBodyComponent(UWaterBodyComponent* InWaterBodyComponent)
{
	RequestWaveDataRebuild();
	WaterBodyComponents.RemoveAt(InWaterBodyComponent->GetWaterBodyIndex());
	OnWaterBodyRemoved.Broadcast(InWaterBodyComponent);
}

int32 FWaterBodyManager::AddWaterZone(AWaterZone* InWaterZone)
{
	int32 LowestFreeIndex = 0;
	int32 WaterZoneIndex = WaterZones.EmplaceAtLowestFreeIndex(LowestFreeIndex, InWaterZone);

	if (WaterViewExtension)
	{
		WaterViewExtension->AddWaterZone(InWaterZone);
	}

	RequestGPUDataRebuild();

	return WaterZoneIndex;
}

void FWaterBodyManager::RemoveWaterZone(AWaterZone* InWaterZone)
{
	if (WaterViewExtension)
	{
		WaterViewExtension->RemoveWaterZone(InWaterZone);
	}

	WaterZones.RemoveAt(InWaterZone->GetWaterZoneIndex());

	RequestGPUDataRebuild();
}

void FWaterBodyManager::RequestGPUDataRebuild()
{
	if (WaterViewExtension)
	{
		WaterViewExtension->MarkGPUDataDirty();
	}
}

void FWaterBodyManager::RequestWaveDataRebuild()
{
	RequestGPUDataRebuild();

	// Recompute the maximum of all MaxWaveHeight : 
	GlobalMaxWaveHeight = 0.0f;
	ForEachWaterBodyComponent([this](UWaterBodyComponent* WaterBodyComponent)
	{
		GlobalMaxWaveHeight = FMath::Max(GlobalMaxWaveHeight, WaterBodyComponent->GetMaxWaveHeight());
		return true;
	});
}

void FWaterBodyManager::ForEachWaterBodyComponent(TFunctionRef<bool(UWaterBodyComponent*)> Pred) const
{
	for (const TWeakObjectPtr<UWaterBodyComponent>& WaterBodyComponentPtr : WaterBodyComponents)
	{
		if (UWaterBodyComponent* WaterBodyComponent = WaterBodyComponentPtr.Get())
		{
			if (!Pred(WaterBodyComponent))
			{
				return;
			}
		}
	}
}

void FWaterBodyManager::ForEachWaterBodyComponent(const UWorld* World, TFunctionRef<bool(UWaterBodyComponent*)> Pred)
{
	if (FWaterBodyManager* Manager = UWaterSubsystem::GetWaterBodyManager(World))
	{
		Manager->ForEachWaterBodyComponent(Pred);
	}
}

void FWaterBodyManager::ForEachWaterZone(TFunctionRef<bool(AWaterZone*)> Pred) const
{
	for (const TWeakObjectPtr<AWaterZone>& WaterZonePtr : WaterZones)
	{
		if (AWaterZone* WaterZone = WaterZonePtr.Get())
		{
			if (!Pred(WaterZone))
			{
				return;
			}
		}
	}
}

void FWaterBodyManager::ForEachWaterZone(const UWorld* World, TFunctionRef<bool(AWaterZone*)> Pred)
{
	if (FWaterBodyManager* Manager = UWaterSubsystem::GetWaterBodyManager(World))
	{
		Manager->ForEachWaterZone(Pred);
	}
}

void FWaterBodyManager::Shrink()
{
	WaterBodyComponents.Shrink();
	WaterZones.Shrink();
}

