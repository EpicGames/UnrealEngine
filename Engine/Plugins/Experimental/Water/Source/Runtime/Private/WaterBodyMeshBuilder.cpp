// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyMeshBuilder.h"

#include "WaterBodyComponent.h"
#include "WaterBodyStaticMeshComponent.h"
#include "WaterBodyInfoMeshComponent.h"

#if WITH_EDITOR
#include "Materials/MaterialInstanceDynamic.h"
#include "WaterUtils.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Operations/MeshPlaneCut.h"
#include "Engine/StaticMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshToMeshDescription.h"
#include "StaticMeshAttributes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "PrimitiveSceneProxy.h"
#include "StaticMeshResources.h"
#include "DynamicMeshBuilder.h"
#include "MeshDescriptionBuilder.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTangents.h"
#include "Algo/RemoveIf.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/CollisionProfile.h"
#include "UObject/Package.h"
#include "StaticMeshCompiler.h"
#include "Algo/AnyOf.h"
#include "WaterModule.h"
#endif // WITH_EDITOR

#if WITH_EDITOR
void FWaterBodyMeshBuilder::BuildWaterInfoMeshes(UWaterBodyComponent* WaterBodyComponent, UWaterBodyInfoMeshComponent* WaterInfoMeshComponent, UWaterBodyInfoMeshComponent* WaterInfoDilatedMeshComponent, bool bMakeWaterInfoMeshConservativeRasterCompatible) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWaterBodyMeshBuilder::BuildWaterInfoMesh);
	
	using namespace UE::Geometry;
	
	if (!ensureMsgf(WaterInfoMeshComponent != nullptr && WaterInfoDilatedMeshComponent != nullptr, TEXT("BuildWaterInfoMeshes called on a water body with null water info mesh component %s"), *WaterBodyComponent->GetOwner()->GetActorNameOrLabel()))
	{
		return;
	}

	FDynamicMesh3 WaterInfoMesh(EMeshComponents::None);
	FDynamicMesh3 WaterInfoDilatedMesh(EMeshComponents::None);
	GetDynamicMesh(WaterBodyComponent, WaterInfoMesh, &WaterInfoDilatedMesh, bMakeWaterInfoMeshConservativeRasterCompatible);
	
	if (IsRunningCookCommandlet())
	{
		if (!WaterInfoMesh.CheckValidity(FDynamicMesh3::FValidityOptions(), EValidityCheckFailMode::ReturnOnly))
		{
			UE_LOGF(LogWater, Warning, "Mesh for %ls failed validation", *GetNameSafe(WaterBodyComponent));
		}

		if (!WaterInfoDilatedMesh.CheckValidity(FDynamicMesh3::FValidityOptions(), EValidityCheckFailMode::ReturnOnly))
		{
			UE_LOGF(LogWater, Warning, "Dilated Mesh for %ls failed validation", *GetNameSafe(WaterBodyComponent));
		}
	}

	UMaterialInterface* WaterInfoMID = WaterBodyComponent->GetWaterInfoMaterialInstance();
	UObject* Outer = WaterBodyComponent->GetOwner();

	auto BuildWaterInfoMesh = [this, Outer, WaterInfoMID](const FDynamicMesh3& DynamicMesh, UWaterBodyInfoMeshComponent* MeshComponent, FName BaseName, bool bIsConservativeRasterMesh) -> UStaticMesh*
	{
		if (DynamicMesh.TriangleCount() == 0)
		{
			// if the generated mesh has no information, clear the previous static mesh.
			MeshComponent->SetStaticMesh(nullptr);
			return nullptr;
		}

		FName WaterInfoMeshName = MakeUniqueObjectName(Outer, UStaticMesh::StaticClass(), BaseName);
		UStaticMesh* StaticMesh = CreateUStaticMesh(Outer, WaterInfoMeshName);

		UpdateStaticMesh(StaticMesh, ConvertDynamicMeshToMeshDescription(DynamicMesh), bIsConservativeRasterMesh);

		StaticMesh->AddMaterial(WaterInfoMID);

		MeshComponent->SetStaticMesh(StaticMesh);
		MeshComponent->bIsConservativeRasterCompatible = bIsConservativeRasterMesh;

		return StaticMesh;
	};

	TArray<UStaticMesh*> StaticMeshes = 
	{
		BuildWaterInfoMesh(WaterInfoMesh, WaterInfoMeshComponent, TEXT("WaterInfoMesh"), bMakeWaterInfoMeshConservativeRasterCompatible),
		BuildWaterInfoMesh(WaterInfoDilatedMesh, WaterInfoDilatedMeshComponent, TEXT("WaterInfoDilatedMesh"), false /*bIsConservativeRasterMesh*/),
	};

	UStaticMesh::BatchBuild(StaticMeshes);
}

FMeshDescription FWaterBodyMeshBuilder::BuildMeshDescription(const UWaterBodyComponent* WaterBodyComponent) const
{
	using namespace UE::Geometry;

	FDynamicMesh3 WaterBodyMesh(EMeshComponents::None);
	GetDynamicMesh(WaterBodyComponent, WaterBodyMesh);

	return ConvertDynamicMeshToMeshDescription(WaterBodyMesh);
}

TArray<TObjectPtr<UWaterBodyStaticMeshComponent>> FWaterBodyMeshBuilder::BuildWaterBodyStaticMesh(UWaterBodyComponent* WaterBodyComponent, const FWaterBodyStaticMeshSettings& Settings, const TArray<TObjectPtr<UWaterBodyStaticMeshComponent>>& ReusableComponents) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWaterBodyMeshBuilder::BuildWaterBodyStaticMesh);
	
	using namespace UE::Geometry;

	FDynamicMesh3 EditMesh(EMeshComponents::None);

	GetDynamicMesh(WaterBodyComponent, EditMesh);

	const FBox MeshBounds3d = FBox(EditMesh.GetBounds());
	const FAxisAlignedBox2d MeshBounds2d(FVector2d(MeshBounds3d.Min), FVector2d(MeshBounds3d.Max));
	const double WaterMeshSectionSize = Settings.SectionSize;
	const EWaterBodyType WaterBodyType = WaterBodyComponent->GetWaterBodyType();
	const double ConstantSurfaceZ = WaterBodyComponent->GetConstantSurfaceZ();

	/** Mesh descriptions for sections which cannot be simplified to quads */
	TArray<FMeshDescription> SectionMeshDescriptions;
	std::atomic<uint32> NumSectionMeshDescriptions = {0};

	/** Positions of sections which can be simplified as quads. */
	TArray<FVector> QuadSectionPositions;
	std::atomic<uint32> NumQuadSectionPositions = {0};

	TArray<TObjectPtr<UWaterBodyStaticMeshComponent>> StaticMeshComponents;

	// Skip trying to section the mesh if the mesh bounds are smaller than size of a section.
	if (Settings.bSectionWaterBodyStaticMesh && (MeshBounds2d.Area() > WaterMeshSectionSize * WaterMeshSectionSize))
	{
		/**
		* Sectioning algorithm outline:
		* 
		* 1. Cut source mesh along X and Y axes at intervals of the section size. This prepares the geometry to then be sorted into buckets for each cell
		* 2. Sort the mesh into an AABB tree to be able to quickly test if vertices belong within a section's cell.
		* 3. In parallel for each section within the bounds of the mesh:
		*		a. Traverse the AABB tree to collect any triangles that are contained within or intersect with the bounds of that section.
		*		b. Identify if any triangles within the section are boundary triangles (they lie on the edge of the mesh).
		*		c. If this is the case, the geometry of the cell must be preserved since it defines the edge of our 2d polygon.
		*		d. If this is not the case, a simple quad mesh can be substituted which completely fills the section.
		*		e. A MeshDescription is then created for either the simplified quad or the triangles within the cell.
		* 4. After all the section meshes are created, create or reuse UWaterBodyStaticMeshComponents to render the individual meshes separately.
		*/

		// Round up when considering the regions to slice
		const uint32 NumberOfMeshSectionsX = FMath::DivideAndRoundUp(MeshBounds2d.Extents().X * 2., WaterMeshSectionSize);
		const uint32 NumberOfMeshSectionsY = FMath::DivideAndRoundUp(MeshBounds2d.Extents().Y * 2., WaterMeshSectionSize);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BuildWaterBodyStaticMesh::CutMesh);

			for (uint32 SectionIndexX = 0; SectionIndexX < NumberOfMeshSectionsX; ++SectionIndexX)
			{
				const FVector3d SlicePos = FVector3d(MeshBounds2d.Min + SectionIndexX * FVector2d(WaterMeshSectionSize, 0.0), 0.0);
				const FVector3d SliceNormal(-1.0, 0.0, 0.0);

				FMeshPlaneCut Cut(&EditMesh, SlicePos, SliceNormal);
				ensure(Cut.CutWithoutDelete(false));
			}

			for (uint32 SectionIndexY = 0; SectionIndexY < NumberOfMeshSectionsY; ++SectionIndexY)
			{
				const FVector3d SlicePos = FVector3d(MeshBounds2d.Min + SectionIndexY * FVector2d(0.0, WaterMeshSectionSize), 0.0);
				const FVector3d SliceNormal(0.0, -1.0, 0.0);

				FMeshPlaneCut Cut(&EditMesh, SlicePos, SliceNormal);
				ensure(Cut.CutWithoutDelete(false));
			}
		}

		FDynamicMeshAABBTree3 AABBTree(&EditMesh, true);

		const FDynamicMeshColorOverlay* ColorOverlay = EditMesh.Attributes()->PrimaryColors();

		const uint32 TotalNumberOfSections = NumberOfMeshSectionsX * NumberOfMeshSectionsY;
		SectionMeshDescriptions.SetNum(TotalNumberOfSections);
		QuadSectionPositions.SetNum(TotalNumberOfSections);

		ParallelFor(TotalNumberOfSections, [&](int32 SectionIndex)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BuildWaterBodyStaticMesh::BuildMeshSection);
			const int32 SectionIndexX = SectionIndex % NumberOfMeshSectionsX;
			const int32 SectionIndexY = SectionIndex / NumberOfMeshSectionsX;

			const FVector2d SectionPos = MeshBounds2d.Min + FVector2d(SectionIndexX, SectionIndexY) * WaterMeshSectionSize;

			const FAxisAlignedBox2d GridRectXY(SectionPos, SectionPos + WaterMeshSectionSize);

			FDynamicMeshAABBTree3::FTreeTraversal Traversal;
			
			int SelectAllDepth = TNumericLimits<int>::Max();
			int CurrentDepth = -1;
			
			TArray<int> Triangles;

			bool bHasBoundaryTriangle = false;

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(BuildWaterBodyStaticMesh::Traversal);

				Traversal.NextBoxF =
					[&GridRectXY, &SelectAllDepth, &CurrentDepth](const FAxisAlignedBox3d& Box, int Depth)
				{
					CurrentDepth = Depth;
					if (Depth > SelectAllDepth)
					{
						// We are deeper than the depth whose AABB was first detected to be contained in the RectangleXY,
						// descend and collect all leaf triangles
						return true;
					}
					
					SelectAllDepth = TNumericLimits<int>::Max();
					
					const FAxisAlignedBox2d BoxXY(FVector2d(Box.Min.X, Box.Min.Y), FVector2d(Box.Max.X, Box.Max.Y));
					if (GridRectXY.Intersects(BoxXY))
					{
						if (GridRectXY.Contains(BoxXY))
						{
							SelectAllDepth = Depth;
						}
						
						return true;		
					}
					return false;
				};
				
				Traversal.NextTriangleF =
					[&GridRectXY, &SelectAllDepth, &CurrentDepth, &Triangles, &EditMesh, &bHasBoundaryTriangle]
					(int TriangleID)
				{
					if (CurrentDepth >= SelectAllDepth)
					{
						// This TriangleID is entirely contained in the selection rectangle so we can skip intersection testing
						Triangles.Add(TriangleID);
						bHasBoundaryTriangle |= EditMesh.IsBoundaryTriangle(TriangleID);
						return;
					}

					FAxisAlignedBox2d ExpandedGridRect(GridRectXY);
					// Expand the rect by 1 unit to add a floating point error margin. 
					ExpandedGridRect.Expand(1);
				
					// this triangle is either on the edge or intersecting with the edge but since we did the plane cuts we know it can't be crossing over into the next cell
					FIndex3i Triangle = EditMesh.GetTriangle(TriangleID);
					FVector3d VertexA;
					FVector3d VertexB;
					FVector3d VertexC;
					EditMesh.GetTriVertices(TriangleID, VertexA, VertexB, VertexC);
					if ((ExpandedGridRect.Contains(FVector2d(VertexA))) &&
						(ExpandedGridRect.Contains(FVector2d(VertexB))) &&
						(ExpandedGridRect.Contains(FVector2d(VertexC))))
					{
						Triangles.Add(TriangleID);
						bHasBoundaryTriangle |= EditMesh.IsBoundaryTriangle(TriangleID);
					}
				};
					
				AABBTree.DoTraversal(Traversal);
			}

			TArray<FVector3d> VertexPositions;
			TArray<FVector4f> VertexColors;
			TArray<FVector> VertexNormals;
			TArray<FVector> VertexTangents;
			TArray<uint32> Indices;

			// Simplification can occur for lakes/rivers since they have flat geometry and no vertex color data.
			if (!bHasBoundaryTriangle && (WaterBodyType != EWaterBodyType::River) && Triangles.Num() > 0)
			{
				const uint32 QuadSectionIndex = NumQuadSectionPositions.fetch_add(1);
				QuadSectionPositions[QuadSectionIndex] = FVector(SectionPos.X, SectionPos.Y, 0);
				return;
			}

			TMap<uint32, uint32> VertexMap;
			for (int32 TriangleID : Triangles)
			{
				FIndex3i Triangle = EditMesh.GetTriangle(TriangleID);
				
				TArray<uint32, TInlineAllocator<3>> TriIndices;
				for (int32 TriIndex = 0; TriIndex < 3; ++TriIndex)
				{
					// Vertex data for this index already exists
					if (uint32* VertexIndexPtr = VertexMap.Find(Triangle[TriIndex]))
					{
						TriIndices.Add(*VertexIndexPtr);
					}
					// Vertex data for this index doesn't exist. Add it now:
					else
					{
						const int32 VertexID = Triangle[TriIndex];
						const FVector3d VertexPos = EditMesh.GetVertex(VertexID);

						// Only rivers have vertex colors for the velocity data.
						const FVector4f Color = WaterBodyType == EWaterBodyType::River ? ColorOverlay->GetElement(VertexID) : FVector4f(0);

						const uint32 NewIndex = VertexPositions.Add(VertexPos);
						VertexColors.Add(Color);
						VertexNormals.Add(FVector(0, 0, 1));
						VertexTangents.Add(FVector(1, 0, 0));

						VertexMap.Add(VertexID, NewIndex);

						TriIndices.Add(NewIndex);
					}
				}

				Indices.Append(TriIndices);
			}

			if (VertexPositions.Num() != 0)
			{
				const int32 NumVertices = VertexPositions.Num();
				check(VertexNormals.Num() == NumVertices);
				check(VertexColors.Num() == NumVertices);

				const int32 NumTriangles = Indices.Num() / 3;

				FMeshDescription MeshDescription;
				FStaticMeshAttributes(MeshDescription).Register();
				MeshDescription.Empty();

				FMeshDescriptionBuilder Builder;
				Builder.SetMeshDescription(&MeshDescription);
				Builder.SuspendMeshDescriptionIndexing();

				Builder.EnablePolyGroups();

				Builder.SetNumUVLayers(1);
				Builder.ReserveNewUVs(NumVertices, 0);

				Builder.ReserveNewVertices(NumVertices);
				
				for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
				{
					const FVertexID VertexID = Builder.AppendVertex(VertexPositions[VertexIndex]);
					const FVertexInstanceID VertexInstance = Builder.AppendInstance(VertexID);

					Builder.SetInstanceColor(VertexInstance, VertexColors[VertexIndex]);
					Builder.SetInstanceNormal(VertexInstance, VertexNormals[VertexIndex]);
					Builder.SetInstanceTangentSpace(VertexInstance, VertexNormals[VertexIndex], VertexTangents[VertexIndex], 1);
				}

				FPolygonGroupID GroupID = Builder.AppendPolygonGroup();

				for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
				{
					FVertexInstanceID Triangle[3];
					for (int32 Vertex = 0; Vertex < 3; ++Vertex)
					{
						Triangle[Vertex] = Indices[TriangleIndex * 3 + Vertex];
					}

					Builder.AppendTriangle(Triangle[0], Triangle[1], Triangle[2], GroupID);
				}

				Builder.ResumeMeshDescriptionIndexing();

				const uint32 SectionMeshDescriptionIndex = NumSectionMeshDescriptions.fetch_add(1);
				SectionMeshDescriptions[SectionMeshDescriptionIndex] = MoveTemp(MeshDescription);
			}
		});
	}
	else
	{
		SectionMeshDescriptions.EmplaceAt(0, ConvertDynamicMeshToMeshDescription(EditMesh));
		++NumSectionMeshDescriptions;
	}

	//SectionMeshDescriptions.SetNum(Algo::RemoveIf(SectionMeshDescriptions, [](const FMeshDescription& MeshDescription) { return MeshDescription.Vertices().Num() == 0; }));
	SectionMeshDescriptions.SetNum(NumSectionMeshDescriptions);
	QuadSectionPositions.SetNum(NumQuadSectionPositions);

	// Attempt to reuse any of the components passed in from the ReusableComponents list
	int32 NumComponentsRequired = SectionMeshDescriptions.Num() + QuadSectionPositions.Num();
	StaticMeshComponents.Reserve(NumComponentsRequired);
	for (int32 Index = 0; Index < FMath::Min(NumComponentsRequired, ReusableComponents.Num()); ++Index)
	{
		StaticMeshComponents.Add(ReusableComponents[Index]);
	}
	// Create any needed new components
	AActor* ActorOwner = WaterBodyComponent->GetOwner();
	for (int32 Index = StaticMeshComponents.Num(); Index < NumComponentsRequired; ++Index)
	{
		const FName Name = MakeUniqueObjectName(ActorOwner, UWaterBodyStaticMeshComponent::StaticClass());
		UWaterBodyStaticMeshComponent* WaterBodyStaticMeshComponent = NewObject<UWaterBodyStaticMeshComponent>(ActorOwner, Name);

		WaterBodyStaticMeshComponent->RegisterComponent();
		WaterBodyStaticMeshComponent->SetMobility(WaterBodyComponent->Mobility);
		WaterBodyStaticMeshComponent->AttachToComponent(WaterBodyComponent, FAttachmentTransformRules::SnapToTargetNotIncludingScale);

		StaticMeshComponents.Add(WaterBodyStaticMeshComponent);
	}

	check(StaticMeshComponents.Num() == NumComponentsRequired);

	TArray<UStaticMesh*> StaticMeshes;
	StaticMeshes.Reserve(NumComponentsRequired);

	UMaterialInterface* WaterStaticMeshMID = WaterBodyComponent->GetWaterStaticMeshMaterialInstance();

	UObject* MeshComponentOuter = WaterBodyComponent->GetOwner();

	int32 StaticMeshComponentIndex = 0;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FWaterBodyMeshBuilder::SetupMeshComponents);

		for (int32 Index = 0; Index < SectionMeshDescriptions.Num(); ++Index)
		{
			UWaterBodyStaticMeshComponent* WaterBodyStaticMeshComponent = StaticMeshComponents[StaticMeshComponentIndex];
			const FMeshDescription& MeshDescription = SectionMeshDescriptions[StaticMeshComponentIndex];

			check(IsValid(WaterBodyStaticMeshComponent));

			FName WaterBodyMeshName = MakeUniqueObjectName(MeshComponentOuter, UStaticMesh::StaticClass(), TEXT("WaterBodyMesh"));
			UStaticMesh* StaticMesh = CreateUStaticMesh(MeshComponentOuter, WaterBodyMeshName);

			StaticMesh->AddMaterial(WaterStaticMeshMID);

			UpdateStaticMesh(StaticMesh, MeshDescription, false /*bIsConservativeRasterMesh*/);

			WaterBodyStaticMeshComponent->SetStaticMesh(StaticMesh);

			// Reset any location/scale offsets which might have been set if this component is being reused.
			WaterBodyStaticMeshComponent->SetRelativeTransform(FTransform::Identity);
			StaticMeshes.Add(StaticMesh);

			++StaticMeshComponentIndex;
		}
	}

	if (QuadSectionPositions.Num() > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FWaterBodyMeshBuilder::SetupQuadComponents);

		// Create a shared UStaticMesh for all the quad sections used by this water body
		FName WaterBodyMeshName = MakeUniqueObjectName(MeshComponentOuter, UStaticMesh::StaticClass(), TEXT("WaterBodyQuadMesh"));
		UStaticMesh* QuadStaticMesh = CreateUStaticMesh(MeshComponentOuter, WaterBodyMeshName);

		QuadStaticMesh->AddMaterial(WaterStaticMeshMID);

		FMeshDescription QuadMeshDescription;
		{
			TArray<FVector> VertexPositions = { {0., 0., 0.}, {1., 0., 0}, {1., 1., 0.}, {0., 1., 0.} };
			TArray<int32> Indices = { 0, 2, 1, 0, 3, 2 };
			const int32 NumVertices = VertexPositions.Num();
			const int32 NumTriangles = Indices.Num() / 3;

			FStaticMeshAttributes(QuadMeshDescription).Register();
			QuadMeshDescription.Empty();

			FMeshDescriptionBuilder Builder;
			Builder.SetMeshDescription(&QuadMeshDescription);
			Builder.SuspendMeshDescriptionIndexing();

			Builder.EnablePolyGroups();

			Builder.SetNumUVLayers(1);
			Builder.ReserveNewUVs(NumVertices, 0);

			Builder.ReserveNewVertices(NumVertices);
			
			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				const FVertexID VertexID = Builder.AppendVertex(VertexPositions[VertexIndex]);
				const FVertexInstanceID VertexInstance = Builder.AppendInstance(VertexID);

				Builder.SetInstanceColor(VertexInstance, FVector4f(0.));
				Builder.SetInstanceNormal(VertexInstance, FVector(0., 0., 1.));
				Builder.SetInstanceTangentSpace(VertexInstance, FVector(0., 0., 1.0), FVector(1.0, .0, .0), 1);
			}

			FPolygonGroupID GroupID = Builder.AppendPolygonGroup();

			for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
			{
				FVertexInstanceID Triangle[3];
				for (int32 Vertex = 0; Vertex < 3; ++Vertex)
				{
					Triangle[Vertex] = Indices[TriangleIndex * 3 + Vertex];
				}

				Builder.AppendTriangle(Triangle[0], Triangle[1], Triangle[2], GroupID);
			}

			Builder.ResumeMeshDescriptionIndexing();
		}

		StaticMeshes.Add(QuadStaticMesh);

		UpdateStaticMesh(QuadStaticMesh, QuadMeshDescription, false /*bIsConservativeRasterMesh*/);

		for (const FVector& QuadMeshPosition : QuadSectionPositions)
		{
			UWaterBodyStaticMeshComponent* WaterBodyStaticMeshComponent = StaticMeshComponents[StaticMeshComponentIndex];

			WaterBodyStaticMeshComponent->SetStaticMesh(QuadStaticMesh);
			WaterBodyStaticMeshComponent->SetRelativeLocation(QuadMeshPosition);
			WaterBodyStaticMeshComponent->SetRelativeScale3D(FVector(WaterMeshSectionSize, WaterMeshSectionSize, 1.0));

			++StaticMeshComponentIndex;
		}
	}

	check(SectionMeshDescriptions.Num() + QuadSectionPositions.Num() == StaticMeshComponents.Num());

	UStaticMesh::BatchBuild(StaticMeshes);

	return StaticMeshComponents;
}

void FWaterBodyMeshBuilder::GetDynamicMesh(const UWaterBodyComponent* WaterBodyComponent, UE::Geometry::FDynamicMesh3& InMesh, UE::Geometry::FDynamicMesh3* InDilatedMesh, bool bMakeWaterInfoMeshConservativeRasterCompatible) const
{
	using namespace UE::Geometry;

	for (FDynamicMesh3* Mesh : { &InMesh, InDilatedMesh })
	{
		if (Mesh)
		{
			Mesh->EnableAttributes();
			Mesh->Attributes()->EnablePrimaryColors();
			Mesh->Attributes()->EnableTangents();
		}
	}

	WaterBodyComponent->GenerateWaterBodyMesh(InMesh, InDilatedMesh);

	// Store positions of the previous and next vertex within each triangle in three UV channels.
	// This is needed for conservative rasterization of the mesh when building the GPU water quadtree.
	if (bMakeWaterInfoMeshConservativeRasterCompatible)
	{
		InMesh.Attributes()->SetNumUVLayers(3);
		FDynamicMeshUVOverlay* UVOverlay0 = InMesh.Attributes()->GetUVLayer(0);
		FDynamicMeshUVOverlay* UVOverlay1 = InMesh.Attributes()->GetUVLayer(1);
		FDynamicMeshUVOverlay* UVOverlay2 = InMesh.Attributes()->GetUVLayer(2);

		for (int TriangleID : InMesh.TriangleIndicesItr())
		{
			FVector3d TriangleVertices[3];
			InMesh.GetTriVertices(TriangleID, TriangleVertices[0], TriangleVertices[1], TriangleVertices[2]);

			FIndex3i UVTriangle0;
			FIndex3i UVTriangle1;
			FIndex3i UVTriangle2;
			for (int TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
			{
				const FVector3f PrevVertex = FVector3f(TriangleVertices[(TriangleVertexIndex + 2) % 3]);
				const FVector3f NextVertex = FVector3f(TriangleVertices[(TriangleVertexIndex + 1) % 3]);

				UVTriangle0[TriangleVertexIndex] = UVOverlay0->AppendElement(FVector2f(PrevVertex.X, PrevVertex.Y));
				UVTriangle1[TriangleVertexIndex] = UVOverlay1->AppendElement(FVector2f(PrevVertex.Z, NextVertex.X));
				UVTriangle2[TriangleVertexIndex] = UVOverlay2->AppendElement(FVector2f(NextVertex.Y, NextVertex.Z));
			}

			UVOverlay0->SetTriangle(TriangleID, UVTriangle0);
			UVOverlay1->SetTriangle(TriangleID, UVTriangle1);
			UVOverlay2->SetTriangle(TriangleID, UVTriangle2);
		}
	}

	for (FDynamicMesh3* Mesh : { &InMesh, InDilatedMesh })
	{
		if (Mesh)
		{
			FDynamicMeshNormalOverlay* NormalOverlay = Mesh->Attributes()->PrimaryNormals();
			FMeshNormals::InitializeOverlayToPerVertexNormals(NormalOverlay);
			FMeshTangentsf::ComputeDefaultOverlayTangents(*Mesh);
		}
	}
}

FMeshDescription FWaterBodyMeshBuilder::ConvertDynamicMeshToMeshDescription(const UE::Geometry::FDynamicMesh3& Mesh) const
{
	FMeshDescription Result;
	FStaticMeshAttributes StaticMeshAttributes(Result);
	StaticMeshAttributes.Register();

	FDynamicMeshToMeshDescription DynamicMeshToMeshDescription;
	DynamicMeshToMeshDescription.Convert(&Mesh, Result);

	return Result;
}

void FWaterBodyMeshBuilder::UpdateStaticMesh(UStaticMesh* WaterMesh, const FMeshDescription& MeshDescription, bool bIsConservativeRasterMesh) const
{
	if (!WaterMesh->IsSourceModelValid(0))
	{
		WaterMesh->AddSourceModel();
	}

	FStaticMeshSourceModel& SrcModel = WaterMesh->GetSourceModel(0);
	SrcModel.BuildSettings.bRecomputeNormals = true;
	SrcModel.BuildSettings.bRecomputeTangents = true;
	SrcModel.BuildSettings.bRemoveDegenerates = true;
	SrcModel.BuildSettings.bUseHighPrecisionTangentBasis = false;
	SrcModel.BuildSettings.bUseFullPrecisionUVs = bIsConservativeRasterMesh; // CR mesh stores vertex positions in the UVs and needs full 32bit precision
	SrcModel.BuildSettings.bGenerateLightmapUVs = false;
	WaterMesh->CreateMeshDescription(0, MeshDescription);

	UStaticMesh::FCommitMeshDescriptionParams CommitParams;
	CommitParams.bMarkPackageDirty = true;
	CommitParams.bUseHashAsGuid = true;
	WaterMesh->CommitMeshDescription(0,CommitParams);

	WaterMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);
}

UStaticMesh* FWaterBodyMeshBuilder::CreateUStaticMesh(UObject* Outer, FName MeshName) const
{
	UStaticMesh* StaticMesh = NewObject<UStaticMesh>(Outer, MeshName, RF_TextExportTransient | RF_NonPIEDuplicateTransient);

	// Disable navigation
	StaticMesh->MarkAsNotHavingNavigationData();

	// Disable ray tracing as we don't want water meshes to show up in LumenScene.
	StaticMesh->bSupportRayTracing = false;

	// Always call CreateBodySetup before attempting to modify it.  The BodySetup is normally created when the mesh data is built, but can be created ahead of time to set default data
	StaticMesh->CreateBodySetup();

	if (UBodySetup* BodySetup = StaticMesh->GetBodySetup())
	{
		BodySetup->DefaultInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		BodySetup->CollisionTraceFlag = CTF_UseSimpleAsComplex;
		// We won't ever enable collisions (since collisions are handled by the dedicated water body collision components), ensure we don't even cook or load any collision data on this mesh: 
		BodySetup->bNeverNeedsCookedCollisionData = true;
		BodySetup->bHasCookedCollisionData = false;
	}
	return StaticMesh;
}
#endif // WITH_EDITOR

