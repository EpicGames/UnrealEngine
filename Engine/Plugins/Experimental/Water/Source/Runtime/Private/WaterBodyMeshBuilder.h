// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshDescription.h"
#include "WaterBodyStaticMeshSettings.h"

class UWaterBodyComponent;
class UWaterBodyStaticMeshComponent;
class UWaterBodyInfoMeshComponent;
class UStaticMesh;

namespace UE::Geometry { class FDynamicMesh3; }

#if WITH_EDITOR
class FWaterBodyMeshBuilder
{
public:
	/** 
	 * Builds and initializes the UWaterBodyStaticMeshComponents for a WaterBodyComponent to be able to render itself without relying on the water info texture or water mesh.
	 * The function will attempt to reuse static mesh components passed in via the ReusableComponents array.
	 * Note: components generated by this process will be attached and registered to the UWaterBodyComponent passed in.
	 */
	TArray<TObjectPtr<UWaterBodyStaticMeshComponent>> BuildWaterBodyStaticMesh(UWaterBodyComponent* WaterBodyComponent, const FWaterBodyStaticMeshSettings& Settings, const TArray<TObjectPtr<UWaterBodyStaticMeshComponent>>& ReusableComponents) const;

	/** 
	 * Builds and assigns static meshes for the WaterBody Info mesh and Dilated Mesh directly to the components passed. Setting bMakeWaterInfoMeshConservativeRasterCompatible to true stores positions of the previous and next vertex within each triangle in three UV channels
	 * of the water info mesh. Such a setup is needed when using conservative rasterization to create the GPU water quadtree. This effectively makes every vertex unique, increasing the number of vertices in the buffer and making the index buffer useless.
	 */
	void BuildWaterInfoMeshes(UWaterBodyComponent* WaterBodyComponent, UWaterBodyInfoMeshComponent* WaterBodyInfoMesh, UWaterBodyInfoMeshComponent* WaterBodyDilatedMesh, bool bMakeWaterInfoMeshConservativeRasterCompatible) const;

	/**
	 * Builds a mesh description out of the water body geometry.
	 */
	FMeshDescription BuildMeshDescription(const UWaterBodyComponent* WaterBodyComponent) const;
private:
	/** Helper function to retrieve the dynamic mesh representations of a water body component with all the vertex attributes initialized. Optionally does the same for the DilatedMesh if the parameter is non-null */
	void GetDynamicMesh(const UWaterBodyComponent* WaterBodyComponent, UE::Geometry::FDynamicMesh3& Mesh, UE::Geometry::FDynamicMesh3* DilateMesh = nullptr, bool bMakeWaterInfoMeshConservativeRasterCompatible = false) const;
	
	FMeshDescription ConvertDynamicMeshToMeshDescription(const UE::Geometry::FDynamicMesh3& Mesh) const;

	void UpdateStaticMesh(UStaticMesh* Mesh, const FMeshDescription& MeshDescription, bool bIsConservativeRasterMesh) const;

	UStaticMesh* CreateUStaticMesh(UObject* Outer, FName MeshName) const;
};
#endif // WITH_EDITOR
