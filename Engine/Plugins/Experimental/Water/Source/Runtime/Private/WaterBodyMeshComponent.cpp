// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyMeshComponent.h"
#include "WaterBodyActor.h"
#include "WaterBodyComponent.h"
#include "DynamicMeshToMeshDescription.h"
#include "Engine/CollisionProfile.h"
#include "PhysicsEngine/BodySetup.h"
#include "WaterModule.h"
#include "StaticMeshResources.h"
#include "Engine/StaticMesh.h"

#if WITH_EDITOR
#include "UObject/Package.h"
#include "StaticMeshAttributes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshToMeshDescription.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTangents.h"
#endif 

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyMeshComponent)

UWaterBodyMeshComponent::UWaterBodyMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetMobility(EComponentMobility::Static);

	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	SetGenerateOverlapEvents(false);
	SetCollisionEnabled(ECollisionEnabled::NoCollision);

	CanCharacterStepUpOn = ECanBeCharacterBase::ECB_No;
	DepthPriorityGroup = SDPG_World;

	bComputeFastLocalBounds = true;
	bComputeBoundsOnceForGame = true;

	AlwaysLoadOnServer = false;
	bSelectable = true;

	bCanEverAffectNavigation = false;
}

bool UWaterBodyMeshComponent::CanCreateSceneProxy() const
{
	if (GetStaticMesh() == nullptr)
	{
		UE_LOGF(LogWater, Verbose, "Skipping CreateSceneProxy for WaterBodyMeshComponent %ls (StaticMesh is null)", *GetFullName());
		return false;
	}

	// Prevent accessing the RenderData during async compilation. The RenderState will be recreated when compilation finishes.
	if (GetStaticMesh()->IsCompiling())
	{
		UE_LOGF(LogWater, Verbose, "Skipping CreateSceneProxy for WaterBodyMeshComponent %ls (StaticMesh is not ready)", *GetFullName());
		return false;
	}

	if (GetStaticMesh()->GetRenderData() == nullptr)
	{
		UE_LOGF(LogWater, Verbose, "Skipping CreateSceneProxy for WaterBodyMeshComponent %ls (RenderData is null)", *GetFullName());
		return false;
	}

	// By now the compilation should be finished so having null render data is not valid.
	if (!GetStaticMesh()->GetRenderData()->IsInitialized())
	{
		UE_LOGF(LogWater, Warning, "Skipping CreateSceneProxy for WaterBodyMeshComponent %ls (RenderData is not initialized)", *GetFullName());
		return false;
	}

	const FStaticMeshLODResourcesArray& LODResources = GetStaticMesh()->GetRenderData()->LODResources;
	const int32 SMCurrentMinLOD = GetStaticMesh()->GetMinLODIdx();
	const int32 EffectiveMinLOD = bOverrideMinLOD ? MinLOD : SMCurrentMinLOD;
	if (LODResources.Num() == 0	|| LODResources[FMath::Clamp<int32>(EffectiveMinLOD, 0, LODResources.Num()-1)].VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() == 0)
	{
		UE_LOGF(LogWater, Warning, "Skipping CreateSceneProxy for WaterBodyMeshComponent %ls (LOD problems)", *GetFullName());
		return false;
	}

	return true;
}


#if WITH_EDITOR
void UWaterBodyMeshComponent::PostLoad()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WaterBodyMeshComponent::PostLoad);

	Super::PostLoad();

	// If the static mesh is currently compiling, we should not call GetBodySetup.
	// This would block the async compilation immediately after it starts for each water body mesh and slows down editor load.
	UStaticMesh* Mesh = GetStaticMesh();
	if (Mesh && !Mesh->IsCompiling())
	{
		FixupCollisionOnBodySetup();
	}
}

void UWaterBodyMeshComponent::PostStaticMeshCompilation()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WaterBodyMeshComponent::PostStaticMeshCompilation);

	Super::PostStaticMeshCompilation();

	FixupCollisionOnBodySetup();
}

void UWaterBodyMeshComponent::FixupCollisionOnBodySetup()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WaterBodyMeshComponent::FixupCollisionOnBodySetup);

	// We had a bug where the body setup was not created with bNeverNeedsCookedCollisionData,
	// and another where bNeverNeedsCookedCollisionData was allowing meshes to generate cooked data.
	// This attempts to clean up the resulting mess.
	UStaticMesh* Mesh = GetStaticMesh();
	if (!Mesh)
	{
		return;
	}

	UBodySetup* BodySetup = GetBodySetup();

	if (BodySetup && BodySetup->bNeverNeedsCookedCollisionData)
	{
		// We could still have cooked collision data attached, but shouldn't.
		// Quietly remove the cooked collision data without invalidating the GUID,
		// because the latter leads to cook non-determinism for affected instances.
		//
		// This is effectively just stripping cached data, and referring back to
		// any cooked collision data in the DDC under the same GUID is totally
		// valid if there were no other modifications that actually truly
		// invalidate physics data.
		FGuid OriginalBodySetupGuid = BodySetup->BodySetupGuid;
		BodySetup->InvalidatePhysicsData();
		BodySetup->BodySetupGuid = OriginalBodySetupGuid;
	}
	else if (!BodySetup || !BodySetup->bNeverNeedsCookedCollisionData)
	{
		// We don't have a body setup, or it's set up the wrong
		// way. Create a new one.
		Mesh->CreateBodySetup();
		BodySetup = GetBodySetup();
		BodySetup->bNeverNeedsCookedCollisionData = true;
		BodySetup->bHasCookedCollisionData = false;
		BodySetup->InvalidatePhysicsData();
	}
}
#endif // WITH_EDITOR

