// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyOceanActor.h"
#include "WaterBodyOceanComponent.h"
#include "OceanCollisionComponent.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyOceanActor)

#if WITH_EDITOR
#endif

// ----------------------------------------------------------------------------------

AWaterBodyOcean::AWaterBodyOcean(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WaterBodyType = EWaterBodyType::Ocean;

#if WITH_EDITORONLY_DATA
	CollisionExtents_DEPRECATED = FVector(50000.f, 50000.f, 10000.f);
	bIsSpatiallyLoaded = false;
#endif // WITH_EDITORONLY_DATA
}

void AWaterBodyOcean::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterBodyComponentRefactor)
	{
		UWaterBodyOceanComponent* OceanComponent = CastChecked<UWaterBodyOceanComponent>(WaterBodyComponent);
		OceanComponent->CollisionExtents = CollisionExtents_DEPRECATED;
		if (OceanGenerator_DEPRECATED)
		{
			OceanComponent->CollisionBoxes = MoveTemp(OceanGenerator_DEPRECATED->CollisionBoxes);
			for (UOceanBoxCollisionComponent* CollisionComponent : OceanComponent->CollisionBoxes)
			{
				if (CollisionComponent)
				{
					CollisionComponent->SetupAttachment(OceanComponent);
				}
			}
			OceanComponent->CollisionHullSets = MoveTemp(OceanGenerator_DEPRECATED->CollisionHullSets);
			for (UOceanCollisionComponent* CollisionComponent : OceanComponent->CollisionHullSets)
			{
				if (CollisionComponent)
				{
					CollisionComponent->SetupAttachment(OceanComponent);
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

UDEPRECATED_OceanGenerator::UDEPRECATED_OceanGenerator(const FObjectInitializer& Initializer) : Super(Initializer) {}
