// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyOceanComponent.h"
#include "DynamicMesh/InfoTypes.h"
#include "WaterModule.h"
#include "WaterSplineComponent.h"
#include "PhysicsEngine/ConvexElem.h"
#include "OceanCollisionComponent.h"
#include "WaterBooleanUtils.h"
#include "WaterSubsystem.h"
#include "ConstrainedDelaunay2.h"
#include "Operations/InsetMeshRegion.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Curve/PolygonOffsetUtils.h"
#include "Engine/World.h"
#include "UObject/UObjectGlobalsInternal.h"

#if WITH_EDITOR
#include "Misc/UObjectToken.h"
#include "WaterRuntimeSettings.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyOceanComponent)

#define LOCTEXT_NAMESPACE "Water"

// ----------------------------------------------------------------------------------

namespace
{
	static TAutoConsoleVariable<bool> CVarEnableDeterministicOceanCollisionComponentNames(
		TEXT("r.Water.EnableDeterministicOceanCollisionComponentNames"),
		true,
		TEXT("When enabled, old ocean collision components are trashed and new components are generated with deterministic, consistent object names. This is required for them to be net adressable"),
		ECVF_Default);
}

UWaterBodyOceanComponent::UWaterBodyOceanComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CollisionExtents = FVector(50000.f, 50000.f, 10000.f);
	OceanExtents = FVector2D(51200., 51200.);

	// @todo_water : Remove these checks (Once AWaterBody is no more Blueprintable, these methods should become PURE_VIRTUAL and this class should overload them)
	check(IsFlatSurface());
	check(IsWaterSplineClosedLoop());
	check(IsHeightOffsetSupported());
}

TArray<UPrimitiveComponent*> UWaterBodyOceanComponent::GetCollisionComponents(bool bInOnlyEnabledComponents) const
{
	TArray<UPrimitiveComponent*> Result;
	Result.Reserve(CollisionBoxes.Num() + CollisionHullSets.Num());

	Algo::TransformIf(CollisionBoxes, Result,
		[bInOnlyEnabledComponents](UOceanBoxCollisionComponent* Comp) { return ((Comp != nullptr) && (!bInOnlyEnabledComponents || (Comp->GetCollisionEnabled() != ECollisionEnabled::NoCollision))); },
		[](UOceanBoxCollisionComponent* Comp) { return Comp; });

	Algo::TransformIf(CollisionHullSets, Result,
		[bInOnlyEnabledComponents](UOceanCollisionComponent* Comp) { return ((Comp != nullptr) && (!bInOnlyEnabledComponents || (Comp->GetCollisionEnabled() != ECollisionEnabled::NoCollision))); },
		[](UOceanCollisionComponent* Comp) { return Comp; });

	return Result;
}

void UWaterBodyOceanComponent::SetHeightOffset(float InHeightOffset)
{
	const float ClampedHeightOffset = FMath::Max(0.0f, InHeightOffset);

	if (HeightOffset != ClampedHeightOffset)
	{
		HeightOffset = ClampedHeightOffset;

		// the physics volume needs to be adjusted : 
		FOnWaterBodyChangedParams Params;
		Params.bShapeOrPositionChanged = true;
		OnWaterBodyChanged(Params);
	}
}

void UWaterBodyOceanComponent::BeginUpdateWaterBody()
{
	Super::BeginUpdateWaterBody();

	// Update WaterSubsystem's OceanActor
	if (UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(GetWorld()))
	{
		WaterSubsystem->SetOceanBodyComponent(this);
	}
}

#if WITH_EDITOR
void UWaterBodyOceanComponent::SetCollisionExtents(const FVector& NewExtents)
{
	CollisionExtents = NewExtents;

	FOnWaterBodyChangedParams Params;
	Params.bShapeOrPositionChanged = true;
	Params.bWeightmapSettingsChanged = false;
	UpdateAll(Params);
}

void UWaterBodyOceanComponent::SetOceanExtent(const FVector2D& NewExtents)
{
	OceanExtents = NewExtents;

	UpdateWaterBodyRenderData();
}

void UWaterBodyOceanComponent::FillWaterZoneWithOcean()
{
	if (const AWaterZone* WaterZone = GetWaterZone())
	{
		OceanExtents = WaterZone->GetZoneExtent();
		UpdateWaterBodyRenderData();
	}
}

void UWaterBodyOceanComponent::OnPostEditChangeProperty(FOnWaterBodyChangedParams& InOutOnWaterBodyChangedParams)
{
	Super::OnPostEditChangeProperty(InOutOnWaterBodyChangedParams);

	const FPropertyChangedEvent& PropertyChangedEvent = InOutOnWaterBodyChangedParams.PropertyChangedEvent;
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UWaterBodyOceanComponent, CollisionExtents))
	{
		// Affects the physics shape
		InOutOnWaterBodyChangedParams.bShapeOrPositionChanged = true;
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UWaterBodyOceanComponent, OceanExtents)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UWaterBodyOceanComponent, SavedZoneLocation))
	{
		InOutOnWaterBodyChangedParams.bShapeOrPositionChanged = true;
	}
}

const TCHAR* UWaterBodyOceanComponent::GetWaterSpriteTextureName() const
{
	return TEXT("/Water/Icons/WaterBodyOceanSprite");
}

#endif

// Adds a quad to a dynamic mesh reusing passed vertex indices if they are not INDEX_NONE
static void AddAABBQuadToDynamicMesh(UE::Geometry::FDynamicMesh3& EditMesh, FVector3d Min, FVector3d Max, TArray<int32, TInlineAllocator<4>>& InOutVertices)
{
	using namespace UE::Geometry;

	int32 VertexAIndex = InOutVertices[0];
	int32 VertexBIndex = InOutVertices[1];
	int32 VertexCIndex = InOutVertices[2];
	int32 VertexDIndex = InOutVertices[3];

	FDynamicMeshColorOverlay* ColorOverlay = EditMesh.Attributes()->PrimaryColors();
	FDynamicMeshNormalOverlay* NormalOverlay = EditMesh.Attributes()->PrimaryNormals();

	if (VertexAIndex == INDEX_NONE)
	{ 
		FVertexInfo VertexA;
		VertexA.Position = Min;
		VertexAIndex = EditMesh.AppendVertex(VertexA);
		ColorOverlay->AppendElement(FVector4f(0.0));
		NormalOverlay->AppendElement(FVector3f(0., 0., 1.));
	}
	
	if (VertexBIndex == INDEX_NONE)
	{
		FVertexInfo VertexB;
		VertexB.Position = FVector3d(Max.X, Min.Y, 0);
		VertexBIndex = EditMesh.AppendVertex(VertexB);
		ColorOverlay->AppendElement(FVector4f(0.0));
		NormalOverlay->AppendElement(FVector3f(0., 0., 1.));
	}

	if (VertexCIndex == INDEX_NONE)
	{
		FVertexInfo VertexC;
		VertexC.Position = Max;
		VertexCIndex = EditMesh.AppendVertex(VertexC);
		ColorOverlay->AppendElement(FVector4f(0.0));
		NormalOverlay->AppendElement(FVector3f(0., 0., 1.));
	}

	if (VertexDIndex == INDEX_NONE)
	{
		FVertexInfo VertexD;
		VertexD.Position = FVector3d(Min.X, Max.Y, 0);
		VertexDIndex = EditMesh.AppendVertex(VertexD);
		ColorOverlay->AppendElement(FVector4f(0.0));
		NormalOverlay->AppendElement(FVector3f(0., 0., 1.));
	}

	// Only add triangles for this quad if the min/max are actually the min/max. This inversion could happen if
	// the island spline extends outside the ocean extent thereby flipping the corner positions around.
	if (Min.X < Max.X && Min.Y < Max.Y)
	{
		const int TriOne = EditMesh.AppendTriangle(VertexAIndex, VertexDIndex, VertexBIndex);
		ColorOverlay->SetTriangle(TriOne, FIndex3i(VertexAIndex, VertexDIndex, VertexBIndex));
		const int TriTwo = EditMesh.AppendTriangle(VertexBIndex, VertexDIndex, VertexCIndex);
		ColorOverlay->SetTriangle(TriTwo, FIndex3i(VertexBIndex, VertexDIndex, VertexCIndex));
	}

	InOutVertices = {VertexAIndex, VertexBIndex, VertexCIndex, VertexDIndex};
}

bool UWaterBodyOceanComponent::GenerateWaterBodyMesh(UE::Geometry::FDynamicMesh3& OutMesh, UE::Geometry::FDynamicMesh3* OutDilatedMesh) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GenerateOceanMesh);

	using namespace UE::Geometry;

	const UWaterSplineComponent* SplineComp = GetWaterSpline();
	const FVector OceanLocation = GetComponentLocation();
	
	if (SplineComp->GetNumberOfSplineSegments() < 3)
	{
		return false;
	}

	FPolygon2d Island;
	TArray<FVector> PolyLineVertices;
	SplineComp->ConvertSplineToPolyLine(ESplineCoordinateSpace::Local, FMath::Square(10.f), PolyLineVertices);

	FAxisAlignedBox2d IslandBounds = FAxisAlignedBox2d(FVector2d(0.f), FVector2d(0.f));
	// Construct a 2D polygon describing the central island
	for (int32 i = PolyLineVertices.Num() - 2; i >= 0; --i) // skip the last vertex since it's the same as the first vertex
	{
		Island.AppendVertex(FVector2D(PolyLineVertices[i]));
		IslandBounds.Contain(FVector2d(PolyLineVertices[i]));
	}
	// Expand the island slightly so we aren't intersecting with the spline
	IslandBounds.Expand(1);

	const FVector2D OceanExtentScaled = OceanExtents / FVector2D(GetComponentScale());

	const FBox OceanBounds3d = CalcBounds(FTransform::Identity).GetBox();
	const FAxisAlignedBox2d OceanBounds = FAxisAlignedBox2d(FVector2d(OceanBounds3d.Min), FVector2d(OceanBounds3d.Max));
	FPolygon2d IslandBoundingPolygon = FPolygon2d::MakeRectangle(IslandBounds.Center(), IslandBounds.Extents().X * 2., IslandBounds.Extents().Y * 2.);

	FConstrainedDelaunay2d Triangulation;
	Triangulation.FillRule = FConstrainedDelaunay2d::EFillRule::Positive;
	Triangulation.Add(IslandBoundingPolygon);

	if (!Island.IsClockwise())
	{
		Island.Reverse();
	}
	Triangulation.Add(Island);
	if (!Triangulation.Triangulate())
	{
		UE_LOGF(LogWater, Warning, "Failed to triangulate Ocean mesh for %ls. Ensure that the Ocean's spline does not form any loops.", *GetOwner()->GetActorNameOrLabel());
	}

	if (Triangulation.Triangles.Num() == 0)
	{
		return false;
	}
	
	int32 IslandBottomLeft = INDEX_NONE;
	FVector3d IslandBottomLeftVertex = FVector3d::ZeroVector;

	int32 IslandBottomRight = INDEX_NONE;
	FVector3d IslandBottomRightVertex = FVector3d::ZeroVector;
	
	int32 IslandTopRight = INDEX_NONE;
	FVector3d IslandTopRightVertex = FVector3d::ZeroVector;

	int32 IslandTopLeft = INDEX_NONE;
	FVector3d IslandTopLeftVertex = FVector3d::ZeroVector;

	check(OutMesh.Attributes());
	FDynamicMeshColorOverlay* ColorOverlay = OutMesh.Attributes()->PrimaryColors();
	FDynamicMeshNormalOverlay* NormalOverlay = OutMesh.Attributes()->PrimaryNormals();

	for (const FVector2d& Vertex : Triangulation.Vertices)
	{
		const FVertexInfo VertexInfo(FVector3d(Vertex, 0.f));

		const int32 Index = OutMesh.AppendVertex(VertexInfo);
		ColorOverlay->AppendElement(FVector4f(0.0));
		NormalOverlay->AppendElement(FVector3f(0., 0., 1.));

		// Collect the corner vertices of the island bounding box so we can stitch the outer quads to them.
		if ((IslandBottomLeft == INDEX_NONE) || (VertexInfo.Position.X < IslandBottomLeftVertex.X || VertexInfo.Position.Y < IslandBottomLeftVertex.Y))
		{
			IslandBottomLeft = Index;
			IslandBottomLeftVertex = VertexInfo.Position;
		}
		if ((IslandBottomRight == INDEX_NONE) || (VertexInfo.Position.X > IslandBottomRightVertex.X || VertexInfo.Position.Y < IslandBottomRightVertex.Y))
		{
			IslandBottomRight = Index;
			IslandBottomRightVertex = VertexInfo.Position;
		}
		if ((IslandTopRight == INDEX_NONE) || (VertexInfo.Position.X > IslandTopRightVertex.X || VertexInfo.Position.Y > IslandTopRightVertex.Y))
		{
			IslandTopRight = Index;
			IslandTopRightVertex = VertexInfo.Position;
		}
		if ((IslandTopLeft == INDEX_NONE) || (VertexInfo.Position.X < IslandTopLeftVertex.X || VertexInfo.Position.Y > IslandTopLeftVertex.Y))
		{
			IslandTopLeft = Index;
			IslandTopLeftVertex = VertexInfo.Position;
		}
	}
	check(IslandBottomLeft != INDEX_NONE &&
		  IslandBottomRight != INDEX_NONE &&
		  IslandTopRight != INDEX_NONE &&
		  IslandTopLeft != INDEX_NONE);

	for (const FIndex3i& Triangle : Triangulation.Triangles)
	{
		const int TriangleID = OutMesh.AppendTriangle(Triangle);
		ColorOverlay->SetTriangle(TriangleID, Triangle);
		NormalOverlay->SetTriangle(TriangleID, Triangle);
	}

	// Add the bounding quads:

	// Each quad has four vertices laid out like so
	// D --- C
	// |  \  |
	// A --- B
	{
		TArray<int32, TInlineAllocator<4>> Vertices({ INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE });

		// Bottom left quad
		Vertices[2] = IslandBottomLeft;
		AddAABBQuadToDynamicMesh(OutMesh, FVector3d(OceanBounds.Min, 0), FVector3d(IslandBounds.Min, 0), Vertices);
		// We need the B and C vertices of the first quad to attach to the last quad
		int32 FirstQuadB = Vertices[1];
		int32 FirstQuadC = Vertices[2];

		// Left middle quad
		Vertices[0] = Vertices[3];
		Vertices[1] = Vertices[2];
		Vertices[2] = IslandTopLeft;
		Vertices[3] = INDEX_NONE;
		AddAABBQuadToDynamicMesh(OutMesh, FVector3d(OceanBounds.Min.X, IslandBounds.Min.Y, 0), FVector3d(IslandBounds.Min.X, IslandBounds.Max.Y, 0), Vertices);

		// Top left quad
		Vertices[0] = Vertices[3];
		Vertices[1] = Vertices[2];
		Vertices[2] = INDEX_NONE;
		Vertices[3] = INDEX_NONE;
		AddAABBQuadToDynamicMesh(OutMesh, FVector3d(OceanBounds.Min.X, IslandBounds.Max.Y, 0), FVector3d(IslandBounds.Min.X, OceanBounds.Max.Y, 0), Vertices);

		// Top middle quad
		Vertices[0] = Vertices[1];
		Vertices[3] = Vertices[2];
		Vertices[1] = IslandTopRight;
		Vertices[2] = INDEX_NONE;
		AddAABBQuadToDynamicMesh(OutMesh, FVector3d(IslandBounds.Min.X, IslandBounds.Max.Y, 0), FVector3d(IslandBounds.Max.X, OceanBounds.Max.Y, 0), Vertices);

		// Top right quad
		Vertices[0] = Vertices[1];
		Vertices[3] = Vertices[2];
		Vertices[1] = INDEX_NONE;
		Vertices[2] = INDEX_NONE;
		AddAABBQuadToDynamicMesh(OutMesh, FVector3d(IslandBounds.Max, 0), FVector3d(OceanBounds.Max, 0), Vertices);

		// Middle right quad
		Vertices[3] = Vertices[0];
		Vertices[2] = Vertices[1];
		Vertices[1] = INDEX_NONE;
		Vertices[0] = IslandBottomRight;
		AddAABBQuadToDynamicMesh(OutMesh, FVector3d(IslandBounds.Max.X, IslandBounds.Min.Y, 0), FVector3d(OceanBounds.Max.X, IslandBounds.Max.Y, 0), Vertices);

		// Bottom right quad
		Vertices[3] = Vertices[0];
		Vertices[2] = Vertices[1];
		Vertices[1] = INDEX_NONE;
		Vertices[0] = INDEX_NONE;
		AddAABBQuadToDynamicMesh(OutMesh, FVector3d(IslandBounds.Max.X, OceanBounds.Min.Y, 0), FVector3d(OceanBounds.Max.X, IslandBounds.Min.Y, 0), Vertices);

		// Middle bottom quad
		Vertices[1] = Vertices[0];
		Vertices[2] = Vertices[3];
		Vertices[0] = FirstQuadB;
		Vertices[3] = FirstQuadC;
		AddAABBQuadToDynamicMesh(OutMesh, FVector3d(IslandBounds.Min.X, OceanBounds.Min.Y, 0), FVector3d(IslandBounds.Max.X, IslandBounds.Min.Y, 0), Vertices);
	}
	
	if (ShapeDilation > 0.f && OutDilatedMesh)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DilateOceanMesh);

		FPolygon2d OceanBoundingPoly;
		OceanBoundingPoly.AppendVertex(FVector2D(OceanBounds.Min));
		OceanBoundingPoly.AppendVertex(FVector2D(OceanBounds.Min.X, OceanBounds.Max.Y));
		OceanBoundingPoly.AppendVertex(FVector2D(OceanBounds.Max));
		OceanBoundingPoly.AppendVertex(FVector2D(OceanBounds.Max.X, OceanBounds.Min.Y));

		FGeneralPolygon2d OceanPoly(OceanBoundingPoly);
		if (Island.IsClockwise())
		{
			Island.Reverse();
		}
		OceanPoly.AddHole(Island);

		TArray<FGeneralPolygon2d> OffsetPolys;
		UE::Geometry::PolygonsOffset(
			ShapeDilation / 2.f,
			{ OceanPoly },
			OffsetPolys,
			false,
			2.0);

		FConstrainedDelaunay2d DilationTriangulation;
		DilationTriangulation.FillRule = FConstrainedDelaunay2d::EFillRule::Positive;

		for (const FGeneralPolygon2d& Poly : OffsetPolys)
		{
			if (Poly.SignedArea() <= 0.)
			{
				UE_LOGF(LogWater, Warning, "Failed to apply offset for shape dilation (%ls", *GetOwner()->GetActorNameOrLabel());
				continue;
			}

			DilationTriangulation.Add(Poly);
		}

		if (!DilationTriangulation.Triangulate())
		{
			UE_LOGF(LogWater, Warning, "Failed to triangulate dilated ocean mesh (%ls", *GetOwner()->GetActorNameOrLabel());
			return false;
		}

		if (DilationTriangulation.Triangles.Num() == 0)
		{
			return false;
		}

		FDynamicMeshColorOverlay* DilatedColorOverlay = OutDilatedMesh->Attributes()->PrimaryColors();
		FDynamicMeshNormalOverlay* DilatedNormalOverlay = OutDilatedMesh->Attributes()->PrimaryNormals();
		for (const FVector2d& Vertex : DilationTriangulation.Vertices)
		{
			FVertexInfo MeshVertex(FVector3d(Vertex.X, Vertex.Y, 0.));

			OutDilatedMesh->AppendVertex(MeshVertex);
			DilatedColorOverlay->AppendElement(FVector4f(0.f));
			DilatedNormalOverlay->AppendElement(FVector3f(0., 0., 1.));
		}

		for (const FIndex3i& Triangle : DilationTriangulation.Triangles)
		{
			const int TriangleID = OutDilatedMesh->AppendTriangle(Triangle);
			DilatedColorOverlay->SetTriangle(TriangleID, Triangle);
			DilatedNormalOverlay->SetTriangle(TriangleID, Triangle);
		}
	}

	return true;
}


void UWaterBodyOceanComponent::Reset()
{
	const bool bEnableDeterministicCollisionComponentNames = CVarEnableDeterministicOceanCollisionComponentNames.GetValueOnGameThread();
	for (UBoxComponent* Component : CollisionBoxes)
	{
		if (Component)
		{
			Component->DestroyComponent();
			if (bEnableDeterministicCollisionComponentNames)
			{
				// Trash the object so it gets renamed into a different package, freeing up its deterministic name.
				// This ensures that newly generated components always have a consistent name.
				TrashObject(Component);
			}
		}
	}
	CollisionBoxes.Reset();
	for (UOceanCollisionComponent* Component : CollisionHullSets)
	{
		if (Component)
		{
			Component->DestroyComponent();
			if (bEnableDeterministicCollisionComponentNames)
			{
				// Trash the object so it gets renamed into a different package, freeing up its deterministic name.
				// This ensures that newly generated components always have a consistent name.
				TrashObject(Component);
			}
		}
	}
	CollisionHullSets.Reset();
}

void UWaterBodyOceanComponent::PostLoad()
{
	Super::PostLoad();
}

void UWaterBodyOceanComponent::OnPostRegisterAllComponents()
{
	Super::OnPostRegisterAllComponents();

#if WITH_EDITOR
	// Only run the fixup code when the object was loaded. This is required as any newly created and not-yet-saved objects will have no linker and therefore return an invalid custom version.
	if (HasAnyFlags(RF_WasLoaded))
	{	
		// In this version the ocean was changed to not be strongly coupled to the water zone and instead rely on the user to keep the mesh extent in sync with the zone extent
		if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterBodyStaticMeshComponents)
		{
			// Trigger a rebuild of the render data manually since this doesn't occur on load anymore. 
			// The mesh extent will be out of sync for old oceans so ensure they rebuild now:
			FillWaterZoneWithOcean();
		}
	}
#endif // WITH_EDITOR
}
	
FBoxSphereBounds UWaterBodyOceanComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	const FTransform& ComponentTransform = GetComponentTransform();
	FVector RelativeLocationToZone = (bCenterOnWaterZone && !OwningWaterZone.IsNull()) ? ComponentTransform.InverseTransformPosition(FVector(SavedZoneLocation, 0.0)) : FVector::ZeroVector;
	RelativeLocationToZone.Z = 0;

	const FVector2D OceanExtentScaled = (OceanExtents / FVector2D(GetComponentScale())) / 2.;

	return FBoxSphereBounds(RelativeLocationToZone, FVector(OceanExtentScaled.X, OceanExtentScaled.Y, GetChannelDepth()), FMath::Max(OceanExtentScaled.X, OceanExtentScaled.Y)).TransformBy(LocalToWorld);
}

void UWaterBodyOceanComponent::OnPostActorCreated()
{
	Super::OnPostActorCreated();

#if WITH_EDITOR
	if (UWorld* World = GetWorld(); World && World->IsGameWorld() == false)
	{
		UpdateWaterZones();
		FillWaterZoneWithOcean();
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR

void UWaterBodyOceanComponent::OnWaterBodyRenderDataUpdated()
{
	Super::OnWaterBodyRenderDataUpdated();

	// Store the location of the zone when the ocean render data is created so we can validate this later and check for inconsistencies:
	if (const AWaterZone* WaterZone = GetWaterZone())
	{
		SavedZoneLocation = FVector2D(WaterZone->GetActorLocation());
	}
}

TArray<TSharedRef<FTokenizedMessage>> UWaterBodyOceanComponent::CheckWaterBodyStatus()
{
	TArray<TSharedRef<FTokenizedMessage>> Result = Super::CheckWaterBodyStatus();

	if (const AWaterZone* WaterZone = GetWaterZone())
	{
		auto DisableWarnOnMismatchExtent = []()
		{
			if (UWaterRuntimeSettings* WaterRuntimeSettings = GetMutableDefault<UWaterRuntimeSettings>())
			{
				WaterRuntimeSettings->SetShouldWarnOnMismatchOceanExtent(false);
			}
		};

		const UWaterRuntimeSettings* WaterRuntimeSettings = GetDefault<UWaterRuntimeSettings>();
		check(WaterRuntimeSettings);

		if (WaterRuntimeSettings->ShouldWarnOnMismatchOceanExtent())
		{
			if ((WaterZone->GetZoneExtent() != OceanExtents)
			 || (FVector2D(WaterZone->GetActorLocation()) != SavedZoneLocation))
			{
				Result.Add(FTokenizedMessage::Create(EMessageSeverity::Warning)
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(FText::Format(
						LOCTEXT("MapCheck_Message_MismatchedOceanExtent", "WaterBodyOcean ({0}) has a serialized mesh which does not match the WaterZone it belongs to ({1})."),
						FText::FromString(*GetNameSafe(this)),
						FText::FromString(*GetNameSafe(WaterZone)))))
					->AddToken(FActionToken::Create(LOCTEXT("MapCheck_MessageAction_SaveOcean", "Click here to fill the zone with ocean."), FText(),
						FOnActionTokenExecuted::CreateUObject(this, &UWaterBodyOceanComponent::FillWaterZoneWithOcean)))
					->AddToken(FActionToken::Create(LOCTEXT("MapCheck_MessageAction_DisableOceanZoneMismatchWarning", "If this is desired behavior, click here to disable this warning."), FText(),
						FOnActionTokenExecuted::CreateLambda(DisableWarnOnMismatchExtent)))
						);
			}
		}
	}
	return Result;
}

#endif // WITH_EDITOR

void UWaterBodyOceanComponent::OnUpdateBody(bool bWithExclusionVolumes)
{
	AActor* OwnerActor = GetOwner();
	check(OwnerActor);

	if (GetCollisionEnabled() != ECollisionEnabled::NoCollision)
	{
		FVector OceanCollisionExtents = GetCollisionExtents();
		OceanCollisionExtents.Z += CollisionHeightOffset / 2;

		// The volume's top is located on the ocean actor's height + the additional ocean level + the collision height offset: 
		// and the volume's bottom is deeper by a value == OceanCollisionExtents.Z :
		FVector OceanBoxLocation = FVector(0, 0, GetHeightOffset() - OceanCollisionExtents.Z + CollisionHeightOffset);
		// No matter the scale, OceanCollisionExtents is always specified in world-space : 
		FVector OceanBoxExtent = OceanCollisionExtents; 

		// get our box information and exclusion volumes
		FTransform ComponentTransform = GetComponentTransform();
		FBoxSphereBounds WorldBounds;
		WorldBounds.Origin = ComponentTransform.TransformPositionNoScale(OceanBoxLocation);
		WorldBounds.BoxExtent = OceanBoxExtent;
		TArray<AWaterBodyExclusionVolume*> Exclusions = bWithExclusionVolumes ? GetExclusionVolumes() : TArray<AWaterBodyExclusionVolume*>();

		// Calculate a set of boxes and meshes that are Difference(Box, Union(ExclusionVolumes))
		// Output is calculated in World space and then transformed into Actor space, ie by inverse of ActorTransform
		TArray<FBoxSphereBounds> Boxes;
		TArray<TArray<FKConvexElem>> ConvexSets;
		double WorldMeshBufferWidth = 1000.0;		// extra space left around exclusion meshes
		double WorldBoxOverlap = 10.0;				// output boxes overlap each other and meshes by this amount
		FWaterBooleanUtils::BuildOceanCollisionComponents(WorldBounds, ComponentTransform, Exclusions,
			Boxes, ConvexSets, WorldMeshBufferWidth, WorldBoxOverlap);

		// Don't delete components unless we have to : this generates determinism issues because UOceanCollisionComponent has a UBodySetup with a GUID :
		const bool bComponentCountEqual = CollisionBoxes.Num() == Boxes.Num() && CollisionHullSets.Num() == ConvexSets.Num();

		// Some assets have saved empty collision box entries, reset if any are invalid
		if (!bComponentCountEqual || CollisionBoxes.Contains(nullptr) || CollisionHullSets.Contains(nullptr))
		{
			Reset();
		}

		const bool bEnableDeterministicCollisionComponentNames = CVarEnableDeterministicOceanCollisionComponentNames.GetValueOnGameThread();

		// create the box components
		for (int32 i = 0; i < Boxes.Num(); ++i)
		{
			const FBoxSphereBounds& Box = Boxes[i];
			FName Name;
			if (bEnableDeterministicCollisionComponentNames)
			{
				Name = *FString::Printf(TEXT("OceanBoxCollisionComponent_%d"), i);
			}
			else
			{
				Name = MakeUniqueObjectName(OwnerActor, UOceanBoxCollisionComponent::StaticClass(), *FString::Printf(TEXT("OceanBoxCollisionComponent_%d"), i));
			}

			UOceanBoxCollisionComponent* BoxComponent = nullptr;
			if (CollisionBoxes.IsValidIndex(i) && (CollisionBoxes[i] != nullptr))
			{
				BoxComponent = CollisionBoxes[i];
			}
			else
			{
				BoxComponent = NewObject<UOceanBoxCollisionComponent>(OwnerActor, Name, RF_Transactional);
				BoxComponent->SetupAttachment(this);
				CollisionBoxes.Add(BoxComponent);
			}

			BoxComponent->SetNetAddressable(); // it's deterministically named so it's addressable over network (needed for collision)
			BoxComponent->bDrawOnlyIfSelected = true;
			BoxComponent->SetRelativeLocation(FVector::ZeroVector);

			CopySharedCollisionSettingsToComponent(BoxComponent);
			CopySharedNavigationSettingsToComponent(BoxComponent);

			FVector RelativePosition = Box.Origin;			// boxes are calculated in space of actor
			BoxComponent->SetRelativeLocation(RelativePosition);
			BoxComponent->SetBoxExtent(Box.BoxExtent);

			// Registers as the last step to reduce amount of updates required when modifying an already registered component
			if (!BoxComponent->IsRegistered())
			{
				BoxComponent->RegisterComponent();
			}
		}

		// create the convex-hull components
		for (int32 i = 0; i < ConvexSets.Num(); ++i)
		{
			const TArray<FKConvexElem>& ConvexSet = ConvexSets[i];
			FName Name;
			
			if (bEnableDeterministicCollisionComponentNames)
			{
				Name = *FString::Printf(TEXT("OceanCollisionComponent_%d"), i);
			}
			else
			{
				Name = MakeUniqueObjectName(OwnerActor, UOceanCollisionComponent::StaticClass(), *FString::Printf(TEXT("OceanCollisionComponent_%d"), i));
			}

			UOceanCollisionComponent* CollisionComponent = nullptr;
			if (CollisionHullSets.IsValidIndex(i) && (CollisionHullSets[i] != nullptr))
			{
				CollisionComponent = CollisionHullSets[i];
			}
			else
			{
				CollisionComponent = NewObject<UOceanCollisionComponent>(OwnerActor, Name, RF_Transactional);
				CollisionComponent->SetupAttachment(this);
				CollisionHullSets.Add(CollisionComponent);
			}

			CollisionComponent->SetNetAddressable(); // it's deterministically named so it's addressable over network (needed for collision)
			CollisionComponent->SetRelativeLocation(FVector::ZeroVector);

			CopySharedCollisionSettingsToComponent(CollisionComponent);
			CopySharedNavigationSettingsToComponent(CollisionComponent);

			CollisionComponent->InitializeFromConvexElements(ConvexSet);

			// Registers as the last step to reduce amount of updates required when modifying an already registered component
			if (!CollisionComponent->IsRegistered())
			{
				CollisionComponent->RegisterComponent();
			}
		}
	}
	else
	{
		// clear existing
		Reset();
	}
}

#undef LOCTEXT_NAMESPACE
