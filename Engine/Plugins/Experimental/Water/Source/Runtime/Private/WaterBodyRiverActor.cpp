// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyRiverActor.h"
#include "WaterBodyRiverComponent.h"
#include "Components/SplineMeshComponent.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyRiverActor)

#if WITH_EDITOR
#endif

// ----------------------------------------------------------------------------------

AWaterBodyRiver::AWaterBodyRiver(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WaterBodyType = EWaterBodyType::River;
}

void AWaterBodyRiver::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterBodyComponentRefactor)
	{
		UWaterBodyRiverComponent* RiverComponent = CastChecked<UWaterBodyRiverComponent>(WaterBodyComponent);
		RiverComponent->SetLakeTransitionMaterial(LakeTransitionMaterial_DEPRECATED);
		RiverComponent->SetOceanTransitionMaterial(OceanTransitionMaterial_DEPRECATED);
		if (RiverGenerator_DEPRECATED)
		{
			RiverComponent->SplineMeshComponents = MoveTemp(RiverGenerator_DEPRECATED->SplineMeshComponents);
			for (USplineMeshComponent* SplineMeshComponent : RiverComponent->SplineMeshComponents)
			{
				if (SplineMeshComponent)
				{
					SplineMeshComponent->SetupAttachment(RiverComponent);
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

// ----------------------------------------------------------------------------------

UDEPRECATED_RiverGenerator::UDEPRECATED_RiverGenerator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
