// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyRiverComponent.h"
#include "Chaos/CollisionConvexMesh.h"
#include "WaterSplineComponent.h"
#include "DynamicMesh/InfoTypes.h"
#include "WaterSubsystem.h"
#include "WaterUtils.h"
#include "Algo/ForEach.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Components/SplineMeshComponent.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "WaterBodyActor.h"

// for working around Chaos issue
#include "Chaos/Convex.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyRiverComponent)

// ----------------------------------------------------------------------------------

UWaterBodyRiverComponent::UWaterBodyRiverComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// @todo_water : Remove these checks (Once AWaterBody is no more Blueprintable, these methods should become PURE_VIRTUAL and this class should overload them)
	check(!IsFlatSurface());
	check(!IsWaterSplineClosedLoop());
	check(!IsHeightOffsetSupported());
}


TArray<UPrimitiveComponent*> UWaterBodyRiverComponent::GetCollisionComponents(bool bInOnlyEnabledComponents) const
{
	TArray<UPrimitiveComponent*> Result;
	Result.Reserve(SplineMeshComponents.Num());

	Algo::TransformIf(SplineMeshComponents, Result, 
		[bInOnlyEnabledComponents](USplineMeshComponent* SplineComp) { return ((SplineComp != nullptr) && (!bInOnlyEnabledComponents || (SplineComp->GetCollisionEnabled() != ECollisionEnabled::NoCollision))); }, 
		[](USplineMeshComponent* SplineComp) { return SplineComp; });

	return Result;
}

TArray<UPrimitiveComponent*> UWaterBodyRiverComponent::GetStandardRenderableComponents() const
{
	TArray<UPrimitiveComponent*> Result = Super::GetStandardRenderableComponents();
	Result.Reserve(Result.Num() + SplineMeshComponents.Num());

	Algo::CopyIf(SplineMeshComponents, Result, 
		[](USplineMeshComponent* SplineComp) { return (SplineComp != nullptr); });

	return Result;
}

void UWaterBodyRiverComponent::OnUpdateBody(bool bWithExclusionVolumes)
{
	if (const USplineComponent* WaterSpline = GetWaterSpline())
	{
		const int32 NumSplinePoints = WaterSpline->GetNumberOfSplinePoints();
		const int32 NumberOfMeshComponentsNeeded = WaterSpline->IsClosedLoop() ? NumSplinePoints : NumSplinePoints - 1;
		if (SplineMeshComponents.Num() != NumberOfMeshComponentsNeeded)
		{
			GenerateMeshes();
		}
		else
		{
			for (int32 SplinePtIndex = 0; SplinePtIndex < NumberOfMeshComponentsNeeded; ++SplinePtIndex)
			{
				USplineMeshComponent* MeshComp = SplineMeshComponents[SplinePtIndex];

				// Validate all mesh components.  Blueprints can null these out somehow
				if (MeshComp)
				{
					UpdateSplineMesh(MeshComp, SplinePtIndex);
					MeshComp->MarkRenderStateDirty();
				}
				else
				{
					GenerateMeshes();
					break;
				}
			}
		}
	}
}

// ----------------------------------------------------------------------------------

static void AddVerticesForRiverSplineStep(
	float DistanceAlongSpline, 
	const UWaterBodyRiverComponent* Component, 
	const UWaterSplineComponent* SplineComp, 
	const UWaterSplineMetadata* WaterSplineMetadata,
	UE::Geometry::FDynamicMesh3& OutMesh,
	UE::Geometry::FDynamicMesh3* OutDilatedMesh)
{
	using namespace UE::Geometry;
	check((Component != nullptr) && (SplineComp != nullptr) && (WaterSplineMetadata != nullptr));
	
	const FVector Tangent = SplineComp->GetTangentAtDistanceAlongSpline(DistanceAlongSpline, ESplineCoordinateSpace::Local).GetSafeNormal();
	const FVector Up = SplineComp->GetUpVectorAtDistanceAlongSpline(DistanceAlongSpline, ESplineCoordinateSpace::Local).GetSafeNormal();

	const FVector Normal = FVector::CrossProduct(Tangent, Up).GetSafeNormal();
	const FVector Pos = SplineComp->GetLocationAtDistanceAlongSpline(DistanceAlongSpline, ESplineCoordinateSpace::Local);

	const float Key = SplineComp->GetInputKeyValueAtDistanceAlongSpline(DistanceAlongSpline);
	const float HalfWidth = WaterSplineMetadata->RiverWidth.Eval(Key) / 2;
	float Velocity = WaterSplineMetadata->WaterVelocityScalar.Eval(Key);

	// Distance from the center of the spline to place our first vertices
	FVector OutwardDistance = Normal * HalfWidth;
	// Prevent there being a relative height difference between the two vertices even when the spline has a slight roll to it
	OutwardDistance.Z = 0.f;

	float FlowDirection = Tangent.HeadingAngle() + FMath::DegreesToRadians(Component->GetRelativeRotation().Yaw);

	// Restrict all angles between [0, 2 PI]. UnwindRadians returns a value between [-Pi, Pi] so we must remap again:
	FlowDirection = FMath::UnwindRadians(FlowDirection);
	if (FlowDirection < 0.f)
	{
		FlowDirection += TWO_PI;
	}

	// If negative velocity, inverse the direction and change the velocity back to positive.
	if (Velocity < 0.f)
	{
		Velocity *= -1.f;
		FlowDirection = FMath::Fmod(PI + FlowDirection, TWO_PI);
	}

	FVertexInfo Left(FVector3d(Pos - OutwardDistance));
	FVertexInfo Right(FVector3d(Pos + OutwardDistance));

	const FVector4f FlowData = FWaterUtils::PackFlowData(Velocity, FlowDirection);
	Left.Color = FlowData;
	Right.Color = FlowData;

	check(OutMesh.Attributes());
	FDynamicMeshColorOverlay* Colors = OutMesh.Attributes()->PrimaryColors();
	FDynamicMeshNormalOverlay* Normals = OutMesh.Attributes()->PrimaryNormals();
	// Append regular geometry to mesh:
	{
		/* Non - dilated river segment geometry:
		    0 --- 1
		    |  /  |
		   -2 --- -1
		*/
		const int32 BaseIndex = OutMesh.GetVerticesBuffer().Num();

		OutMesh.AppendVertex(Left);
		Colors->AppendElement(FlowData);
		Normals->AppendElement(FVector3f(0., 0., 1.));

		OutMesh.AppendVertex(Right);
		Colors->AppendElement(FlowData);
		Normals->AppendElement(FVector3f(0., 0., 1.));

		if (BaseIndex != 0)
		{
			int TriangleID1 = OutMesh.AppendTriangle(BaseIndex - 2, BaseIndex + 1, BaseIndex - 1);
			int TriangleID2 = OutMesh.AppendTriangle(BaseIndex - 2, BaseIndex, BaseIndex + 1);

			Colors->SetTriangle(TriangleID1, FIndex3i(BaseIndex - 2, BaseIndex + 1, BaseIndex - 1));
			Colors->SetTriangle(TriangleID2, FIndex3i(BaseIndex - 2, BaseIndex, BaseIndex + 1));
		}
	}

	const float DilationAmount = Component->ShapeDilation;
	// If dilation is required, append dilated geometry:
	if (DilationAmount > 0.f && OutDilatedMesh)
	{
		const FVector DilationOffset = Normal * DilationAmount;

		const FVertexInfo DilatedFarLeft(FVector3d(Pos - OutwardDistance - DilationOffset));
		const FVertexInfo DilatedLeft(FVector3d(Pos - OutwardDistance));
		const FVertexInfo DilatedRight(FVector3d(Pos + OutwardDistance));
		const FVertexInfo DilatedFarRight(FVector3d(Pos + OutwardDistance + DilationOffset));

		/* Dilated River segment geometry:
			 1 --- 2     3 --- 4
			 |  /  |      |  /  |
			-4 --- -3    -2 --- -1
		*/
		const int32 BaseIndex = OutDilatedMesh->GetVerticesBuffer().Num();

		OutDilatedMesh->AppendVertex(DilatedFarLeft);
		OutDilatedMesh->AppendVertex(DilatedLeft);
		OutDilatedMesh->AppendVertex(DilatedRight);
		OutDilatedMesh->AppendVertex(DilatedFarRight);

		// If this is the first point we need to add the triangles for the starting dilated center quad since that references vertices which were only just appended
		if (DistanceAlongSpline == 0.f)
		{
			OutDilatedMesh->AppendTriangle(BaseIndex - 3, BaseIndex + 2, BaseIndex - 2);
			OutDilatedMesh->AppendTriangle(BaseIndex - 3, BaseIndex + 1, BaseIndex + 2);
		}
		
		if (BaseIndex != 0)
		{
			// Append left dilation quad
			OutDilatedMesh->AppendTriangle(BaseIndex - 4, BaseIndex + 1, BaseIndex - 3);
			OutDilatedMesh->AppendTriangle(BaseIndex - 4, BaseIndex, BaseIndex + 1);
			// Append right dilation quad
			OutDilatedMesh->AppendTriangle(BaseIndex - 2, BaseIndex + 2, BaseIndex + 3);
			OutDilatedMesh->AppendTriangle(BaseIndex - 2, BaseIndex + 3, BaseIndex - 1);
		}
	}
}

enum class ERiverBoundaryEdge {
	Start,
	End,
};

static void AddTerminalVerticesForRiverSpline(
	ERiverBoundaryEdge Edge, 
	const UWaterBodyRiverComponent* Component,
	const UWaterSplineComponent* SplineComp, 
	const UWaterSplineMetadata* WaterSplineMetadata, 
	UE::Geometry::FDynamicMesh3& OutDilatedMesh)
{
	using namespace UE::Geometry;

	check((Component != nullptr) && (SplineComp != nullptr) && (WaterSplineMetadata != nullptr));

	const float DistanceAlongSpline = (Edge == ERiverBoundaryEdge::Start) ? 0.f : SplineComp->GetSplineLength();
	
	const FVector Tangent = SplineComp->GetTangentAtDistanceAlongSpline(DistanceAlongSpline, ESplineCoordinateSpace::Local).GetSafeNormal();
	const FVector Up = SplineComp->GetUpVectorAtDistanceAlongSpline(DistanceAlongSpline, ESplineCoordinateSpace::Local).GetSafeNormal();
	
	const FVector Normal = FVector::CrossProduct(Tangent, Up).GetSafeNormal();
	const FVector Pos = SplineComp->GetLocationAtDistanceAlongSpline(DistanceAlongSpline, ESplineCoordinateSpace::Local);
	
	const float Key = SplineComp->GetInputKeyValueAtDistanceAlongSpline(DistanceAlongSpline);
	const float HalfWidth = WaterSplineMetadata->RiverWidth.Eval(Key) / 2;

	const float DilationAmount = Component->ShapeDilation;
	const FVector DilationOffset = Normal * DilationAmount;
	FVector OutwardDistance = Normal * HalfWidth;
	OutwardDistance.Z = 0.f;

	FVector TangentialOffset = Tangent * DilationAmount;
	TangentialOffset.Z = 0.f;

	// For the starting edge the tangential offset is negative to push it backwards
	if (Edge == ERiverBoundaryEdge::Start)
	{
		TangentialOffset *= -1;
	}

	const FVertexInfo BackLeft(FVector3d(Pos - OutwardDistance + TangentialOffset - DilationOffset));
	const FVertexInfo Left(FVector3d(Pos - OutwardDistance + TangentialOffset));
	const FVertexInfo Right(FVector3d(Pos + OutwardDistance + TangentialOffset));
	const FVertexInfo BackRight(FVector3d(Pos + OutwardDistance + TangentialOffset + DilationOffset));

	const int32 BaseIndex = OutDilatedMesh.GetVerticesBuffer().Num();

	OutDilatedMesh.AppendVertex(BackLeft);
	OutDilatedMesh.AppendVertex(Left);
	OutDilatedMesh.AppendVertex(Right);
	OutDilatedMesh.AppendVertex(BackRight);


	if (Edge == ERiverBoundaryEdge::End)
	{
		/* Dilated edge segment geometry:
			 0 --- 1 ------- 2 --- 3
			 |  /  |    /    |  /  |
			-4 --- -3 ----- -2 --- -1
		*/

		OutDilatedMesh.AppendTriangle(BaseIndex - 4, BaseIndex + 0, BaseIndex + 1);
		OutDilatedMesh.AppendTriangle(BaseIndex - 4, BaseIndex + 1, BaseIndex - 3);

		OutDilatedMesh.AppendTriangle(BaseIndex - 3, BaseIndex + 1, BaseIndex + 2);
		OutDilatedMesh.AppendTriangle(BaseIndex - 3, BaseIndex + 2, BaseIndex - 2);

		OutDilatedMesh.AppendTriangle(BaseIndex - 2, BaseIndex + 2, BaseIndex + 3);
		OutDilatedMesh.AppendTriangle(BaseIndex - 2, BaseIndex + 3, BaseIndex - 1);
	}
}

// ----------------------------------------------------------------------------------

bool UWaterBodyRiverComponent::GenerateWaterBodyMesh(UE::Geometry::FDynamicMesh3& OutMesh, UE::Geometry::FDynamicMesh3* OutDilatedMesh) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GenerateRiverMesh);

	const UWaterSplineComponent* SplineComp = GetWaterSpline();

	check(GetWaterBodyActor());

	const UWaterSplineMetadata* SplineMetadata = GetWaterBodyActor()->GetWaterSplineMetadata();
	if ((SplineComp == nullptr) || (SplineMetadata == nullptr) || (SplineComp->GetNumberOfSplinePoints() < 2))
	{
		return false;
	}

	TArray<double> Distances;
	TArray<FVector> Points;
	SplineComp->ConvertSplineToPolyLineWithDistances(ESplineCoordinateSpace::Local, FMath::Square(5.f), Points, Distances);
	if (Distances.Num() == 0)
	{
		return false;
	}
	
	if (ShapeDilation > 0.f && OutDilatedMesh)
	{
		AddTerminalVerticesForRiverSpline(ERiverBoundaryEdge::Start, this, SplineComp, SplineMetadata, *OutDilatedMesh);
	}

	for (double DistanceAlongSpline : Distances)
	{
		AddVerticesForRiverSplineStep(DistanceAlongSpline, this, SplineComp, SplineMetadata, OutMesh, OutDilatedMesh);
	}
	
	if (ShapeDilation > 0.f && OutDilatedMesh)
	{
		AddTerminalVerticesForRiverSpline(ERiverBoundaryEdge::End, this, SplineComp, SplineMetadata, *OutDilatedMesh);
	}
	return true;
}

FBoxSphereBounds UWaterBodyRiverComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox BoxExtent(ForceInit);
	for (const USplineMeshComponent* SplineMeshComponent : SplineMeshComponents)
	{
		if (SplineMeshComponent != nullptr)
		{
			const FBox SplineMeshComponentBounds = SplineMeshComponent->CalcBounds(SplineMeshComponent->GetRelativeTransform()).GetBox();
			BoxExtent += SplineMeshComponentBounds;
		}
	}
	// Spline mesh components aren't storing our vertical bounds so account for that with the ChannelDepth parameter.
	BoxExtent.Max.Z += MaxWaveHeightOffset;
	BoxExtent.Min.Z -= GetChannelDepth();
	return FBoxSphereBounds(BoxExtent).TransformBy(LocalToWorld);
}

void UWaterBodyRiverComponent::UpdateMaterialInstances()
{
	CreateOrUpdateLakeTransitionMID();
	CreateOrUpdateOceanTransitionMID();

	// Must be called after the transition MIDs are created. Super::UpdateMaterialInstances will rebuild the water mesh if necessary to push new MIDs.
	Super::UpdateMaterialInstances();
}

void UWaterBodyRiverComponent::SetLakeTransitionMaterial(UMaterialInterface* InMaterial)
{
	LakeTransitionMaterial = InMaterial;
	UpdateMaterialInstances();
}

void UWaterBodyRiverComponent::SetOceanTransitionMaterial(UMaterialInterface* InMaterial)
{
	OceanTransitionMaterial = InMaterial;
	UpdateMaterialInstances();
}

void UWaterBodyRiverComponent::SetLakeAndOceanTransitionMaterials(UMaterialInterface* InLakeTransition, UMaterialInterface* InOceanTransition)
{
	const bool bUpdateInstances = LakeTransitionMaterial != InLakeTransition || OceanTransitionMaterial != InOceanTransition;

	LakeTransitionMaterial = InLakeTransition;
	OceanTransitionMaterial = InOceanTransition;

	if (bUpdateInstances)
	{
		UpdateMaterialInstances();
	}
}

float UWaterBodyRiverComponent::GetRiverWidthAtSplineInputKey(float InKey) const
{
	return WaterSplineMetadata ? WaterSplineMetadata->RiverWidth.Eval(InKey, 0.f) : 0.0f;
}

float UWaterBodyRiverComponent::GetRiverDepthAtSplineInputKey(float InKey) const
{
	return WaterSplineMetadata ? WaterSplineMetadata->Depth.Eval(InKey, 0.f) : 0.0f;
}

void UWaterBodyRiverComponent::SetRiverWidthAtSplineInputKey(float InKey, float InWidth)
{
	if (WaterSplineMetadata)
	{
		const int32 PointIndexForKey = WaterSplineMetadata->RiverWidth.GetPointIndexForInputValue(InKey);
		WaterSplineMetadata->RiverWidth.Points[PointIndexForKey].OutVal = InWidth;
	}
}

void UWaterBodyRiverComponent::SetRiverDepthAtSplineInputKey(float InKey, float InDepth)
{
	if (WaterSplineMetadata)
	{
		const int32 PointIndexForKey = WaterSplineMetadata->Depth.GetPointIndexForInputValue(InKey);
		WaterSplineMetadata->Depth.Points[PointIndexForKey].OutVal = InDepth;
	}
}

void UWaterBodyRiverComponent::Reset()
{
	for (USplineMeshComponent* Comp : SplineMeshComponents)
	{
		if (Comp)
		{
			Comp->DestroyComponent();
		}
	}
	SplineMeshComponents.Empty();
}

UMaterialInstanceDynamic* UWaterBodyRiverComponent::GetRiverToLakeTransitionMaterialInstance()
{
	CreateOrUpdateLakeTransitionMID();
	return LakeTransitionMID;
}

UMaterialInstanceDynamic* UWaterBodyRiverComponent::GetRiverToOceanTransitionMaterialInstance()
{
	CreateOrUpdateOceanTransitionMID();
	return OceanTransitionMID;
}

UMaterialInterface* UWaterBodyRiverComponent::GetRiverToLakeTransitionMaterial() const
{
	return LakeTransitionMaterial;
}

UMaterialInterface* UWaterBodyRiverComponent::GetRiverToOceanTransitionMaterial() const
{
	return OceanTransitionMaterial;
}

#if WITH_EDITOR
TArray<UPrimitiveComponent*> UWaterBodyRiverComponent::GetBrushRenderableComponents() const
{
	TArray<UPrimitiveComponent*> BrushRenderableComponents;
	
	BrushRenderableComponents.Reserve(SplineMeshComponents.Num());
	Algo::ForEach(SplineMeshComponents, [&BrushRenderableComponents](USplineMeshComponent* Comp)
	{
		if (Comp != nullptr)
		{
			BrushRenderableComponents.Add(StaticCast<UPrimitiveComponent*>(Comp));
		}
	});
	
	return BrushRenderableComponents;
}
#endif //WITH_EDITOR

void UWaterBodyRiverComponent::CreateOrUpdateLakeTransitionMID()
{
	if (GetWorld())
	{
		LakeTransitionMID = FWaterUtils::GetOrCreateTransientMID(LakeTransitionMID, TEXT("LakeTransitionMID"), LakeTransitionMaterial, GetTransientMIDFlags());

		SetDynamicParametersOnMID(LakeTransitionMID);
	}
}

void UWaterBodyRiverComponent::CreateOrUpdateOceanTransitionMID()
{
	if (GetWorld())
	{
		OceanTransitionMID = FWaterUtils::GetOrCreateTransientMID(OceanTransitionMID, TEXT("OceanTransitionMID"), OceanTransitionMaterial, GetTransientMIDFlags());

		SetDynamicParametersOnMID(OceanTransitionMID);
	}
}

void UWaterBodyRiverComponent::GenerateMeshes()
{
	Reset();

	AActor* Owner = GetOwner();
	check(Owner);
	
	if (const USplineComponent* WaterSpline = GetWaterSpline())
	{
		const int32 NumSplinePoints = WaterSpline->GetNumberOfSplinePoints();

		const int32 NumberOfMeshComponentsNeeded = WaterSpline->IsClosedLoop() ? NumSplinePoints : NumSplinePoints - 1;
		for (int32 SplinePtIndex = 0; SplinePtIndex < NumberOfMeshComponentsNeeded; ++SplinePtIndex)
		{
			FString Name = FString::Printf(TEXT("SplineMeshComponent_%d"), SplinePtIndex);
			USplineMeshComponent* MeshComp = NewObject<USplineMeshComponent>(Owner, *Name, RF_Transactional);
			MeshComp->SetNetAddressable(); // it's deterministically named so it's addressable over network (needed for collision)
			SplineMeshComponents.Add(MeshComp);
			MeshComp->SetMobility(Owner->GetRootComponent()->Mobility);
			MeshComp->SetupAttachment(this);
			if (GetWorld() && GetWorld()->bIsWorldInitialized)
			{
				MeshComp->RegisterComponent();
			}

			// Call UpdateSplineMesh after RegisterComponent so that physics state creation can happen (needs the component to be registered)
			UpdateSplineMesh(MeshComp, SplinePtIndex);
		}
	}
}

void UWaterBodyRiverComponent::UpdateSplineMesh(USplineMeshComponent* MeshComp, int32 SplinePointIndex)
{
	if (const USplineComponent* WaterSpline = GetWaterSpline())
	{
		const int32 NumSplinePoints = WaterSpline->GetNumberOfSplinePoints();

		const int32 StartSplinePointIndex = SplinePointIndex;
		const int32 StopSplinePointIndex = WaterSpline->IsClosedLoop() && StartSplinePointIndex == NumSplinePoints - 1 ? 0 : StartSplinePointIndex + 1;

		UStaticMesh* StaticMesh = GetWaterMeshOverride();
		if (StaticMesh == nullptr)
		{
			StaticMesh = UWaterSubsystem::StaticClass()->GetDefaultObject<UWaterSubsystem>()->DefaultRiverMesh;
		}
		check(StaticMesh != nullptr);
		MeshComp->SetStaticMesh(StaticMesh);
		MeshComp->SetMaterial(0, GetWaterMaterialInstance());

		CopySharedCollisionSettingsToComponent(MeshComp);
		CopySharedNavigationSettingsToComponent(MeshComp);
		MeshComp->SetCastShadow(false);

		const bool bUpdateMesh = false;
		const FVector StartScale = WaterSpline->GetScaleAtSplinePoint(StartSplinePointIndex);
		const FVector EndScale = WaterSpline->GetScaleAtSplinePoint(StopSplinePointIndex);

		// Scale the water mesh so that it is the size of the bounds
		FVector StaticMeshExtent = 2.0f * StaticMesh->GetBounds().BoxExtent;
		StaticMeshExtent.X = FMath::Max(StaticMeshExtent.X, 0.0001f);
		StaticMeshExtent.Y = FMath::Max(StaticMeshExtent.Y, 0.0001f);
		StaticMeshExtent.Z = 1.0f;

		MeshComp->SetStartScale(FVector2D(StartScale / StaticMeshExtent), bUpdateMesh);
		MeshComp->SetEndScale(FVector2D(EndScale / StaticMeshExtent), bUpdateMesh);

		FVector StartPos, StartTangent;
		WaterSpline->GetLocationAndTangentAtSplinePoint(StartSplinePointIndex, StartPos, StartTangent, ESplineCoordinateSpace::Local);

		FVector EndPos, EndTangent;
		WaterSpline->GetLocationAndTangentAtSplinePoint(StopSplinePointIndex, EndPos, EndTangent, ESplineCoordinateSpace::Local);

		MeshComp->SetStartAndEnd(StartPos, StartTangent, EndPos, EndTangent, bUpdateMesh);

		MeshComp->UpdateMesh();

		//
		// GROSS HACK to work around the issue above that CreatePhysicsMeshes() doesn't currently work at Runtime.
		// The "cook" for a FKConvexElem just creates and caches a Chaos::FConvex instance, and the restore from cooked
		// data just restores that and passes it to the FKConvexElem. So we're just going to do that ourselves.
		// Code below is taken from FChaosDerivedDataCooker::BuildConvexMeshes() 
		//
		if (MeshComp->GetBodySetup())
		{
			for (FKConvexElem& Elem : MeshComp->GetBodySetup()->AggGeom.ConvexElems)
			{
				const int32 NumHullVerts = Elem.VertexData.Num();
				TArray<Chaos::FConvex::FVec3Type> ConvexVertices;
				ConvexVertices.SetNum(NumHullVerts);
				for (int32 VertIndex = 0; VertIndex < NumHullVerts; ++VertIndex)
				{
					ConvexVertices[VertIndex] = Elem.VertexData[VertIndex];
				}

				Chaos::FConvexPtr ChaosConvex( new Chaos::FConvex(ConvexVertices, 0.0f));

				Elem.SetConvexMeshObject(MoveTemp(ChaosConvex));
			}

			MeshComp->RecreatePhysicsState();
		}
	}
}

#if WITH_EDITOR
void UWaterBodyRiverComponent::OnPostEditChangeProperty(FOnWaterBodyChangedParams& InOutOnWaterBodyChangedParams)
{
	Super::OnPostEditChangeProperty(InOutOnWaterBodyChangedParams);

	if (InOutOnWaterBodyChangedParams.PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UWaterBodyRiverComponent, LakeTransitionMaterial) ||
		InOutOnWaterBodyChangedParams.PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UWaterBodyRiverComponent, OceanTransitionMaterial))
	{
		UpdateMaterialInstances();
	}
}
const TCHAR* UWaterBodyRiverComponent::GetWaterSpriteTextureName() const
{
	return TEXT("/Water/Icons/WaterBodyRiverSprite");
}
#endif
