// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/StaticMeshComponent.h"
#include "WaterBodyMeshComponent.h"

#include "WaterBodyStaticMeshComponent.generated.h"

class UWaterBodyComponent;

/**
* WaterBodyMeshComponent used to when rendering water statically without relying on the dynamic tessellation of the water zone/water mesh.
*/
UCLASS(MinimalAPI)
class UWaterBodyStaticMeshComponent : public UWaterBodyMeshComponent
{
	GENERATED_BODY()
public:
};
