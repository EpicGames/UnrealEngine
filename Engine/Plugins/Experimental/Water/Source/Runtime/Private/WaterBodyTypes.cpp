// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBodyTypes.h"
#include "GerstnerWaveEvaluation.h"
#include "Engine/World.h"
#include "LandscapeInfo.h"
#include "LandscapeProxy.h"
#include "WaterBodyComponent.h"
#include "WaterSplineComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBodyTypes)

float FWaterBodyQueryResult::LazilyComputeSplineKey(const UWaterBodyComponent& InWaterBodyComponent, const FVector& InWorldLocation)
{
	// only compute if not done (or set) before : 
	if (!SplineInputKey.IsSet())
	{
		SplineInputKey = TOptional<float>(InWaterBodyComponent.FindInputKeyClosestToWorldLocation(InWorldLocation));
	}
	return SplineInputKey.GetValue();
}

float FWaterBodyQueryResult::LazilyComputeSplineKey(const FWaterSplineDataPhysics& InWaterSpline, const FVector& InWorldLocation)
{
	// only compute if not done (or set) before : 
	if (!SplineInputKey.IsSet())
	{
		SplineInputKey = TOptional<float>(InWaterSpline.FindInputKeyClosestToWorldLocation(InWorldLocation));
	}
	return SplineInputKey.GetValue();
}

float FWaterSplineDataPhysics::FindInputKeyClosestToWorldLocation(const FVector& WorldLocation) const
{
	const FVector LocalLocation = ComponentTransform.InverseTransformPosition(WorldLocation);
	float Dummy;
	return SplineCurves.Position.FindNearest(LocalLocation, Dummy);
}

FVector FWaterSplineDataPhysics::GetUpVectorAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace /*= ESplineCoordinateSpace::World*/) const
{
	const FQuat Quat = GetQuaternionAtSplineInputKey(InKey, ESplineCoordinateSpace::Local);
	FVector UpVector = Quat.RotateVector(FVector::UpVector);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		UpVector = ComponentTransform.TransformVectorNoScale(UpVector);
	}

	return UpVector;
}

FQuat FWaterSplineDataPhysics::GetQuaternionAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace /*= ESplineCoordinateSpace::World*/) const
{
	FQuat Quat = SplineCurves.Rotation.Eval(InKey, FQuat::Identity);
	Quat.Normalize();

	const FVector Direction = SplineCurves.Position.EvalDerivative(InKey, FVector::ZeroVector).GetSafeNormal();
	const FVector UpVector = Quat.RotateVector(DefaultUpVector);

	FQuat Rot = (FRotationMatrix::MakeFromXZ(Direction, UpVector)).ToQuat();

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Rot = ComponentTransform.GetRotation() * Rot;
	}

	return Rot;
}

FVector FWaterSplineDataPhysics::GetDirectionAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace /*= ESplineCoordinateSpace::World*/) const
{
	FVector Direction = SplineCurves.Position.EvalDerivative(InKey, FVector::ZeroVector).GetSafeNormal();

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Direction = ComponentTransform.TransformVector(Direction);
		Direction.Normalize();
	}

	return Direction;
}

FVector FWaterSplineDataPhysics::GetLocationAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace /*= ESplineCoordinateSpace::World*/) const
{
	FVector Location = SplineCurves.Position.Eval(InKey, FVector::ZeroVector);

	if (CoordinateSpace == ESplineCoordinateSpace::World)
	{
		Location = ComponentTransform.TransformPosition(Location);
	}

	return Location;
}

FWaterSplineDataPhysics::FWaterSplineDataPhysics(UWaterSplineComponent* WaterSplineComponent)
{
	if (WaterSplineComponent)
	{
		ComponentTransform = WaterSplineComponent->GetComponentTransform();
		SplineCurves = WaterSplineComponent->GetSplineCurves();
		DefaultUpVector = WaterSplineComponent->GetDefaultUpVector(ESplineCoordinateSpace::World);
	}
}

FWaterSplineDataPhysics& FWaterSplineDataPhysics::operator=(const UWaterSplineComponent* WaterSplineComponent)
{
	if (WaterSplineComponent)
	{
		ComponentTransform = WaterSplineComponent->GetComponentTransform();
		SplineCurves = WaterSplineComponent->GetSplineCurves();
		DefaultUpVector = WaterSplineComponent->GetDefaultUpVector(ESplineCoordinateSpace::World);
	}
	return *this;
}

FSolverSafeWaterBodyData::FSolverSafeWaterBodyData(UWaterBodyComponent* WaterBodyComponent)
{
	if (WaterBodyComponent)
	{
		World = WaterBodyComponent->GetWorld();
		if (World)
		{
			if (ALandscapeProxy* LandscapeProxyActor = WaterBodyComponent->FindLandscape())
			{
				if (ULandscapeInfo* Info = LandscapeProxyActor->GetLandscapeInfo())
				{
					for (auto CollisionComponent : Info->XYtoCollisionComponentMap)
					{
						LandscapeCollisionComponents.Add(Cast<UPrimitiveComponent>(CollisionComponent.Value));
					}
				}
			}
			WaterSpline = WaterBodyComponent->GetWaterSpline();
			WaterSplineMetadata = WaterBodyComponent->GetWaterSplineMetadata();
			Location = WaterBodyComponent->GetComponentLocation();
			WaterBodyType = WaterBodyComponent->GetWaterBodyType();
			OceanHeightOffset = WaterBodyComponent->MaxWaveHeightOffset;
			if (const UGerstnerWaterWaves* WavesObject = Cast< UGerstnerWaterWaves>(WaterBodyComponent->GetWaterWaves()))
			{
				WaveParams = WavesObject->GetGerstnerWaves();
			}
			WaveSpeedFactor = 1.f;
			TargetWaveMaskDepth = WaterBodyComponent->TargetWaveMaskDepth;
			MaxWaveHeight = WaterBodyComponent->GetMaxWaveHeight();
			WaterBodyIndex = WaterBodyComponent->GetWaterBodyIndex();
		}
	}
}

FWaterBodyQueryResult FSolverSafeWaterBodyData::QueryWaterInfoClosestToWorldLocation(const FVector& InWorldLocation, EWaterBodyQueryFlags InQueryFlags, float InWaveReferenceTime, const TOptional<float>& InSplineInputKey /*= TOptional<float>()*/) const
{
	//SCOPE_CYCLE_COUNTER(STAT_WaterBody_ComputeWaterInfo);

	// Use the (optional) input spline input key if it has already been computed: 
	FWaterBodyQueryResult Result(InSplineInputKey);
	Result.SetQueryFlags(CheckAndAjustQueryFlags(InQueryFlags));

	//TODO : Get exclusion volumes to work

	//if (!EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::IgnoreExclusionVolumes))
	//{
	//	// No early-out, so that the requested information is still set. It is expected for the caller to check for IsInExclusionVolume() because technically, the returned information will be invalid :
	//	Result.SetIsInExclusionVolume(IsWorldLocationInExclusionVolume(InWorldLocation));
	//}

	// Lakes and oceans have surfaces aligned with the XY plane
	const bool bUseSplineHeight = (WaterBodyType == EWaterBodyType::River || WaterBodyType == EWaterBodyType::Transition);

	// Compute water plane location :
	if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeLocation))
	{
		FVector WaterPlaneLocation = InWorldLocation;
		// If in exclusion volume, force the water plane location at the query location. It is technically invalid, but it's up to the caller to check whether we're in an exclusion volume. 
		//  If the user fails to do so, at least it allows immersion depth to be 0.0f, which means the query location is NOT in water :
		if (!Result.IsInExclusionVolume())
		{
			if (bUseSplineHeight)
			{
				WaterPlaneLocation.Z = WaterSpline.GetLocationAtSplineInputKey(Result.LazilyComputeSplineKey(WaterSpline, InWorldLocation), ESplineCoordinateSpace::World).Z;
			}
			else if (WaterBodyType == EWaterBodyType::Ocean)
			{
				// For the ocean, part of the surface height is dynamic so we need to take that into account : 
				WaterPlaneLocation.Z = Location.Z + OceanHeightOffset;
			}
			else
			{
				WaterPlaneLocation.Z = Location.Z;
			}
		}

		Result.SetWaterPlaneLocation(WaterPlaneLocation);
		// When not including waves, water surface == water plane : 
		Result.SetWaterSurfaceLocation(WaterPlaneLocation);
	}

	// Compute water plane normal :
	FVector WaterPlaneNormal = FVector::UpVector;
	if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeNormal))
	{
		// Default to Z up for the normal
		if (bUseSplineHeight)
		{
			// For rivers default to using spline up vector to account for sloping rivers
			WaterPlaneNormal = WaterSpline.GetUpVectorAtSplineInputKey(Result.LazilyComputeSplineKey(WaterSpline, InWorldLocation), ESplineCoordinateSpace::World);
		}

		Result.SetWaterPlaneNormal(WaterPlaneNormal);
		// When not including waves, water surface == water plane : 
		Result.SetWaterSurfaceNormal(WaterPlaneNormal);
	}

	// Compute water plane depth : 
	float WaveAttenuationFactor = 1.0f;
	if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeDepth))
	{
		//SCOPE_CYCLE_COUNTER(STAT_WaterBody_ComputeWaterDepth);

		check(EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeLocation));
		float WaterPlaneDepth = 0.0f;

		// The better option for computing water depth for ocean and lake is landscape : 
		const bool bTryUseLandscape = (WaterBodyType == EWaterBodyType::Ocean || WaterBodyType == EWaterBodyType::Lake);
		if (bTryUseLandscape)
		{
			//TODO: Do a scene query to figure out landscape
			TOptional<float> LandscapeHeightOptional;
			{
				TArray<FHitResult> HitResults;
				FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(WaterImmersionDepthTrace), true);
				FCollisionObjectQueryParams ObjectQueryParams(FCollisionObjectQueryParams::InitType::AllStaticObjects);

				const FVector& StartTrace = InWorldLocation;
				const FVector EndTrace = InWorldLocation + FVector(0.f, 0.f, -100000.f);

				if (World->LineTraceMultiByObjectType(HitResults, StartTrace, EndTrace, ObjectQueryParams))
				{
					for (FHitResult& Hit : HitResults)
					{
						if (LandscapeCollisionComponents.Contains(Hit.GetComponent()))
						{
							LandscapeHeightOptional = Hit.ImpactPoint.Z;
							break;
						}
					}
				}
			}
			//FindLandscape();
			//if (ALandscapeProxy* LandscapePtr = Landscape.Get())
			//{
			//	SCOPE_CYCLE_COUNTER(STAT_WaterBody_ComputeLandscapeDepth);
			//	LandscapeHeightOptional = LandscapePtr->GetHeightAtLocation(InWorldLocation);
			//}

			bool bValidLandscapeData = LandscapeHeightOptional.IsSet();
			if (bValidLandscapeData)
			{
				WaterPlaneDepth = Result.GetWaterPlaneLocation().Z - LandscapeHeightOptional.GetValue();
				// Special case : cancel out waves for under-landcape ocean
				if ((WaterPlaneDepth < 0.0f) && (WaterBodyType == EWaterBodyType::Ocean))
				{
					WaveAttenuationFactor = 0.0f;
				}
			}

			// If the height is invalid, we either have invalid landscape data or we're under the landscape
			if (!bValidLandscapeData || (WaterPlaneDepth < 0.0f))
			{
				if (WaterBodyType == EWaterBodyType::Ocean)
				{
					// Fallback value when landscape is not found under the ocean water.
					WaterPlaneDepth = CVarWaterOceanFallbackDepth.GetValueOnAnyThread();
				}
				else
				{
					check(WaterBodyType == EWaterBodyType::Lake);
					// For an underwater lake, consider an uniform depth across the projection segment on the lake spline :
					WaterPlaneDepth = WaterSplineMetadata.Depth.Eval(Result.LazilyComputeSplineKey(WaterSpline, InWorldLocation), 0.f);
				}
			}
		}
		else
		{
			// For rivers and transitions, depth always come from the spline :
			WaterPlaneDepth = WaterSplineMetadata.Depth.Eval(Result.LazilyComputeSplineKey(WaterSpline, InWorldLocation), 0.f);
		}

		WaterPlaneDepth = FMath::Max(WaterPlaneDepth, 0.0f);
		Result.SetWaterPlaneDepth(WaterPlaneDepth);

		// When not including waves, water surface == water plane : 
		Result.SetWaterSurfaceDepth(WaterPlaneDepth);
	}

	// Optionally compute water surface location/normal/depth for waves : 
	if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::IncludeWaves) && WaterBodyTypeSupportsWaves())
	{
		//SCOPE_CYCLE_COUNTER(STAT_WaterBody_ComputeWaveHeight);
		FWaveInfo WaveInfo;

		if (!Result.IsInExclusionVolume())
		{
			WaveInfo.AttenuationFactor = WaveAttenuationFactor;
			WaveInfo.Normal = WaterPlaneNormal;
			const bool bSimpleWaves = EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::SimpleWaves);
			GetWaveInfoAtPosition(Result.GetWaterPlaneLocation(), Result.GetWaterSurfaceDepth(), InWaveReferenceTime, bSimpleWaves, WaveInfo);
		}

		Result.SetWaveInfo(WaveInfo);

		if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeLocation))
		{
			FVector WaterSurfaceLocation = Result.GetWaterSurfaceLocation();
			WaterSurfaceLocation.Z += WaveInfo.Height;
			Result.SetWaterSurfaceLocation(WaterSurfaceLocation);
		}

		if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeNormal))
		{
			Result.SetWaterSurfaceNormal(WaveInfo.Normal);
		}

		if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeDepth))
		{
			Result.SetWaterSurfaceDepth(Result.GetWaterSurfaceDepth() + WaveInfo.Height);
		}
	}

	if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeImmersionDepth))
	{
		check(EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeLocation));

		// Immersion depth indicates how much under the water surface is the world location. 
		//  therefore, it takes into account the waves if IncludeWaves is passed :
		Result.SetImmersionDepth(Result.GetWaterSurfaceLocation().Z - InWorldLocation.Z);
		// When in an exclusion volume, the queried location is considered out of water (immersion depth == 0.0f)
		check(!Result.IsInExclusionVolume() || (Result.GetImmersionDepth() == 0.0f));
	}

	// Compute velocity : 
	if (EnumHasAnyFlags(Result.GetQueryFlags(), EWaterBodyQueryFlags::ComputeVelocity))
	{
		FVector Velocity = FVector::ZeroVector;
		if (!Result.IsInExclusionVolume())
		{
			Velocity = GetWaterVelocityVectorAtSplineInputKey(Result.LazilyComputeSplineKey(WaterSpline, InWorldLocation));
		}

		Result.SetVelocity(Velocity);
	}

	return Result;
}

float FSolverSafeWaterBodyData::GetWaterVelocityAtSplineInputKey(float InKey) const
{
	return WaterSplineMetadata.WaterVelocityScalar.Eval(InKey, 0.f);
}

FVector FSolverSafeWaterBodyData::GetWaterVelocityVectorAtSplineInputKey(float InKey) const
{
	float WaterVelocityScalar = WaterSplineMetadata.WaterVelocityScalar.Eval(InKey, 0.f);
	const FVector SplineDirection = WaterSpline.GetDirectionAtSplineInputKey(InKey, ESplineCoordinateSpace::World);
	return SplineDirection * WaterVelocityScalar;
}

EWaterBodyQueryFlags FSolverSafeWaterBodyData::CheckAndAjustQueryFlags(EWaterBodyQueryFlags InQueryFlags) const
{
	EWaterBodyQueryFlags Result = InQueryFlags;

	// Waves only make sense for the following queries : 
	check(!EnumHasAnyFlags(Result, EWaterBodyQueryFlags::IncludeWaves)
		|| EnumHasAnyFlags(Result, EWaterBodyQueryFlags::ComputeLocation | EWaterBodyQueryFlags::ComputeNormal | EWaterBodyQueryFlags::ComputeDepth | EWaterBodyQueryFlags::ComputeImmersionDepth));

	// Simple waves only make sense when computing waves : 
	check(!EnumHasAnyFlags(Result, EWaterBodyQueryFlags::SimpleWaves)
		|| EnumHasAnyFlags(Result, EWaterBodyQueryFlags::IncludeWaves));

	if (EnumHasAnyFlags(InQueryFlags, EWaterBodyQueryFlags::ComputeDepth | EWaterBodyQueryFlags::ComputeImmersionDepth))
	{
		// We need location when querying depth : 
		Result |= EWaterBodyQueryFlags::ComputeLocation;
	}

	if (EnumHasAnyFlags(InQueryFlags, EWaterBodyQueryFlags::IncludeWaves) && WaterBodyTypeSupportsWaves())
	{
		// We need location and water depth when computing waves :
		Result |= EWaterBodyQueryFlags::ComputeLocation | EWaterBodyQueryFlags::ComputeDepth;
	}

	return Result;
}

float FSolverSafeWaterBodyData::GetWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime, FVector& OutNormal) const
{
	check(WaterBodyTypeSupportsWaves());
	return GerstnerWaveEvaluation::GetWaveHeightAtPosition(WaveParams, InPosition, InTime, OutNormal);
}

float FSolverSafeWaterBodyData::GetSimpleWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime) const
{
	check(WaterBodyTypeSupportsWaves());
	return GerstnerWaveEvaluation::GetSimpleWaveHeightAtPosition(WaveParams, InPosition, InTime);
}

float FSolverSafeWaterBodyData::GetWaveAttenuationFactor(const FVector& Position, float WaterDepth) const
{
	return GerstnerWaveEvaluation::GetWaveAttenuationFactor(WaterDepth, TargetWaveMaskDepth);
}

bool FSolverSafeWaterBodyData::GetWaveInfoAtPosition(const FVector& InPosition, float InWaterDepth, float InWaveReferenceTime, bool bInSimpleWaves, FWaveInfo& InOutWaveInfo) const
{
	if (!WaterBodyTypeSupportsWaves())
	{
		return false; //Collision needs to be fixed for rivers
	}

	InOutWaveInfo.ReferenceTime = InWaveReferenceTime;
	InOutWaveInfo.AttenuationFactor *= GetWaveAttenuationFactor(InPosition, InWaterDepth);

	// No need to perform computation if we're going to cancel it out afterwards :
	if (InOutWaveInfo.AttenuationFactor > 0.0f)
	{
		// Maximum amplitude that the wave can reach at this location : 
		InOutWaveInfo.MaxHeight = MaxWaveHeight * InOutWaveInfo.AttenuationFactor;

		float WaveHeight;
		if (bInSimpleWaves)
		{
			WaveHeight = GetSimpleWaveHeightAtPosition(InPosition, InWaterDepth, InOutWaveInfo.ReferenceTime);
		}
		else
		{
			FVector ComputedNormal;
			WaveHeight = GetWaveHeightAtPosition(InPosition, InWaterDepth, InOutWaveInfo.ReferenceTime, ComputedNormal);
			// Attenuate the normal :
			ComputedNormal = FMath::Lerp(InOutWaveInfo.Normal, ComputedNormal, InOutWaveInfo.AttenuationFactor);
			if (!ComputedNormal.IsZero())
			{
				InOutWaveInfo.Normal = ComputedNormal;
			}
		}

		// Attenuate the wave amplitude :
		InOutWaveInfo.Height = WaveHeight * InOutWaveInfo.AttenuationFactor;
	}

	return true;
}

FWaterSplineMetadataPhysics& FWaterSplineMetadataPhysics::operator=(const UWaterSplineMetadata* Metadata)
{
	if (Metadata)
	{
		Depth = Metadata->Depth;
		WaterVelocityScalar = Metadata->WaterVelocityScalar;
	}
	return *this;
}

