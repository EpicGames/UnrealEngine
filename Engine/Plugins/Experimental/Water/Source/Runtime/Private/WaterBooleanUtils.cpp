// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBooleanUtils.h"
#include "Generators/MinimalBoxMeshGenerator.h"
#include "Operations/MeshBoolean.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "Operations/MeshRegionOperator.h"
#include "Selections/MeshConnectedComponents.h"
#include "DynamicMesh/MeshTransforms.h"
#include "MeshSimplification.h"
#include "MeshQueries.h"
#include "CompGeom/PolygonTriangulation.h"
#include "MeshBoundaryLoops.h"
#include "Operations/MinimalHoleFiller.h"
#include "Operations/MeshPlaneCut.h"
#include "PhysicsEngine/ConvexElem.h"

#include "WaterBodyExclusionVolume.h"
#include "Model.h"
#include "WaterModule.h"

double FWaterBooleanUtils::UnitAspectRatio(const FVector3d& A, const FVector3d& B, const FVector3d& C)
{
	double AspectRatio = VectorUtil::AspectRatio(A, B, C);
	return (AspectRatio > 1.0) ? FMathd::Clamp(1.0 / AspectRatio, 0.0, 1.0) : AspectRatio;
}

double FWaterBooleanUtils::UnitAspectRatio(const FDynamicMesh3& Mesh, int32 TriangleID)
{
	FVector3d A, B, C;
	Mesh.GetTriVertices(TriangleID, A, B, C);
	return UnitAspectRatio(A, B, C);
}

void FWaterBooleanUtils::PlanarFlipsOptimization(FDynamicMesh3& Mesh, double PlanarDotThresh)
{
	struct FFlatEdge
	{
		int32 eid;
		double MinAspect;
	};

	TArray<double> AspectRatios;
	TArray<FVector3d> Normals;
	AspectRatios.SetNum(Mesh.MaxTriangleID());
	Normals.SetNum(Mesh.MaxTriangleID());
	for (int32 tid : Mesh.TriangleIndicesItr())
	{
		FVector3d A, B, C;
		Mesh.GetTriVertices(tid, A, B, C);
		AspectRatios[tid] = UnitAspectRatio(A, B, C);
		Normals[tid] = VectorUtil::Normal(A, B, C);
	}

	TArray<FFlatEdge> Flips;
	for (int32 eid : Mesh.EdgeIndicesItr())
	{
		if (Mesh.IsBoundaryEdge(eid) == false)
		{
			FIndex2i EdgeT = Mesh.GetEdgeT(eid);
			if (AspectRatios[EdgeT.A] < 0.01 && AspectRatios[EdgeT.B] < 0.01)
			{
				continue;		// if both are degenerate we can't fix by flipping edge between them
			}
			double MinAspect = FMathd::Min(AspectRatios[EdgeT.A], AspectRatios[EdgeT.B]);
			double NormDot = Normals[EdgeT.A].Dot(Normals[EdgeT.B]);
			if (NormDot > PlanarDotThresh)
			{
				Flips.Add({ eid, MinAspect });
			}
		}
	}

	Flips.Sort([&](const FFlatEdge& A, const FFlatEdge& B) { return A.MinAspect < B.MinAspect; });

	for (int32 k = 0; k < Flips.Num(); ++k)
	{
		int32 eid = Flips[k].eid;
		FIndex2i EdgeV = Mesh.GetEdgeV(eid);
		int32 a = EdgeV.A, b = EdgeV.B;
		FIndex2i EdgeT = Mesh.GetEdgeT(eid);
		FIndex3i Tri0 = Mesh.GetTriangle(EdgeT.A), Tri1 = Mesh.GetTriangle(EdgeT.B);
		int32 c = IndexUtil::OrientTriEdgeAndFindOtherVtx(a, b, Tri0);
		int32 d = IndexUtil::FindTriOtherVtx(a, b, Tri1);

		double AspectA = AspectRatios[EdgeT.A], AspectB = AspectRatios[EdgeT.B];
		double Metric = FMathd::Min(AspectA, AspectB);
		FVector3d Normal = (AspectA > AspectB) ? Normals[EdgeT.A] : Normals[EdgeT.B];

		FVector3d A = Mesh.GetVertex(a), B = Mesh.GetVertex(b);
		FVector3d C = Mesh.GetVertex(c), D = Mesh.GetVertex(d);

		double FlipAspect1 = UnitAspectRatio(C, D, B);
		double FlipAspect2 = UnitAspectRatio(D, C, A);
		FVector3d FlipNormal1 = VectorUtil::Normal(C, D, B);
		FVector3d FlipNormal2 = VectorUtil::Normal(D, C, A);
		if (FlipNormal1.Dot(Normal) < PlanarDotThresh || FlipNormal2.Dot(Normal) < PlanarDotThresh)
		{
			continue;		// should not happen?
		}

		if (FMathd::Min(FlipAspect1, FlipAspect2) > Metric)
		{
			FDynamicMesh3::FEdgeFlipInfo FlipInfo;
			if (Mesh.FlipEdge(eid, FlipInfo) == EMeshResult::Ok)
			{
				AspectRatios[EdgeT.A] = UnitAspectRatio(Mesh, EdgeT.A);
				AspectRatios[EdgeT.B] = UnitAspectRatio(Mesh, EdgeT.B);

				// safety check - if somehow we flipped the normal, flip it back
				bool bInvertedNormal = (Mesh.GetTriNormal(EdgeT.A).Dot(Normal) < PlanarDotThresh) ||
					(Mesh.GetTriNormal(EdgeT.B).Dot(Normal) < PlanarDotThresh);
				if (bInvertedNormal)
				{
					UE_LOGF(LogWater, Warning, "UE::Water::PlanarFlipsOptimization - Invalid Flip!");
					Mesh.FlipEdge(eid, FlipInfo);
					AspectRatios[EdgeT.A] = UnitAspectRatio(Mesh, EdgeT.A);
					AspectRatios[EdgeT.B] = UnitAspectRatio(Mesh, EdgeT.B);
				}
			}
		}
	}
}

void FWaterBooleanUtils::ExtractMesh(AVolume* Volume, FDynamicMesh3& Mesh)
{
	Mesh.DiscardAttributes();

	UModel* Model = Volume->Brush;
	FTransformSRT3d XForm(Volume->GetTransform());

	// Each "BspNode" is a planar polygon, triangulate each polygon and accumulate in a mesh.
	// Note that this does not make any attempt to weld vertices/edges
	for (const FBspNode& Node : Model->Nodes)
	{
		FVector3d Normal = (FVector3d)Node.Plane;
		FFrame3d Plane(Node.Plane.W * Normal, Normal);

		int32 NumVerts = (Node.NodeFlags & PF_TwoSided) ? Node.NumVertices / 2 : Node.NumVertices;  // ??
		if (NumVerts > 0)
		{
			TArray<int32> VertIndices;
			TArray<FVector2d> VertPositions2d;
			VertIndices.SetNum(NumVerts);
			VertPositions2d.SetNum(NumVerts);
			for (int32 VertexIndex = 0; VertexIndex < NumVerts; ++VertexIndex)
			{
				const FVert& Vert = Model->Verts[Node.iVertPool + VertexIndex];
				FVector3d Point = (FVector3d)Model->Points[Vert.pVertex];
				Point = XForm.TransformPosition(Point);
				VertIndices[VertexIndex] = Mesh.AppendVertex(Point);
				VertPositions2d[VertexIndex] = Plane.ToPlaneUV(Point, 2);
			}

			TArray<FIndex3i> PolyTriangles;
			PolygonTriangulation::TriangulateSimplePolygon(VertPositions2d, PolyTriangles);

			for (FIndex3i Tri : PolyTriangles)
			{
				Mesh.AppendTriangle(VertIndices[Tri.A], VertIndices[Tri.B], VertIndices[Tri.C]);
			}
		}
	}

	// Merge the mesh edges to create a closed solid
	double MinLen, MaxLen, AvgLen;
	TMeshQueries<FDynamicMesh3>::EdgeLengthStats(Mesh, MinLen, MaxLen, AvgLen);
	FMergeCoincidentMeshEdges Merge(&Mesh);
	Merge.MergeVertexTolerance = FMathd::Max(Merge.MergeVertexTolerance, MinLen * 0.1);
	Merge.Apply();

	// If the mesh is not closed, the merge failed or the volume had cracks/holes. 
	// Do trivial hole fills to ensure the output is solid   (really want autorepair here)
	if (Mesh.IsClosed() == false)
	{
		FMeshBoundaryLoops BoundaryLoops(&Mesh, true);
		for (FEdgeLoop& Loop : BoundaryLoops.Loops)
		{
			FMinimalHoleFiller Filler(&Mesh, Loop);
			Filler.Fill();
		}
	}

	// try to flip towards better triangles in planar areas, should reduce/remove degenerate geo
	for (int32 k = 0; k < 5; ++k)
	{
		PlanarFlipsOptimization(Mesh);
	}
}

void FWaterBooleanUtils::ExtractMesh(FBoxSphereBounds Bounds, FDynamicMesh3& Mesh)
{
	FMinimalBoxMeshGenerator MeshGen;
	FOrientedBox3d Box;
	Box.Frame = FFrame3d((FVector3d)Bounds.Origin);
	Box.Extents = (FVector3d)Bounds.BoxExtent;
	MeshGen.Box = Box;
	MeshGen.Generate();
	Mesh.Copy(&MeshGen);
	Mesh.DiscardAttributes();
}

void FWaterBooleanUtils::ExtractMesh(FAxisAlignedBox3d Bounds, FDynamicMesh3& Mesh)
{
	FMinimalBoxMeshGenerator MeshGen;
	FOrientedBox3d Box;
	Box.Frame = FFrame3d(Bounds.Center());
	Box.Extents = Bounds.Extents();
	MeshGen.Box = Box;
	MeshGen.Generate();
	Mesh.Copy(&MeshGen);
	Mesh.DiscardAttributes();
}

void FWaterBooleanUtils::ApplyBooleanRepairs(FDynamicMesh3& Mesh, double MergeTolerance)
{
	if (Mesh.IsClosed() == false)
	{
		// try to close any cracks
		FMergeCoincidentMeshEdges Merge(&Mesh);
		Merge.MergeVertexTolerance = MergeTolerance;
		Merge.Apply();

		// fill any holes
		FMeshBoundaryLoops BoundaryLoops(&Mesh, true);
		for (FEdgeLoop& Loop : BoundaryLoops.Loops)
		{
			FMinimalHoleFiller Filler(&Mesh, Loop);
			Filler.Fill();
		}
	}
}

void FWaterBooleanUtils::MakeNormalizationTransform(
	const FAxisAlignedBox3d& Bounds,
	FTransformSRT3d& ToNormalizedOut, FTransformSRT3d& FromNormalizedOut)
{
	double WorldSize = Bounds.MaxDim();
	FromNormalizedOut = FTransformSRT3d::Identity();
	FromNormalizedOut.SetTranslation(Bounds.Center());
	FromNormalizedOut.SetScale(WorldSize * FVector3d::One());
	ToNormalizedOut = FromNormalizedOut.InverseUnsafe(); // Ok to invert because there is no rotation
}

void FWaterBooleanUtils::SetToFaceNormals(FDynamicMesh3& Mesh)
{
	Mesh.EnableAttributes();
	FDynamicMeshNormalOverlay* Normals = Mesh.Attributes()->PrimaryNormals();
	for (int32 tid : Mesh.TriangleIndicesItr())
	{
		FIndex3i Tri = Mesh.GetTriangle(tid);
		FVector3d Normal = Mesh.GetTriNormal(tid);
		int32 e0 = Normals->AppendElement((FVector3f)Normal);
		int32 e1 = Normals->AppendElement((FVector3f)Normal);
		int32 e2 = Normals->AppendElement((FVector3f)Normal);
		Normals->SetTriangle(tid, FIndex3i(e0, e1, e2));
	}
}

FDynamicMesh3 FWaterBooleanUtils::AccumulateExtrusionVolumes(const TArray<AWaterBodyExclusionVolume*>& ExclusionVolumes, const FTransformSRT3d& Transform)
{
	// First step is to boolean-union all the volumes into this mesh
	FDynamicMesh3 CombinedVolumes;

	FTransformSRT3d IdentityTransform;
	for (AWaterBodyExclusionVolume* Volume : ExclusionVolumes)
	{
		// convert volume to mesh
		FDynamicMesh3 VolMesh(EMeshComponents::None);
		ExtractMesh(Volume, VolMesh);
		// transform from world to unit space
		MeshTransforms::ApplyTransform(VolMesh, Transform);

		// calculate the boolean
		FDynamicMesh3 BooleanResultMesh(EMeshComponents::None);
		FMeshBoolean Boolean(&CombinedVolumes, IdentityTransform, &VolMesh, IdentityTransform, &BooleanResultMesh, FMeshBoolean::EBooleanOp::Union);

		// if boolean fails, try filling holes
		bool bOK = Boolean.Compute();
		if (!bOK)
		{
			ApplyBooleanRepairs(BooleanResultMesh);
		}

		// currently the Boolean output has an arbitrary transform, apply that now
		MeshTransforms::ApplyTransform(BooleanResultMesh, Boolean.ResultTransform);
		// this is now our accumulated result for the next volume
		CombinedVolumes = MoveTemp(BooleanResultMesh);

		// collapse any unnecessary edge splits introduced by booleans
		FQEMSimplification Simplifier(&CombinedVolumes);
		Simplifier.SimplifyToMinimalPlanar(0.25);
	}

	return MoveTemp(CombinedVolumes);
}

FDynamicMesh3 FWaterBooleanUtils::MakeBoxCollisionMesh(FAxisAlignedBox3d WorldBoxBounds, const TArray<AWaterBodyExclusionVolume*>& ExclusionVolumes)
{
	// construct a transform that scales the input world-space box to unit bounds, to improve numerical precision
	FTransformSRT3d ToUnitTransform, FromUnitTransform;
	MakeNormalizationTransform(WorldBoxBounds, ToUnitTransform, FromUnitTransform);

	// First step is to boolean-union all the volumes into this mesh
	FDynamicMesh3 CombinedVolumes = AccumulateExtrusionVolumes(ExclusionVolumes, ToUnitTransform);

	// calculate edgelength stats on the combined volume, we will use this as a collapse tolerance below
	// until we have a better simplification option
	double MinLen, MaxLen, AvgLen;
	TMeshQueries<FDynamicMesh3>::EdgeLengthStats(CombinedVolumes, MinLen, MaxLen, AvgLen);

	// make a mesh for the world box
	FDynamicMesh3 BoxMesh(EMeshComponents::None);
	ExtractMesh(WorldBoxBounds, BoxMesh);
	MeshTransforms::ApplyTransform(BoxMesh, ToUnitTransform);

	// subtract combined volumes from box mesh
	FDynamicMesh3 FinalResultMesh(EMeshComponents::None);
	FTransform3d IdentityTransform;
	FMeshBoolean Subtract(&BoxMesh, IdentityTransform, &CombinedVolumes, IdentityTransform, &FinalResultMesh, FMeshBoolean::EBooleanOp::Difference);
	bool bOK = Subtract.Compute();
	if (!bOK)
	{
		ApplyBooleanRepairs(FinalResultMesh);
	}
	// apply boolean output transform
	MeshTransforms::ApplyTransform(FinalResultMesh, Subtract.ResultTransform);

	// simplification pass
	FQEMSimplification Simplifier(&FinalResultMesh);
	Simplifier.SimplifyToMinimalPlanar(0.25);

	// try to flip away degenerates and get better triangles overall
	for (int32 k = 0; k < 5; ++k)
	{
		PlanarFlipsOptimization(FinalResultMesh);
	}

	// transform normalized-space result back to world space
	MeshTransforms::ApplyTransform(FinalResultMesh, FromUnitTransform);

	return MoveTemp(FinalResultMesh);
}

FAxisAlignedBox3d FWaterBooleanUtils::GetBounds(const AWaterBodyExclusionVolume* Volume, double ExpansionSize)
{
	FBoxSphereBounds VolBoundsF = Volume->GetBounds();
	return FAxisAlignedBox3d(
		(FVector3d)VolBoundsF.Origin - (FVector3d)VolBoundsF.BoxExtent - ExpansionSize * FVector3d::One(),
		(FVector3d)VolBoundsF.Origin + (FVector3d)VolBoundsF.BoxExtent + ExpansionSize * FVector3d::One());
}

FAxisAlignedBox3d FWaterBooleanUtils::GetBounds(const TArray<AWaterBodyExclusionVolume*>& Volumes)
{
	FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();
	for (AWaterBodyExclusionVolume* Volume : Volumes)
	{
		Bounds.Contain(GetBounds(Volume));
	}
	return Bounds;
}

void FWaterBooleanUtils::ClusterExclusionVolumes(
	const TArray<AWaterBodyExclusionVolume*>& AllVolumes,
	TArray<TArray<AWaterBodyExclusionVolume*>>& VolumeClustersOut,
	double BoxExpandSize)
{
	int32 NumVolumes = AllVolumes.Num();
	if (NumVolumes == 1)
	{
		VolumeClustersOut.Add(AllVolumes);
		return;
	}

	// create initial cluster
	VolumeClustersOut.Add({ AllVolumes[0] });
	TArray<FAxisAlignedBox3d> ClusterBounds;
	ClusterBounds.Add(GetBounds(AllVolumes[0], BoxExpandSize));

	// accumulate each volume into existing or new clusters
	for (int32 k = 1; k < NumVolumes; ++k)
	{
		FAxisAlignedBox3d VolBounds = GetBounds(AllVolumes[k], BoxExpandSize);

		// try to find a cluster we already intersect with
		bool bIntersected = false;
		for (int32 ci = 0; ci < VolumeClustersOut.Num() && bIntersected == false; ++ci)
		{
			if (ClusterBounds[ci].Intersects(VolBounds))
			{
				ClusterBounds[ci].Contain(VolBounds);
				VolumeClustersOut[ci].Add(AllVolumes[k]);
				bIntersected = true;
			}
		}

		// if we did not find, make a new cluster
		if (!bIntersected)
		{
			VolumeClustersOut.Add({ AllVolumes[k] });
			ClusterBounds.Add(VolBounds);
		}
	}

	// now greedily merge overlapping clusters until none are overlapping anymore
	int32 MergeFrom = -1, MergeTo = -1;
	do
	{
		// Search for a valid cluster merge. Once we find the first merge, we exit the loop to do it
		MergeFrom = -1;
		for (int32 i = 0; i < VolumeClustersOut.Num() && MergeFrom < 0; ++i)
		{
			for (int32 j = i + 1; j < VolumeClustersOut.Num(); ++j)
			{
				if (ClusterBounds[i].Intersects(ClusterBounds[j]))
				{
					MergeFrom = j;
					MergeTo = i;
					break;
				}
			}
		}
		// if we found a merge, combine the two clusters
		if (MergeFrom >= 0)
		{
			TArray<AWaterBodyExclusionVolume*> Tmp = VolumeClustersOut[MergeFrom];
			for (int32 k = 0; k < Tmp.Num(); ++k)
			{
				VolumeClustersOut[MergeTo].Add(Tmp[k]);
			}
			ClusterBounds[MergeTo].Contain(ClusterBounds[MergeFrom]);

			VolumeClustersOut.RemoveAtSwap(MergeFrom, EAllowShrinking::No);
			ClusterBounds.RemoveAtSwap(MergeFrom, EAllowShrinking::No);
		}

	} while (MergeFrom >= 0);
}

void FWaterBooleanUtils::FindRemainingBoxSpaceXY(const FAxisAlignedBox3d& OuterBox, const FAxisAlignedBox3d& RemoveBox,
	TArray<FAxisAlignedBox3d>& BoxesOut)
{
	if (OuterBox.Intersects(RemoveBox) == false)
	{
		BoxesOut.Add(OuterBox);
		return;
	}

	// space on left and right sides
	FAxisAlignedBox3d LeftBox = OuterBox;
	LeftBox.Max.X = FMathd::Max(RemoveBox.Min.X, LeftBox.Min.X);
	FAxisAlignedBox3d RightBox = OuterBox;
	RightBox.Min.X = FMathd::Min(RemoveBox.Max.X, RightBox.Max.X);

	// strip in the middle
	FAxisAlignedBox3d MiddleBox = OuterBox;
	MiddleBox.Min.X = LeftBox.Max.X;
	MiddleBox.Max.X = RightBox.Min.X;

	// space on top and bottom inside middle strip
	FAxisAlignedBox3d TopBox = MiddleBox;
	TopBox.Min.Y = FMathd::Min(RemoveBox.Max.Y, MiddleBox.Max.Y);
	FAxisAlignedBox3d BottomBox = MiddleBox;
	BottomBox.Max.Y = FMathd::Max(RemoveBox.Min.Y, BottomBox.Min.Y);

	FAxisAlignedBox3d Boxes[4] = { LeftBox, RightBox, TopBox, BottomBox };
	for (int32 j = 0; j < 4; ++j)
	{
		checkSlow(OuterBox.Contains(Boxes[j]));
		if (Boxes[j].Volume() > 0)
		{
			BoxesOut.Add(Boxes[j]);
		}
	}
}

void FWaterBooleanUtils::FillEmptySpaceAroundBoxesXY(
	const FAxisAlignedBox3d& Box,
	const TArray<FAxisAlignedBox3d>& ContentBoxes,
	TArray<FAxisAlignedBox3d>& FillBoxesOut,
	double OverlapAmount)
{
	// Start with outer box
	TArray<FAxisAlignedBox3d> CurrentSet, NextSet;
	CurrentSet.Add(Box);
	// Incrementally subtract each content box from the remaining set of boxes.
	// Each subtraction produces up to 4 new boxes.
	for (FAxisAlignedBox3d ContentBox : ContentBoxes)
	{
		for (const FAxisAlignedBox3d& CurrentBox : CurrentSet)
		{
			FindRemainingBoxSpaceXY(CurrentBox, ContentBox, NextSet);
		}

		CurrentSet = NextSet;
		NextSet.Reset();
	}
	// add in overlap and return boxes
	for (FAxisAlignedBox3d CurrentBox : CurrentSet)
	{
		CurrentBox.Min.X -= OverlapAmount;
		CurrentBox.Min.Y -= OverlapAmount;
		CurrentBox.Max.X += OverlapAmount;
		CurrentBox.Max.Y += OverlapAmount;
		FillBoxesOut.Add(CurrentBox);
	}
}

bool FWaterBooleanUtils::IsConvex(const FPolygon2d& Polygon)
{
	const TArray<FVector2d>& Vertices = Polygon.GetVertices();
	int32 N = Vertices.Num();
	int32 CurIndex = 0;
	FLine2d CurLine;			// can possibly use (A,B) segments here and avoid all the normalization
	FVector2d CurLineEnd;
	bool bFoundStart = false;
	do {
		CurLineEnd = Vertices[CurIndex + 1];
		CurLine = FLine2d::FromPoints(Vertices[CurIndex], CurLineEnd);
		bFoundStart = (CurLine.Direction.SquaredLength() > 0.99);		// normalized
		CurIndex++;
	} while (bFoundStart == false && CurIndex < N - 1);

	if (CurIndex == N - 1)
	{
		return false;		// fully degenerate, treat as non-convex
	}

	int32 TurningSide = 0;

	CurIndex++;
	while (CurIndex <= N + 1)		// must test point 2 against line [N-1,0]
	{
		FVector2d Next = Vertices[CurIndex % N];
		int32 Side = CurLine.WhichSide(Next);
		if (Side != 0)
		{
			if (TurningSide == 0)
			{
				TurningSide = Side;		// initialize at first nonzero value
			}
			else if (Side * TurningSide < 0)
			{
				return false;	// if we are on "other" side compared to previous turns, poly is nonconvex
			}
		}
		FLine2d NextLine = FLine2d::FromPoints(CurLineEnd, Next);
		if (NextLine.Direction.SquaredLength() > 0.99)
		{
			CurLine = NextLine;		// only switch to next line if it is non-degenerate
			CurLineEnd = Next;
		}
		CurIndex++;
	}
	if (TurningSide == 0)		// polygon has collapsed to line...is this convex?
	{
		return true;
	}
	return true;
}

void FWaterBooleanUtils::MakeSweepConvex(const TArray<FVector3d>& VertexLoop, const FVector3d& SweepVector, FKConvexElem& ConvexOut)
{
	int32 N = VertexLoop.Num();
	ConvexOut.VertexData.SetNum(2 * N);
	for (int32 k = 0; k < N; ++k)
	{
		ConvexOut.VertexData[k] = (FVector)(VertexLoop[k]);
		ConvexOut.VertexData[N + k] = (FVector)(VertexLoop[k] + SweepVector);
	}
	// despite the name this actually computes the convex hull of the point set...
	ConvexOut.UpdateElemBox();
}

void FWaterBooleanUtils::MakeSweepConvex(const TArray<FVector3d>& VertexLoop, const FFrame3d& BasePlane, FKConvexElem& ConvexOut)
{
	int32 N = VertexLoop.Num();
	ConvexOut.VertexData.SetNum(2 * N);
	for (int32 k = 0; k < N; ++k)
	{
		ConvexOut.VertexData[k] = (FVector)(VertexLoop[k]);
		FVector3d PlanePos = BasePlane.ToPlane(VertexLoop[k]);
		ConvexOut.VertexData[N + k] = (FVector)(PlanePos);
	}
	// despite the name this actually computes the convex hull of the point set...
	ConvexOut.UpdateElemBox();
}

void FWaterBooleanUtils::MakeSweepConvex(const FPolygon2d& Polygon, const FFrame3d& Plane, const FVector3d& SweepVector, FKConvexElem& ConvexOut)
{
	const TArray<FVector2d>& Vertices = Polygon.GetVertices();
	int32 N = Vertices.Num();
	ConvexOut.VertexData.SetNum(2 * N);
	for (int32 k = 0; k < N; ++k)
	{
		FVector2d Pos2 = Vertices[k];
		FVector3d Position = Plane.FromPlaneUV(Pos2);
		ConvexOut.VertexData[k] = (FVector)(Position);
		ConvexOut.VertexData[N + k] = (FVector)(Position + SweepVector);
	}
	// despite the name this actually computes the convex hull of the point set...
	ConvexOut.UpdateElemBox();
}

void FWaterBooleanUtils::MakePerTriangleSweepConvexDecomposition(const FDynamicMesh3& Mesh, TArray<FKConvexElem>& Convexes, double DotThreshold)
{
	FAxisAlignedBox3d Bounds = Mesh.GetBounds();
	FVector3d VerticalDir = FVector3d::UnitZ();
	FVector3d OffsetVec = (-VerticalDir) * (Bounds.Max.Z - Bounds.Min.Z);
	FFrame3d BasePlane(Bounds.Center(), VerticalDir);
	BasePlane.Origin.Z = Bounds.Min.Z;

	TArray<FVector3d> Tri;
	Tri.SetNum(3);
	for (int32 tid : Mesh.TriangleIndicesItr())
	{
		FVector3d TriNormal = Mesh.GetTriNormal(tid);
		if (TriNormal.Dot(VerticalDir) > DotThreshold)
		{
			Mesh.GetTriVertices(tid, Tri[0], Tri[1], Tri[2]);
			FKConvexElem Convex;
			//MakeSweepConvex(Tri, OffsetVec, Convex);
			MakeSweepConvex(Tri, BasePlane, Convex);
			Convexes.Add(Convex);
		}
	}
}

void FWaterBooleanUtils::FindConvexPairedTrisFromPlanarMesh(FDynamicMesh3& Mesh, const FFrame3d& Plane, TArray<FPolygon2d>& PlanePolygons)
{
	auto GetXY = [](const FVector3d& P) { return FVector2d(P.X, P.Y); };		// useful for this code

	// map vertices to 2D
	FVector3d Normal = Plane.Z();
	for (int32 vid : Mesh.VertexIndicesItr())
	{
		FVector3d Pos = Mesh.GetVertex(vid);
		FVector2d Pos2 = Plane.ToPlaneUV(Pos);
		double Dist = (Pos - Plane.Origin).Dot(Normal);
		Mesh.SetVertex(vid, FVector3d(Pos2.X, Pos2.Y, Dist));
	}

	// build triangle sets
	TArray<bool> DoneT;
	DoneT.Init(false, Mesh.MaxTriangleID());
	TArray<int32> RemainingT;
	RemainingT.Reserve(Mesh.TriangleCount());
	for (int32 tid : Mesh.TriangleIndicesItr())
	{
		RemainingT.Add(tid);
	}

	// Repeatedly pick a triangle and then try to find a convex pairing. 
	// If that fails just emit the triangle itself as a polygon.
	TArray<FVector2d> Loop, SaveLoop;
	while (RemainingT.Num() > 0)
	{
		int32 tid = RemainingT.Pop(EAllowShrinking::No);
		if (DoneT[tid])
		{
			continue;
		}
		DoneT[tid] = true;

		FIndex3i TriVerts = Mesh.GetTriangle(tid);
		FIndex3i TriEdges = Mesh.GetTriEdges(tid);

		Loop.SetNum(3);
		Loop[0] = GetXY(Mesh.GetVertex(TriVerts.A));
		Loop[1] = GetXY(Mesh.GetVertex(TriVerts.B));
		Loop[2] = GetXY(Mesh.GetVertex(TriVerts.C));
		int32 Sign = FLine2d::FromPoints(Loop[0], Loop[1]).WhichSide(Loop[2]);

		for (int32 i = 0; i < 3; ++i)
		{
			FIndex2i EdgeT = Mesh.GetEdgeT(TriEdges[i]);
			int32 OtherTriID = (EdgeT.A == tid) ? EdgeT.B : EdgeT.A;
			if (OtherTriID == FDynamicMesh3::InvalidID || DoneT[OtherTriID])
			{
				continue;
			}

			FIndex3i OTherTriVerts = Mesh.GetTriangle(OtherTriID);
			int32 j = (i + 1) % 3;
			int32 k = (i + 2) % 3;
			int32 d = IndexUtil::FindTriOtherVtx(TriVerts[j], TriVerts[i], OTherTriVerts);

			FVector2d V = GetXY(Mesh.GetVertex(d));
			FLine2d LineKI = FLine2d::FromPoints(Loop[k], Loop[i]);
			if (LineKI.WhichSide(V) < 0)
			{
				continue;
			}
			FLine2d LineKJ = FLine2d::FromPoints(Loop[k], Loop[j]);
			if (LineKJ.WhichSide(V) > 0)
			{
				continue;
			}

			// Above code appears to sometimes fail for near-degenerate triangles, 
			// so we are going to be extra-safe here
			SaveLoop = Loop;
			Loop.Insert(V, j);
			if (IsConvex(FPolygon2d(Loop)) == false)
			{
				Loop = SaveLoop;
			}
			else
			{
				// found valid convex match
				DoneT[OtherTriID] = true;
				break;
			}
		}

		FPolygon2d Polygon(Loop);
		PlanePolygons.Add(Polygon);
	}

	// map mesh vertices back to 3D
	for (int32 vid : Mesh.VertexIndicesItr())
	{
		FVector3d Pos2 = Mesh.GetVertex(vid);
		FVector3d Pos3 = Plane.FromPlaneUV(FVector2d(Pos2.X, Pos2.Y)) + Pos2.Z * Normal;
		Mesh.SetVertex(vid, Pos3);
	}
}

bool FWaterBooleanUtils::GetTriangleSetBoundaryLoop(const FDynamicMesh3& Mesh, const TArray<int32>& Tris, FEdgeLoop& Loop)
{
	Loop.Mesh = &Mesh;

	// todo: special-case single triangle

	// collect list of border edges
	TArray<int32> Edges;
	for (int32 tid : Tris)
	{
		FIndex3i TriEdges = Mesh.GetTriEdges(tid);
		for (int32 j = 0; j < 3; ++j)
		{
			FIndex2i EdgeT = Mesh.GetEdgeT(TriEdges[j]);
			int32 OtherT = (EdgeT.A == tid) ? EdgeT.B : EdgeT.A;
			if (OtherT == FDynamicMesh3::InvalidID || Tris.Contains(OtherT) == false)
			{
				Edges.AddUnique(TriEdges[j]);
			}
		}
	}

	// Start at first edge and walk around loop, adding one vertex and edge each time.
	// Abort if we encounter any nonmanifold configuration 
	int32 NumEdges = Edges.Num();
	int32 StartEdge = Edges[0];
	FIndex2i StartEdgeT = Mesh.GetEdgeT(StartEdge);
	int32 InTri = Tris.Contains(StartEdgeT.A) ? StartEdgeT.A : StartEdgeT.B;
	FIndex2i StartEdgeV = Mesh.GetEdgeV(StartEdge);
	IndexUtil::OrientTriEdge(StartEdgeV.A, StartEdgeV.B, Mesh.GetTriangle(InTri));
	Loop.Vertices.Reset();
	Loop.Vertices.Add(StartEdgeV.A);
	Loop.Vertices.Add(StartEdgeV.B);
	int32 CurEndVert = Loop.Vertices.Last();
	int32 PrevEdge = StartEdge;
	Loop.Edges.Reset();
	Loop.Edges.Add(StartEdge);
	int32 NumEdgesUsed = 1;
	bool bContinue = true;
	do {
		bContinue = false;
		for (int32 eid : Mesh.VtxEdgesItr(CurEndVert))
		{
			if (eid != PrevEdge && Edges.Contains(eid) && Loop.Edges.Contains(eid) == false)
			{
				FIndex2i EdgeV = Mesh.GetEdgeV(eid);
				int32 NextV = (EdgeV.A == CurEndVert) ? EdgeV.B : EdgeV.A;
				if (NextV == Loop.Vertices[0])		// closed loop
				{
					Loop.Edges.Add(eid);
					NumEdgesUsed++;
					bContinue = false;
					break;
				}
				else
				{
					if (Loop.Vertices.Contains(NextV))
					{
						return false;		// hit a middle vertex, we have nonmanifold set of edges, abort
					}
					Loop.Edges.Add(eid);
					PrevEdge = eid;
					Loop.Vertices.Add(NextV);
					NumEdgesUsed++;
					CurEndVert = NextV;
					bContinue = true;
					break;
				}
			}
		}

	} while (bContinue);

	if (NumEdgesUsed != Edges.Num())	// closed loop but we still have edges? must have nonmanifold configuration, abort.
	{
		return false;
	}

	return true;
}

bool FWaterBooleanUtils::TriSetToBoundaryPolygon(FDynamicMesh3& MeshXY, const TArray<int32>& Tris, FPolygon2d& PolygonOut)
{
	FEdgeLoop BoundaryLoop;
	if (GetTriangleSetBoundaryLoop(MeshXY, Tris, BoundaryLoop))
	{
		for (int32 vid : BoundaryLoop.Vertices)
		{
			FVector3d V = MeshXY.GetVertex(vid);
			PolygonOut.AppendVertex(FVector2d(V.X, V.Y));
		}
		return true;
	}
	return false;
}

bool FWaterBooleanUtils::CanAppendTriConvex(FDynamicMesh3& MeshXY, const TArray<int32>& Tris, int32 TestTri)
{
	TArray<int32> AllTris(Tris);
	AllTris.Add(TestTri);
	FPolygon2d Polygon;
	if (TriSetToBoundaryPolygon(MeshXY, AllTris, Polygon))
	{
		return IsConvex(Polygon);
	}
	return false;
}

void FWaterBooleanUtils::FindConvexPolygonsFromPlanarMesh(FDynamicMesh3& Mesh, const FFrame3d& Plane, TArray<FPolygon2d>& PlanePolygons)
{
	auto GetXY = [](const FVector3d& P) { return FVector2d(P.X, P.Y); };		// useful for this code

	// map vertices to 2D
	FVector3d Normal = Plane.Z();
	for (int32 vid : Mesh.VertexIndicesItr())
	{
		FVector3d Pos = Mesh.GetVertex(vid);
		FVector2d Pos2 = Plane.ToPlaneUV(Pos);
		double Dist = (Pos - Plane.Origin).Dot(Normal);
		Mesh.SetVertex(vid, FVector3d(Pos2.X, Pos2.Y, Dist));
	}

	// collect up set of triangles
	TArray<bool> DoneT;
	DoneT.Init(false, Mesh.MaxTriangleID());
	TArray<int32> RemainingT;
	RemainingT.Reserve(Mesh.TriangleCount());
	for (int32 tid : Mesh.TriangleIndicesItr())
	{
		RemainingT.Add(tid);
	}

	// TODO: consider adding entire vertex one-rings? This can result in a large additional convex chunk
	// that could not be discovered by incrementally adding triangles.
	while (RemainingT.Num() > 0)
	{
		int32 SeedTID = RemainingT.Pop(EAllowShrinking::No);
		if (DoneT[SeedTID])
		{
			continue;
		}
		DoneT[SeedTID] = true;

		TArray<int32> ConvexSet;
		ConvexSet.Add(SeedTID);

		TArray<int32> NbrSet;
		auto PushNewNbrTrisFunc = [&](int32 tid) {
			FIndex3i NbrTris = Mesh.GetTriNeighbourTris(tid);
			for (int32 j = 0; j < 3; ++j)
			{
				if (NbrTris[j] >= 0 && DoneT[NbrTris[j]] == false && NbrSet.Contains(NbrTris[j]) == false)
				{
					NbrSet.Add(NbrTris[j]);
				}
			}
		};
		PushNewNbrTrisFunc(SeedTID);

		bool bFound = false;
		do
		{
			bFound = false;
			for (int32 NbrTID : NbrSet)
			{
				if (CanAppendTriConvex(Mesh, ConvexSet, NbrTID))
				{
					ConvexSet.Add(NbrTID);
					DoneT[NbrTID] = true;
					NbrSet.Remove(NbrTID);
					PushNewNbrTrisFunc(NbrTID);
					bFound = true;
					break;
				}
			}
		} while (bFound);


		FPolygon2d Polygon;
		bool bResult = TriSetToBoundaryPolygon(Mesh, ConvexSet, Polygon);
		if (bResult)
		{
			PlanePolygons.Add(Polygon);
		}
	}

	// map vertices back to 3D
	for (int32 vid : Mesh.VertexIndicesItr())
	{
		FVector3d Pos2 = Mesh.GetVertex(vid);
		FVector3d Pos3 = Plane.FromPlaneUV(FVector2d(Pos2.X, Pos2.Y)) + Pos2.Z * Normal;
		Mesh.SetVertex(vid, Pos3);
	}
}

void FWaterBooleanUtils::MakeClusteredTrianglesSweepConvexDecomposition(const FDynamicMesh3& Mesh, TArray<FKConvexElem>& Convexes, double DotThreshold)
{
	FAxisAlignedBox3d Bounds = Mesh.GetBounds();
	FVector3d VerticalDir = FVector3d::UnitZ();
	FVector3d OffsetVec = (-VerticalDir) * (Bounds.Max.Z - Bounds.Min.Z);

	// collect upward-facing triangles, as well as downward-facing ones (these might be flipped)
	TArray<int32> Triangles;
	TArray<int32> PossibleFlipTris;
	for (int32 tid : Mesh.TriangleIndicesItr())
	{
		double VertDot = Mesh.GetTriNormal(tid).Dot(VerticalDir);
		if (VertDot > DotThreshold)
		{
			Triangles.Add(tid);
		}
		else if (FMathd::Abs(VertDot) > DotThreshold)
		{
			PossibleFlipTris.Add(tid);
		}
	}

	// if a flipped tri is connected to one or more upward-facing tris, and coplanar with them, 
	// assume it should be included
	for (int32 tid : PossibleFlipTris)
	{
		FIndex3i TriNbrs = Mesh.GetTriNeighbourTris(tid);
		FAxisAlignedBox3d NbrVertBounds = FAxisAlignedBox3d::Empty();
		int32 NbrInSetCount = 0;
		for (int32 j = 0; j < 3; ++j)
		{
			if (Triangles.Contains(TriNbrs[j]))
			{
				NbrVertBounds.Contain(Mesh.GetTriBounds(TriNbrs[j]));
			}
			NbrInSetCount++;
		}
		FInterval1d ZRange(NbrVertBounds.Min.Z, NbrVertBounds.Max.Z);
		if (NbrInSetCount > 0 && ZRange.Contains(Mesh.GetTriCentroid(tid).Z))
		{
			Triangles.Add(tid);
		}
	}

	// Extract connected planar components. For each Component, extract the submesh and decompose the triangles
	// into convex polygons, then make sweep convexes out of them
	FMeshConnectedComponents Components(&Mesh);
	Components.FindConnectedTriangles(Triangles);
	for (int32 k = 0; k < Components.Num(); ++k)
	{
		FDynamicSubmesh3 SubmeshCalc(&Mesh, Components.GetComponent(k).Indices);
		FDynamicMesh3& Submesh = SubmeshCalc.GetSubmesh();
		FFrame3d Plane(Submesh.GetBounds().Center(), VerticalDir);

		TArray<FPolygon2d> Polygons;
		FindConvexPairedTrisFromPlanarMesh(Submesh, Plane, Polygons);
		//FindConvexPolygonsFromPlanarMesh(Submesh, Plane, Polygons);

		for (const FPolygon2d& Polygon : Polygons)
		{
			if (IsConvex(Polygon) == false)
			{
				UE_LOGF(LogWater, Warning, "UE::Water::MakeClusteredTrianglesSweepConvexDecomposition : Polygon is not convex!");
			}
			FKConvexElem Convex;
			MakeSweepConvex(Polygon, Plane, OffsetVec, Convex);
			Convexes.Add(Convex);
		}
	}

}

void FWaterBooleanUtils::GenerateSubtractSweepConvexDecomposition(const FDynamicMesh3& MeshIn,
	FAxisAlignedBox3d& BaseClipBoxOut,
	TArray<FKConvexElem>& ConvexesOut)
{
	FAxisAlignedBox3d Bounds = MeshIn.GetBounds();
	double MinZ = Bounds.Min.Z, MaxZ = Bounds.Max.Z;
	double ZHeight = MaxZ - MinZ;

	// compute Z vertex histogram
	int32 IntervalSteps = 64;
	TArray<int32> ZHistogram;
	TArray<FInterval1d> ZHistBinRanges;
	ZHistogram.Init(0, IntervalSteps);
	ZHistBinRanges.Init(FInterval1d::Empty(), IntervalSteps);
	for (const FVector3d Pos : MeshIn.VerticesItr())
	{
		double Zt = (Pos.Z - MinZ) / ZHeight;
		int32 Bin = FMathd::Clamp((int32)(Zt * (double)IntervalSteps), 0, IntervalSteps - 1);
		ZHistogram[Bin] += 1;
		ZHistBinRanges[Bin].Contain(Pos.Z);
	}

	// find largest "middle" bin - use this as the cut plane height
	int32 MaxBinIndex = 1;
	for (int32 k = 2; k < IntervalSteps - 1; ++k)
	{
		if (ZHistogram[k] > ZHistogram[MaxBinIndex])
		{
			MaxBinIndex = k;
		}
	}
	double BinSize = ZHeight / (double)IntervalSteps;
	double BaseZ = ZHistBinRanges[MaxBinIndex].Max;
	if (MaxBinIndex == 1 && ZHistogram[1] == 0)
	{
		BaseZ = Bounds.Center().Z;		// if we did not find a bin to cut at, this might be a through-hole, so cut at the middle ? failure case really..
		UE_LOGF(LogWater, Display, "UE::Water::GenerateSubtractSweepConvexDecomposition : Invalid Configuration - either no cut, or through-hole");
	}

	// set up the "bottom" box
	BaseClipBoxOut = Bounds;
	BaseClipBoxOut.Max.Z = BaseZ;

	// slice the input mesh at the cut plane, to find the remaining volume to fill
	FFrame3d CutPlane(
		FVector3d(0, 0, BaseZ + 100.0f * FMathf::ZeroTolerance),
		FVector3d::UnitZ());
	FDynamicMesh3 CutMesh = MeshIn;
	FMeshPlaneCut Cut(&CutMesh, CutPlane.Origin, -CutPlane.Z());
	Cut.Cut();

	// build the sweep convexes
	//MakePerTriangleSweepConvexDecomposition(CutMesh, ConvexesOut);
	MakeClusteredTrianglesSweepConvexDecomposition(CutMesh, ConvexesOut);
}

void FWaterBooleanUtils::BuildOceanCollisionComponents(
	FBoxSphereBounds WorldBoxIn,
	FTransform ActorTransform,
	const TArray<AWaterBodyExclusionVolume*>& ExclusionVolumes,
	TArray<FBoxSphereBounds>& Boxes,
	TArray<TArray<FKConvexElem>>& MeshConvexes,
	double WorldMeshBufferWidth,
	double WorldBoxOverlap)
{
	FAxisAlignedBox3d WorldBox(
		(FVector3d)WorldBoxIn.Origin - (FVector3d)WorldBoxIn.BoxExtent,
		(FVector3d)WorldBoxIn.Origin + (FVector3d)WorldBoxIn.BoxExtent);

	// Find Exclusion Volumes that might actually intersect the world box.
	// Volumes that do not intersect can be ignored.
	TArray<AWaterBodyExclusionVolume*> IntersectingVolumes;
	for (AWaterBodyExclusionVolume* Volume : ExclusionVolumes)
	{
		// bounding box of volume has to intersect world box and not be fully contained
		FAxisAlignedBox3d VolBounds = GetBounds(Volume);
		if (WorldBox.Intersects(VolBounds) && WorldBox.Contains(VolBounds) == false)
		{
			IntersectingVolumes.Add(Volume);
		}
	}

	// If we have no intersectinge exclusion volumes we can early-out
	if (IntersectingVolumes.Num() == 0)
	{
		FBoxSphereBounds LocalBox = WorldBoxIn.TransformBy(ActorTransform.Inverse());
		Boxes.Add(LocalBox);
		return;
	}

	// group any overlapping volumes
	TArray<TArray<AWaterBodyExclusionVolume*>> ClusteredVolumes;
	ClusterExclusionVolumes(IntersectingVolumes, ClusteredVolumes, WorldMeshBufferWidth);

	FTransform3d ActorTransformd(ActorTransform);

	// for each cluster, clip ocean box to bounds-plus-buffer of cluster and then
	// compute Boolean subtraction of exclusion volumes from that box
	TArray<FAxisAlignedBox3d> MeshBoxes;
	for (int32 ClusterIndex = 0; ClusterIndex < ClusteredVolumes.Num(); ++ClusterIndex)
	{
		TArray<AWaterBodyExclusionVolume*>& VolumeSet = ClusteredVolumes[ClusterIndex];

		FAxisAlignedBox3d MeshBoxBounds = GetBounds(VolumeSet);
		// Clip the full world box to the XY region around the intersecting exclusion volumes, 
		// plus a buffer strip so that none of the exclusions are right on the edge
		FAxisAlignedBox3d ClippedWorldBox = WorldBox;
		ClippedWorldBox.Min.X = MeshBoxBounds.Min.X - WorldMeshBufferWidth;
		ClippedWorldBox.Max.X = MeshBoxBounds.Max.X + WorldMeshBufferWidth;
		ClippedWorldBox.Min.Y = MeshBoxBounds.Min.Y - WorldMeshBufferWidth;
		ClippedWorldBox.Max.Y = MeshBoxBounds.Max.Y + WorldMeshBufferWidth;

		// Build our Box-Minus-Exclusions mesh in the clipped bounds
		FDynamicMesh3 FullCollisionMesh = MakeBoxCollisionMesh(ClippedWorldBox, VolumeSet);

		// Transform from world space back to actor local space
		MeshTransforms::ApplyTransformInverse(FullCollisionMesh, ActorTransformd);

		FAxisAlignedBox3d CollisionBaseBox;
		TArray<FKConvexElem> Convexes;
		GenerateSubtractSweepConvexDecomposition(FullCollisionMesh, CollisionBaseBox, Convexes);

		FBoxSphereBounds LocalBox;
		LocalBox.Origin = (FVector)CollisionBaseBox.Center();
		LocalBox.BoxExtent = (FVector)CollisionBaseBox.Extents();
		Boxes.Add(LocalBox);

		MeshConvexes.Add(Convexes);

		// keep track of 'filled' space
		MeshBoxes.Add(ClippedWorldBox);
	}

	// generate a set of bounding-boxes that fill the space around the mesh boxes, in XY plane
	TArray<FAxisAlignedBox3d> FillBoxesOut;
	FillEmptySpaceAroundBoxesXY(WorldBox, MeshBoxes, FillBoxesOut, WorldBoxOverlap);

	// add fill boxes to output box set
	for (const FAxisAlignedBox3d& WorldSpaceBox : FillBoxesOut)
	{
		FBoxSphereBounds LocalBox;
		LocalBox.Origin = ActorTransform.InverseTransformPosition((FVector)WorldSpaceBox.Center());
		LocalBox.BoxExtent = (FVector)WorldSpaceBox.Extents();

		Boxes.Add(LocalBox);
	}
}
