// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FrameTypes.h"
#include "GeometryBase.h"
#include "Polygon2.h"

namespace UE::Geometry { class FDynamicMesh3; }
namespace UE::Geometry { class FEdgeLoop; }

using namespace UE::Geometry;

class AWaterBodyExclusionVolume;
class AVolume;
struct FKConvexElem;


class FWaterBooleanUtils
{
public:
	/**
		* Construct a set of Collision Meshes and Boxes for an Ocean bounding-box with Exclusion volumes.
		* This requires Boolean-subtracting the Exclusion volumes from the Ocean Box. For many reasons
		* (accuracy, runtime perf, etc) it is desirable to minimize the fill-volume that is mesh. So the
		* necessary mesh regions are clipped to boxes, and the remaining space is filled with bounding-boxes.
		*
		* @param WorldBoxIn Ocean bounding box in world space
		* @param ActorTransform transform on the Ocean Actor. Output meshes/boxes will be positioned relative to this Actor/Transform.
		* @param ExclusionVolumes AVolumes we will subtract from the Ocean box
		* @param Boxes list out output boxes
		* @param Meshes list of output meshes
		* @param WorldMeshBufferWidth the output meshes will contain a band this wide around the subtracted exclusion meshes (default 1000, risky to set to zero)
		* @param WorldBoxOverlap output boxes will be expanded this amount on each axis, to slightly overlap (default 10)
		*
		*/
	static void BuildOceanCollisionComponents(
		FBoxSphereBounds WorldBoxIn,
		FTransform ActorTransform,
		const TArray<AWaterBodyExclusionVolume*>& ExclusionVolumes,
		TArray<FBoxSphereBounds>& Boxes,
		TArray<TArray<FKConvexElem>>& MeshConvexes,
		double WorldMeshBufferWidth = 1000.0,
		double WorldBoxOverlap = 10.0);

private:
	/** @return triangle aspect ratio transformed to be in [0,1] range */
	static double UnitAspectRatio(const FVector3d& A, const FVector3d& B, const FVector3d& C);
	/** @return triangle aspect ratio transformed to be in [0,1] range */
	static double UnitAspectRatio(const FDynamicMesh3& Mesh, int32 TriangleID);

	/**
	 * If both triangles on an edge are coplanar, we can arbitrarily flip the interior edge to
	 * improve triangle quality. Similarly if one triangle on an edge is degenerate, we can flip
	 * the edge without affecting the shape to try to remove it. This code does a single pass of
	 * such an optimization.
	 * Note: could be more efficient to do multiple passes internally, would save on the initial computation
	 */
	static void PlanarFlipsOptimization(FDynamicMesh3& Mesh, double PlanarDotThresh = 0.99);

	/**
	 * Extracts a FDynamicMesh3 from an AVolume
	 * The output mesh is in World Space.
	 */
	static void ExtractMesh(AVolume* Volume, FDynamicMesh3& Mesh);

	/**
	 * Create a FDynamicMesh3 for the box of a FBoxSphereBounds
	 */
	static void ExtractMesh(FBoxSphereBounds Bounds, FDynamicMesh3& Mesh);

	/**
	 * Create a FDynamicMesh3 for an FAxisAlignedBox3d
	 */
	static void ExtractMesh(FAxisAlignedBox3d Bounds, FDynamicMesh3& Mesh);

	/**
	 * Utility function to try several auto-repair methods on a mesh, to make sure it is closed
	 */
	static void ApplyBooleanRepairs(FDynamicMesh3& Mesh, double MergeTolerance = FMathf::ZeroTolerance * 10.0);

	/**
	 * Construct transforms to and from a normalized space, for a given bounds.
	 * ("Normalized space" is such that scaled box fits in origin-centered unit box)
	 */
	static void MakeNormalizationTransform(const FAxisAlignedBox3d& Bounds, FTransformSRT3d& ToNormalizedOut, FTransformSRT3d& FromNormalizedOut);

	/**
	 * Utility function to set per-triangle attribute normals on a DynamicMesh
	 */
	static void SetToFaceNormals(FDynamicMesh3& Mesh);

	/**
	 * Mesh a set of AVolumes and then Boolean-Union them into a FDynamicMesh3.
	 * @param Transform this transform is applied to each volume before the Boolean is computed
	 */
	static FDynamicMesh3 AccumulateExtrusionVolumes(const TArray<AWaterBodyExclusionVolume*>& ExclusionVolumes, const FTransformSRT3d& Transform);

	/**
	 * Generate a mesh that is a Box with a set of Volumes Boolean-subtracted.
	 */
	static FDynamicMesh3 MakeBoxCollisionMesh(FAxisAlignedBox3d WorldBoxBounds, const TArray<AWaterBodyExclusionVolume*>& ExclusionVolumes);

	/** @return bounds of an AVolume in world-space */
	static FAxisAlignedBox3d GetBounds(const AWaterBodyExclusionVolume* Volume, double ExpansionSize = 0);

	/** @return bounds of a set of AVolumes in world-space */
	static FAxisAlignedBox3d GetBounds(const TArray<AWaterBodyExclusionVolume*>& Volumes);

	/**
	 * Group the input set of volumes into clusters based on overlapping bounding boxes.
	 * All bounding boxes are expanded by BoxExpandSize.
	 */
	static void ClusterExclusionVolumes(
		const TArray<AWaterBodyExclusionVolume*>& AllVolumes,
		TArray<TArray<AWaterBodyExclusionVolume*>>& VolumeClustersOut,
		double BoxExpandSize);

	/**
	 * Fill the space Difference(OuterBox,RemoveBox)  (ie empty space around RemoveBox inside OuterBox) with
	 * up to 4 boxes, appended to BoxesOut array. Zero-volume boxes are not included.
	 * Assumption is all boxes have the same Z-height, so we are operating in XY space.
	 */
	static void FindRemainingBoxSpaceXY(const FAxisAlignedBox3d& OuterBox, const FAxisAlignedBox3d& RemoveBox,
		TArray<FAxisAlignedBox3d>& BoxesOut);

	/**
	 * Generate a set of boxes in FillBoxesOut that fill the empty space inside Box around ContentBoxes.
	 * Precondition is that ContentBoxes cannot overlap.
	 * Assumption is all boxes have the same Z-height, so we are operating in XY space.
	 * This is a simple approach, does not produce optimal set of boxes.
	 * @param OverlapAmount optional "fudge factor", output boxes will be expanded by this amount on X and Y axes
	 */
	static void FillEmptySpaceAroundBoxesXY(
		const FAxisAlignedBox3d& Box,
		const TArray<FAxisAlignedBox3d>& ContentBoxes,
		TArray<FAxisAlignedBox3d>& FillBoxesOut,
		double OverlapAmount = 0);

	/**
	 * Check if a Polygon is Convex. Works for CW and CCW orientation. Starting at the first edge,
	 * walks around Polygon and makes sure we are consistently turning in the same direction at
	 * each vertex (or going straight).
	 *
	 * Degenerate edges are ignored. However if the entire Polygon is degenerate, returns false.
	 * Returns true if polygon is a line (this is somewhat arbitrary).
	 * @return true if polygon is convex
	 */
	static bool IsConvex(const FPolygon2d& Polygon);

	/**
	 * Construct a Convex hull collision element for a polyhedra created by sweeping VertexLoop along SweepVector
	 */
	static void MakeSweepConvex(const TArray<FVector3d>& VertexLoop, const FVector3d& SweepVector, FKConvexElem& ConvexOut);

	/**
	 * Construct a Convex Hull collision element for a polyhedra created from VertexLoop and the projection
	 * of VertexLoop onto BasePlane
	 */
	static void MakeSweepConvex(const TArray<FVector3d>& VertexLoop, const FFrame3d& BasePlane, FKConvexElem& ConvexOut);

	/**
	 * Construct a Convex Hull collision element for a polyhedra created by projecting the given Polygon into the XY
	 * plane of the given Plane, and then a second loop offset by SweepVector
	 */
	static void MakeSweepConvex(const FPolygon2d& Polygon, const FFrame3d& Plane, const FVector3d& SweepVector, FKConvexElem& ConvexOut);

	/**
	 * This function extracts upwards-facing triangles from Mesh and creates a separate Convex for each
	 * triangle by sweeping it downwards to the bottom of the mesh bounding-box.
	 * Assumption is that the mesh is a height-field.
	 * @param DotThreshold only faces where Dot(Normal,+Z) > DotThreshold are swept. Default 0.95 (~18 deg)
	 */
	static void MakePerTriangleSweepConvexDecomposition(const FDynamicMesh3& Mesh, TArray<FKConvexElem>& Convexes, double DotThreshold = 0.95);

	/**
	 * Decompose the Mesh into convex pieces by trying to find pairs of triangles whose outer polygon boundary is Convex.
	 * Assumption is that mesh is aligned with given Plane, so that we can work in 2D space of that plane via projection.
	 */
	static void FindConvexPairedTrisFromPlanarMesh(FDynamicMesh3& Mesh, const FFrame3d& Plane, TArray<FPolygon2d>& PlanePolygons);

	/**
	 * Find the edge loop border around a set of triangles of a Mesh.
	 * This is computed via local walk and so does not create any full-mesh data structures.
	 * However current implementation may not be efficient for large triangle sets.
	 * Algorithm terminates if a non-manifold boundary is detected, and returns false if some triangles are unused.
	 * @param Loop output loop will be stored here. This value is garbage if false is returned.
	 * @return true if a single well-formed loop was found, false if non-manifold or failure case encountered
	 */
	static bool GetTriangleSetBoundaryLoop(const FDynamicMesh3& Mesh, const TArray<int32>& Tris, FEdgeLoop& Loop);

	/**
	 * Find the 2D polygonal boundary of a set of triangles of a Mesh
	 * @param MeshXY triangle mesh lying in XY plane
	 * @return false if no single well-formed (manifold) boundary loop was found (including multiple regions)
	 */
	static bool TriSetToBoundaryPolygon(FDynamicMesh3& MeshXY, const TArray<int32>& Tris, FPolygon2d& PolygonOut);

	/**
	 * Check if the 2D polygonal boundary of a set of triangles, plus a test triangle, is convex
	 * @param MeshXY triangle mesh lying in XY plane
	 * @return true if combined boundary is convex
	 */
	static bool CanAppendTriConvex(FDynamicMesh3& MeshXY, const TArray<int32>& Tris, int32 TestTri);

	/**
	 * Decompose the Mesh into convex polygons by grouping triangles whose outer polygon boundary is convex.
	 * Assumption is that mesh is aligned with given Plane, so that we can work in 2D space of that plane via projection.
	 * The strategy is to pick arbitrary triangles and try to append adjacent triangles.
	 * Current implementation is somewhat expensive...
	 */
	static void FindConvexPolygonsFromPlanarMesh(FDynamicMesh3& Mesh, const FFrame3d& Plane, TArray<FPolygon2d>& PlanePolygons);

	/**
	 * This function extracts upwards-facing triangles from Mesh and creates Convex polyhedra for groups of
	 * those triangles that have convex polygonal boundaries, then sweeps them downwards to the bottom of the mesh bounding-box.
	 * Assumption is that the mesh is a height-field.
	 * @param DotThreshold only faces where Dot(Normal,+Z) > DotThreshold are swept. Default 0.95 (~18 deg)
	 */
	static void MakeClusteredTrianglesSweepConvexDecomposition(const FDynamicMesh3& Mesh, TArray<FKConvexElem>& Convexes, double DotThreshold = 0.95);

	/**
	 * Build a Convex Decomposition of the input Mesh. The assumption here is that
	 * the mesh was created by subtracting a vertical sweep for a box. So the convex
	 * decomposition is (1) a "base box" cutting off the bottom, at the highest level
	 * of the subtracted area and (2) a set of vertical-sweep convex-hulls filling the
	 * remaining volume, created by identifying "upward-facing" triangles and sweeping
	 * them down to the cut plane.
	 *
	 *
	 */
	static void GenerateSubtractSweepConvexDecomposition(const FDynamicMesh3& MeshIn,
		FAxisAlignedBox3d& BaseClipBoxOut,
		TArray<FKConvexElem>& ConvexesOut);
};
