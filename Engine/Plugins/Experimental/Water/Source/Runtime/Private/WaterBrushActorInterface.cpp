// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterBrushActorInterface.h"
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterBrushActorInterface)

UWaterBrushActorInterface::UWaterBrushActorInterface(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

#if WITH_EDITOR
TArray<UPrimitiveComponent*> IWaterBrushActorInterface::GetBrushRenderableComponents() const
{
	return TArray<UPrimitiveComponent*>();
}

IWaterBrushActorInterface::FWaterBrushActorChangedEvent& IWaterBrushActorInterface::GetOnWaterBrushActorChangedEvent()
{
	static IWaterBrushActorInterface::FWaterBrushActorChangedEvent WaterBrushActorChangedEvent;
	return WaterBrushActorChangedEvent;
}

#endif //WITH_EDITOR



