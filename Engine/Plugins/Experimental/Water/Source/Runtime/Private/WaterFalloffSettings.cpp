// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterFalloffSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterFalloffSettings)

FWaterFalloffSettings::FWaterFalloffSettings()
	: FalloffMode(EWaterBrushFalloffMode::Angle)
	, FalloffAngle(45.0f)
	, FalloffWidth(1024.0f)
	, EdgeOffset(0)
	, ZOffset(0)
{
}

