// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterIconHelper.h"

#if WITH_EDITOR

#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"
#include "UObject/ConstructorHelpers.h"
#include "WaterRuntimeSettings.h"
#include "Modules/ModuleManager.h"
#include "WaterEditorServices.h"
#include "WaterModule.h"

UBillboardComponent* FWaterIconHelper::EnsureSpriteComponentCreated_Internal(AActor* Actor, UClass* InClass, const TCHAR* InIconTextureName)
{
	UBillboardComponent* ActorIcon = nullptr;
	
	ActorIcon = Actor->FindComponentByClass<UBillboardComponent>();
	if (ActorIcon == nullptr)
	{
		ActorIcon = Actor->CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"), true);
	}
	if (ActorIcon != nullptr)
	{
		ConstructorHelpers::FObjectFinderOptional<UTexture2D> Texture(InIconTextureName);
		IWaterModuleInterface& WaterModule = FModuleManager::GetModuleChecked<IWaterModuleInterface>(TEXT("Water"));
		if (IWaterEditorServices* WaterEditorServices = WaterModule.GetWaterEditorServices())
		{
			WaterEditorServices->RegisterWaterActorSprite(InClass, Texture.Get());
		}
		ActorIcon->Sprite = Texture.Get();
		ActorIcon->bHiddenInGame = true;
		ActorIcon->SpriteInfo.Category = TEXT("Water");
		ActorIcon->SpriteInfo.DisplayName = NSLOCTEXT("SpriteCategory", "Water", "Water");
		ActorIcon->SetVisibility(true);
		ActorIcon->SetupAttachment(Actor->GetRootComponent());
		UpdateSpriteComponent(Actor, ActorIcon->Sprite);
	}
	return ActorIcon;
}

void FWaterIconHelper::UpdateSpriteComponent(AActor* Actor, UTexture2D* InTexture)
{
	if (UBillboardComponent* ActorIcon = Actor->FindComponentByClass<UBillboardComponent>())
	{
		FVector ZOffset(0.0f, 0.0f, GetDefault<UWaterRuntimeSettings>()->WaterBodyIconWorldZOffset);

		ActorIcon->SetRelativeScale3D(FVector(1.f));
		ActorIcon->Sprite = InTexture;
		ActorIcon->SetRelativeLocation(ZOffset);
		ActorIcon->bIsScreenSizeScaled = true;
		ActorIcon->MarkRenderStateDirty();
	}
}

#endif
