// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterInfoRendering.h"

#include "CommonRenderResources.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "EngineUtils.h"
#include "LegacyScreenPercentageDriver.h"
#include "Modules/ModuleManager.h"
#include "RenderCaptureInterface.h"
#include "RHIStaticStates.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "SceneCaptureRendering.h"
#include "PostProcess/DrawRectangle.h"
#include "Math/OrthoMatrix.h"
#include "GameFramework/WorldSettings.h"
#include "ScreenRendering.h"
#include "WaterZoneActor.h"
#include "LandscapeRender.h"
#include "LandscapeModule.h"
#include "TextureResource.h"
#include "WaterBodyComponent.h"
#include "WaterBodyInfoMeshComponent.h"
#include "Containers/StridedView.h"

#include "RenderGraphBuilder.h"
#include "SceneRenderTargetParameters.h"

#include "SceneRendering.h"
#include "Rendering/CustomRenderPass.h"

static int32 WaterInfoRenderLandscapeMinimumMipLevel = 0;
static FAutoConsoleVariableRef CVarWaterInfoRenderLandscapeMinimumMipLevel(
	TEXT("r.Water.WaterInfo.LandscapeMinimumMipLevel"),
	WaterInfoRenderLandscapeMinimumMipLevel,
	TEXT("Clamps the minimum allowed mip level for the landscape when rendering the water info texture. Used on the lowest end platforms which cannot support rendering all the landscape vertices at the highest LOD."));

namespace UE::WaterInfo
{

struct FUpdateWaterInfoParams
{
	FSceneInterface* Scene = nullptr;
	FSceneRenderer* DepthRenderer = nullptr;
	FSceneRenderer* ColorRenderer = nullptr;
	FSceneRenderer* DilationRenderer = nullptr;
	FRenderTarget* RenderTarget = nullptr;
	FTexture* OutputTexture = nullptr;

	FVector WaterZoneExtents;
	FVector2f WaterHeightExtents;
	float GroundZMin;
	float CaptureZ;
	int32 VelocityBlurRadius;
};


// ---------------------------------------------------------------------------------------------------------------------

/** Pixel shader for merging water flow velocity (XY), normalized water height (Z) and normalized ground height (W).*/
class FWaterInfoMergePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterInfoMergePS);
	SHADER_USE_PARAMETER_STRUCT(FWaterInfoMergePS, FGlobalShader);

	class FEnable128BitRT : SHADER_PERMUTATION_BOOL("ENABLE_128_BIT");
	using FPermutationDomain = TShaderPermutationDomain<FEnable128BitRT>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, DepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DepthTextureSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, ColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ColorTextureSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, DilationTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DilationTextureSampler)
		SHADER_PARAMETER(FVector2f, WaterHeightExtents)
		SHADER_PARAMETER(float, GroundZMin)
		SHADER_PARAMETER(float, CaptureZ)
		SHADER_PARAMETER(float, UndergroundDilationDepthOffset)
		SHADER_PARAMETER(float, DilationOverwriteMinimumDistance)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain GetPermutationVector(bool bUse128BitRT)
	{
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FEnable128BitRT>(bUse128BitRT);
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		const bool bPlatformRequiresExplicit128bitRT = FDataDrivenShaderPlatformInfo::GetRequiresExplicit128bitRT(Parameters.Platform);
		return (!PermutationVector.Get<FEnable128BitRT>() || bPlatformRequiresExplicit128bitRT);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FEnable128BitRT>())
		{
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_A32B32G32R32F);
		}
	}
};

IMPLEMENT_GLOBAL_SHADER(FWaterInfoMergePS, "/Plugin/Water/Private/WaterInfoMerge.usf", "Main", SF_Pixel);

static void MergeWaterInfoAndDepth(
	FRDGBuilder& GraphBuilder,
	const FSceneViewFamily& ViewFamily,
	const FSceneView& View,
	FRDGTextureRef OutputTexture,
	FRDGTextureRef DepthTexture,
	FRDGTextureRef ColorTexture,
	FRDGTextureRef DilationTexture,
	const FUpdateWaterInfoParams& Params)
{
	RDG_EVENT_SCOPE(GraphBuilder, "WaterInfoDepthMerge");

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(View.GetFeatureLevel());

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();

	const bool bUse128BitRT = PlatformRequires128bitRT(OutputTexture->Desc.Format);
	const FWaterInfoMergePS::FPermutationDomain PixelPermutationVector = FWaterInfoMergePS::GetPermutationVector(bUse128BitRT);

	{
		static auto* CVarDilationOverwriteMinimumDistance = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.Water.WaterInfo.DilationOverwriteMinimumDistance"));
		static auto* CVarUndergroundDilationDepthOffset = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.Water.WaterInfo.UndergroundDilationDepthOffset"));

		FWaterInfoMergePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterInfoMergePS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ENoAction);
		PassParameters->SceneTextures = GetSceneTextureShaderParameters(View);
		PassParameters->DepthTexture = DepthTexture;
		PassParameters->DepthTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->ColorTexture = ColorTexture;
		PassParameters->ColorTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->DilationTexture = DilationTexture;
		PassParameters->DilationTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->CaptureZ = Params.CaptureZ;
		PassParameters->WaterHeightExtents = Params.WaterHeightExtents;
		PassParameters->GroundZMin = Params.GroundZMin;
		PassParameters->DilationOverwriteMinimumDistance = CVarDilationOverwriteMinimumDistance ? CVarDilationOverwriteMinimumDistance->GetValueOnRenderThread() : 128.0f;
		PassParameters->UndergroundDilationDepthOffset = CVarUndergroundDilationDepthOffset ? CVarUndergroundDilationDepthOffset->GetValueOnRenderThread() : 64.0f;

		TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
		TShaderMapRef<FWaterInfoMergePS> PixelShader(ShaderMap, PixelPermutationVector);

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("WaterInfoDepthMerge"),
			PassParameters,
			ERDGPassFlags::Raster,
			[PassParameters, GraphicsPSOInit, VertexShader, PixelShader, &View](FRHICommandListImmediate& RHICmdList)
			{
				FGraphicsPipelineStateInitializer LocalGraphicsPSOInit = GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(LocalGraphicsPSOInit);
				SetGraphicsPipelineState(RHICmdList, LocalGraphicsPSOInit, 0);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

				UE::Renderer::PostProcess::DrawRectangle(RHICmdList, VertexShader, View, EDRF_UseTriangleOptimization);
			});
	}
}

// ---------------------------------------------------------------------------------------------------------------------


/** Pixel shader for blurring the water flow velocity component (XY) of the output of the water info merge pass. */
class FWaterInfoFinalizePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterInfoFinalizePS);
	SHADER_USE_PARAMETER_STRUCT(FWaterInfoFinalizePS, FGlobalShader);

	class FEnable128BitRT : SHADER_PERMUTATION_BOOL("ENABLE_128_BIT");
	using FPermutationDomain = TShaderPermutationDomain<FEnable128BitRT>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WaterInfoTexture)
		SHADER_PARAMETER(float, WaterZMin)
		SHADER_PARAMETER(float, WaterZMax)
		SHADER_PARAMETER(float, GroundZMin)
		SHADER_PARAMETER(float, CaptureZ)
		SHADER_PARAMETER(int, BlurRadius)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain GetPermutationVector(bool bUse128BitRT)
	{
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FEnable128BitRT>(bUse128BitRT);
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		const bool bPlatformRequiresExplicit128bitRT = FDataDrivenShaderPlatformInfo::GetRequiresExplicit128bitRT(Parameters.Platform);
		return (!PermutationVector.Get<FEnable128BitRT>() || bPlatformRequiresExplicit128bitRT);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FEnable128BitRT>())
		{
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_A32B32G32R32F);
		}
	}
};

IMPLEMENT_GLOBAL_SHADER(FWaterInfoFinalizePS, "/Plugin/Water/Private/WaterInfoFinalize.usf", "Main", SF_Pixel);

static void FinalizeWaterInfo(
	FRDGBuilder& GraphBuilder,
	const FSceneViewFamily& ViewFamily,
	const FSceneView& View,
	FRDGTextureRef WaterInfoTexture,
	FRDGTextureRef OutputTexture,
	const FUpdateWaterInfoParams& Params)
{
	RDG_EVENT_SCOPE(GraphBuilder, "WaterInfoFinalize");

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(View.GetFeatureLevel());

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();

	const bool bUse128BitRT = PlatformRequires128bitRT(OutputTexture->Desc.Format);
	const FWaterInfoFinalizePS::FPermutationDomain PixelPermutationVector = FWaterInfoFinalizePS::GetPermutationVector(bUse128BitRT);

	{
		FWaterInfoFinalizePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterInfoFinalizePS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ENoAction);
		PassParameters->SceneTextures = GetSceneTextureShaderParameters(View);
		PassParameters->WaterInfoTexture = WaterInfoTexture;
		PassParameters->WaterZMin = Params.WaterHeightExtents.X;
		PassParameters->WaterZMax = Params.WaterHeightExtents.Y;
		PassParameters->GroundZMin = Params.GroundZMin;
		PassParameters->CaptureZ = Params.CaptureZ;
		PassParameters->BlurRadius = Params.VelocityBlurRadius;

		TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
		TShaderMapRef<FWaterInfoFinalizePS> PixelShader(ShaderMap, PixelPermutationVector);

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("WaterInfoFinalize"),
			PassParameters,
			ERDGPassFlags::Raster,
			[PassParameters, GraphicsPSOInit, VertexShader, PixelShader, &View](FRHICommandListImmediate& RHICmdList)
			{
				FGraphicsPipelineStateInitializer LocalGraphicsPSOInit = GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(LocalGraphicsPSOInit);
				SetGraphicsPipelineState(RHICmdList, LocalGraphicsPSOInit, 0);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

				UE::Renderer::PostProcess::DrawRectangle(RHICmdList, VertexShader, View, EDRF_UseTriangleOptimization);
			});
	}
}
// ---------------------------------------------------------------------------------------------------------------------

static FMatrix BuildOrthoMatrix(float InOrthoWidth, float InOrthoHeight)
{
	const FMatrix::FReal OrthoWidth = InOrthoWidth / 2.0f;
	const FMatrix::FReal OrthoHeight = InOrthoHeight / 2.0f;

	const FMatrix::FReal NearPlane = 0.f;
	const FMatrix::FReal FarPlane = UE_FLOAT_HUGE_DISTANCE / 4.0f;

	const FMatrix::FReal ZScale = 1.0f / (FarPlane - NearPlane);
	const FMatrix::FReal ZOffset = 0;

	return FReversedZOrthoMatrix(
		OrthoWidth,
		OrthoHeight,
		ZScale,
		ZOffset
		);
}

// ---------------------------------------------------------------------------------------------------------------------

void UpdateWaterInfoRendering2(FSceneView& InView, const FRenderingContext& Context, int32 RenderTargetArrayLayer, const FVector& WaterInfoCenter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WaterInfo::UpdateWaterInfoRendering2);

	if (!IsValid(Context.TextureRenderTarget))
	{
		return;
	}

	const FVector ZoneExtent = Context.ZoneToRender->GetDynamicWaterInfoExtent();

	FVector ViewLocation = WaterInfoCenter;
	ViewLocation.Z = Context.CaptureZ;

	const FBox2D CaptureBounds(FVector2D(ViewLocation - ZoneExtent), FVector2D(ViewLocation + ZoneExtent));

	// Zone rendering always happens facing towards negative z.
	const FVector LookAt = ViewLocation - FVector(0.f, 0.f, 1.f);

	FSceneView::FWaterInfoTextureRenderingParams RenderingParams;
	RenderingParams.RenderTarget = Context.TextureRenderTarget->GameThread_GetRenderTargetResource();
	RenderingParams.ViewLocation = ViewLocation;
	RenderingParams.ViewRotationMatrix = FLookAtMatrix(ViewLocation, LookAt, FVector(0.f, -1.f, 0.f));
	RenderingParams.ViewRotationMatrix = RenderingParams.ViewRotationMatrix.RemoveTranslation();
	RenderingParams.ViewRotationMatrix.RemoveScaling();
	RenderingParams.ProjectionMatrix = BuildOrthoMatrix(ZoneExtent.X, ZoneExtent.Y);
	RenderingParams.CaptureZ = ViewLocation.Z;
	RenderingParams.WaterHeightExtents = Context.ZoneToRender->GetWaterHeightExtents();
	RenderingParams.GroundZMin = Context.ZoneToRender->GetGroundZMin();
	RenderingParams.VelocityBlurRadius = FMath::Clamp(Context.ZoneToRender->GetVelocityBlurRadius(), 0, 16);
	RenderingParams.WaterZoneExtents = ZoneExtent;
	RenderingParams.RenderTargetArrayLayer = RenderTargetArrayLayer;

	if (Context.GroundPrimitiveComponents.Num() > 0)
	{
		RenderingParams.TerrainComponentIds.Reserve(Context.GroundPrimitiveComponents.Num());
		for (TWeakObjectPtr<UPrimitiveComponent> GroundPrimComp : Context.GroundPrimitiveComponents)
		{
			if (GroundPrimComp.IsValid())
			{
				RenderingParams.TerrainComponentIds.Add(GroundPrimComp.Get()->GetPrimitiveSceneId());
			}
		}
	}
	if (Context.WaterBodies.Num() > 0)
	{
		RenderingParams.WaterBodyComponentIds.Reserve(Context.WaterBodies.Num());
		RenderingParams.DilatedWaterBodyComponentIds.Reserve(Context.WaterBodies.Num());
		for (const TWeakObjectPtr<UWaterBodyComponent> WaterBodyToRenderPtr : Context.WaterBodies)
		{
			if (UWaterBodyComponent* WaterBodyToRender = WaterBodyToRenderPtr.Get())
			{
				// Perform our own simple culling based on the known Capture bounds:
				const FBox WaterBodyBounds = WaterBodyToRender->Bounds.GetBox();
				if (CaptureBounds.Intersect(FBox2D(FVector2D(WaterBodyBounds.Min), FVector2D(WaterBodyBounds.Max))))
				{
					RenderingParams.WaterBodyComponentIds.Add(WaterBodyToRender->GetWaterInfoMeshComponent()->GetPrimitiveSceneId());
					RenderingParams.DilatedWaterBodyComponentIds.Add(WaterBodyToRender->GetDilatedWaterInfoMeshComponent()->GetPrimitiveSceneId());
				}
			}
		}
	}

	InView.WaterInfoTextureRenderingParams.Add(MoveTemp(RenderingParams));
}


// ----------------------------------------------------------------------------------

/** Base class containing common functionalities for all water info passes. */
class FWaterInfoCustomRenderPassBase : public FCustomRenderPassBase
{
public:
	FWaterInfoCustomRenderPassBase(const FString& InDebugName, FCustomRenderPassBase::ERenderMode InRenderMode, FCustomRenderPassBase::ERenderOutput InRenderOutput, const FIntPoint& InRenderTargetSize)
		: FCustomRenderPassBase(InDebugName, InRenderMode, InRenderOutput, InRenderTargetSize)
	{
	}

	// Abstract class :
	virtual const FName& GetTypeName() const PURE_VIRTUAL(FWaterInfoCustomRenderPassBase::GetTypeName, static FName Name; return Name;);
};


// ----------------------------------------------------------------------------------

class FWaterInfoRenderingDepthPass final : public FWaterInfoCustomRenderPassBase
{
public:
	IMPLEMENT_CUSTOM_RENDER_PASS(FWaterInfoRenderingDepthPass);

	FWaterInfoRenderingDepthPass(const FIntPoint& InRenderTargetSize, const TMap<uint32, int32>& InLandscapeLODOverrides)
		: FWaterInfoCustomRenderPassBase(TEXT("WaterInfoDepthPass"), FCustomRenderPassBase::ERenderMode::DepthPass, FCustomRenderPassBase::ERenderOutput::SceneDepth, InRenderTargetSize)
	{
		if (!InLandscapeLODOverrides.IsEmpty())
		{
			SetUserData(MakeUnique<FLandscapeLODOverridesCustomRenderPassUserData>(InLandscapeLODOverrides));
		}
	}

	virtual void OnPreRender(FRDGBuilder& GraphBuilder) override
	{
		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2D(RenderTargetSize, PF_R32_FLOAT, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);
		RenderTargetTexture = GraphBuilder.CreateTexture(TextureDesc, TEXT("WaterDepthTexture"));
		AddClearRenderTargetPass(GraphBuilder, RenderTargetTexture, FLinearColor::Black, Views[0]->UnscaledViewRect);
	}
};

const FName& GetWaterInfoDepthPassName() { return FWaterInfoRenderingDepthPass::GetTypeNameStatic(); }


// ----------------------------------------------------------------------------------

class FWaterInfoRenderingColorPass final : public FWaterInfoCustomRenderPassBase
{
public:
	IMPLEMENT_CUSTOM_RENDER_PASS(FWaterInfoRenderingColorPass);

	FWaterInfoRenderingColorPass(const FIntPoint& InRenderTargetSize)
		: FWaterInfoCustomRenderPassBase(TEXT("WaterInfoColorPass"), FCustomRenderPassBase::ERenderMode::DepthAndBasePass, FCustomRenderPassBase::ERenderOutput::SceneColorAndDepth, InRenderTargetSize)
	{}

	virtual void OnPreRender(FRDGBuilder& GraphBuilder) override
	{
		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2D(RenderTargetSize, PF_A32B32G32R32F, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);
		RenderTargetTexture = GraphBuilder.CreateTexture(TextureDesc, TEXT("WaterColorTexture"));
		AddClearRenderTargetPass(GraphBuilder, RenderTargetTexture, FLinearColor::Black, Views[0]->UnscaledViewRect);
	}
};

const FName& GetWaterInfoColorPassName() { return FWaterInfoRenderingColorPass::GetTypeNameStatic(); }


// ----------------------------------------------------------------------------------

class FWaterInfoRenderingDilationPass final : public FWaterInfoCustomRenderPassBase
{
public:
	IMPLEMENT_CUSTOM_RENDER_PASS(FWaterInfoRenderingDilationPass);

	FWaterInfoRenderingDilationPass(const FIntPoint& InRenderTargetSize)
		: FWaterInfoCustomRenderPassBase(TEXT("WaterInfoDilationPass"), FCustomRenderPassBase::ERenderMode::DepthPass, FCustomRenderPassBase::ERenderOutput::SceneDepth, InRenderTargetSize)
	{}

	virtual void OnPreRender(FRDGBuilder& GraphBuilder) override
	{
		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2D(RenderTargetSize, PF_R32_FLOAT, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);
		RenderTargetTexture = GraphBuilder.CreateTexture(TextureDesc, TEXT("WaterDilationTexture"));
		AddClearRenderTargetPass(GraphBuilder, RenderTargetTexture, FLinearColor::Black, Views[0]->UnscaledViewRect);
	}
	
	virtual void OnPostRender(FRDGBuilder& GraphBuilder) override
	{
		const FRDGTextureDesc ResultTextureDesc = FRDGTextureDesc::Create2D(RenderTargetSize, WaterInfoRenderTarget->GetRenderTargetTexture()->GetDesc().Format, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);
		
		FRDGTextureRef MergeTargetTexture = GraphBuilder.CreateTexture(ResultTextureDesc, TEXT("WaterInfoMerged"));
		MergeWaterInfoAndDepth(GraphBuilder, *Views[0]->Family, *Views[0], MergeTargetTexture, DepthPass->GetRenderTargetTexture(), ColorPass->GetRenderTargetTexture(), RenderTargetTexture, Params);
	
		FRDGTextureRef FinalizedTexture = GraphBuilder.CreateTexture(ResultTextureDesc, TEXT("WaterInfoFinalized"));
		FinalizeWaterInfo(GraphBuilder, *Views[0]->Family, *Views[0], MergeTargetTexture, FinalizedTexture, Params);

		FRDGTextureRef WaterInfoTexture = RegisterExternalTexture(GraphBuilder, WaterInfoRenderTarget->GetRenderTargetTexture(), TEXT("WaterInfoTexture"));
		GraphBuilder.UseInternalAccessMode(WaterInfoTexture);
		FRHICopyTextureInfo CopyInfo;
		CopyInfo.DestSliceIndex = RenderTargetArraySlice;
		AddCopyTexturePass(GraphBuilder, FinalizedTexture, WaterInfoTexture, CopyInfo);
		GraphBuilder.UseExternalAccessMode(WaterInfoTexture, ERHIAccess::SRVMask);
	}

	FRenderTarget* WaterInfoRenderTarget = nullptr;
	FWaterInfoRenderingDepthPass* DepthPass = nullptr;
	FWaterInfoRenderingColorPass* ColorPass = nullptr;
	FUpdateWaterInfoParams Params;
	int32 RenderTargetArraySlice = 0;
};

const FName& GetWaterInfoDilationPassName() { return FWaterInfoRenderingDilationPass::GetTypeNameStatic(); }


// ----------------------------------------------------------------------------------

static TMap<uint32, int32> GatherLandscapeLODOverrides(const UWorld* World, const FIntPoint& RenderTargetSize, const FVector& WaterZoneExtents)
{
	// In order to prevent overdrawing the landscape components, we compute the lowest-detailed LOD level which satisfies the pixel coverage of the Water Info texture
	// and force it on all landscape components. This override is set different per Landscape actor in case there are multiple under the same water zone.
	//
	// Ex: If the WaterInfoTexture only has 1 pixel per 100 units, and the highest landscape LOD has 1 vertex per 20 units, we don't need to use the maximum landscape LOD
	// and can force a lower level of detail (in this case LOD2) while still satisfying the resolution of the water info texture.

	const double MinWaterInfoTextureExtent = FMath::Min(RenderTargetSize.X, RenderTargetSize.Y);
	const double MaxWaterZoneExtent = FMath::Max(WaterZoneExtents.X, WaterZoneExtents.Y);
	const double WaterInfoUnitsPerPixel =  MaxWaterZoneExtent / MinWaterInfoTextureExtent;

	TMap<uint32, int32> LandscapeLODOverrides;
	for (const ALandscapeProxy* LandscapeProxy : TActorRange<ALandscapeProxy>(World))
	{
		if (LandscapeProxy == nullptr)
		{
			continue;
		}
		
		const uint32 LandscapeKey = LandscapeProxy->ComputeLandscapeKey();
		int32 OptimalLODLevel = INDEX_NONE;

		// All components within the same landscape (and thus its render system) should have the same number of quads and the same extent.
		// therefore we can simply find the first component and compute its optimal LOD level.
		const double FullExtent = LandscapeProxy->SubsectionSizeQuads * FMath::Max(LandscapeProxy->GetTransform().GetScale3D().X, LandscapeProxy->GetTransform().GetScale3D().Y);
		const double NumQuads = LandscapeProxy->ComponentSizeQuads;
		const double LandscapeResolution = FullExtent / NumQuads;

		// Double the required landscape resolution to achieve 2 quads per pixel.
		const double LandscapeComponentUnitsPerQuad = 2.0 * LandscapeResolution;
		check(LandscapeComponentUnitsPerQuad > 0.f);

		// Derived from:
		// (ComponentWorldExtent / WaterInfoWorldspaceExtent) * WaterInfoTextureResolution = (NumComponentQuads / 2 ^ (LodLevel))
		OptimalLODLevel = FMath::Max(WaterInfoRenderLandscapeMinimumMipLevel, FMath::FloorToInt(FMath::Log2(WaterInfoUnitsPerPixel / LandscapeComponentUnitsPerQuad)));

		LandscapeLODOverrides.Add(LandscapeKey, OptimalLODLevel);
	}

	return LandscapeLODOverrides;
}

void UpdateWaterInfoRendering_CustomRenderPass(
	FSceneInterface* Scene,
	const FSceneViewFamily& ViewFamily,
	const WaterInfo::FRenderingContext& Context,
	int32 TextureArraySlice,
	const FVector& WaterInfoCenter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WaterInfo::UpdateWaterInfoRendering_CustomRenderPass);

	if (!IsValid(Context.TextureRenderTarget) || Scene == nullptr)
	{
		return;
	}

	static auto* CVarRenderCaptureNextWaterInfoDraws = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Water.WaterInfo.RenderCaptureNextWaterInfoDraws"));
	int32 RenderCaptureNextWaterInfoDraws = CVarRenderCaptureNextWaterInfoDraws ? CVarRenderCaptureNextWaterInfoDraws->AsVariableInt()->GetValueOnGameThread() : 0;
	bool bPerformRenderCapture = false;
	if (RenderCaptureNextWaterInfoDraws != 0)
	{
		RenderCaptureNextWaterInfoDraws = FMath::Max(0, RenderCaptureNextWaterInfoDraws - 1);
		CVarRenderCaptureNextWaterInfoDraws->SetWithCurrentPriority(RenderCaptureNextWaterInfoDraws);
		bPerformRenderCapture = true;
	}

	const FVector ZoneExtent = Context.ZoneToRender->GetDynamicWaterInfoExtent();

	FVector ViewLocation = WaterInfoCenter;
	ViewLocation.Z = Context.CaptureZ;

	const FBox2D CaptureBounds(FVector2D(ViewLocation - ZoneExtent), FVector2D(ViewLocation + ZoneExtent));

	// Zone rendering always happens facing towards negative z.
	const FVector LookAt = ViewLocation - FVector(0.f, 0.f, 1.f);

	const FIntPoint RenderTargetSize(Context.TextureRenderTarget->GetSurfaceWidth(), Context.TextureRenderTarget->GetSurfaceHeight());

	FTextureRenderTargetResource* TextureRenderTargetResource = Context.TextureRenderTarget->GameThread_GetRenderTargetResource();

	FSceneInterface::FCustomRenderPassRendererInput PassInput;
	PassInput.ViewLocation = ViewLocation;
	PassInput.ViewRotationMatrix = FLookAtMatrix(ViewLocation, LookAt, FVector(0.f, -1.f, 0.f));
	PassInput.ViewRotationMatrix = PassInput.ViewRotationMatrix.RemoveTranslation();
	PassInput.ViewRotationMatrix.RemoveScaling();
	PassInput.ProjectionMatrix = BuildOrthoMatrix(ZoneExtent.X, ZoneExtent.Y);
	PassInput.ViewActor = Context.ZoneToRender;
	PassInput.EngineShowFlags.Decals = 0;

	TSet<FPrimitiveComponentId> ComponentsToRenderInDepthPass;
	if (Context.GroundPrimitiveComponents.Num() > 0)
	{
		ComponentsToRenderInDepthPass.Reserve(Context.GroundPrimitiveComponents.Num());
		for (TWeakObjectPtr<UPrimitiveComponent> GroundPrimComp : Context.GroundPrimitiveComponents)
		{
			if (GroundPrimComp.IsValid())
			{
				ComponentsToRenderInDepthPass.Add(GroundPrimComp.Get()->GetPrimitiveSceneId());
			}
		}
	}
	PassInput.ShowOnlyPrimitives = MoveTemp(ComponentsToRenderInDepthPass);

	FWaterInfoRenderingDepthPass* DepthPass = new FWaterInfoRenderingDepthPass(RenderTargetSize, GatherLandscapeLODOverrides(Scene->GetWorld(), RenderTargetSize, ZoneExtent));
	if (bPerformRenderCapture)
	{
		// Initiate a render capture when this pass runs :
		DepthPass->PerformRenderCapture(FCustomRenderPassBase::ERenderCaptureType::BeginCapture);
	}
	PassInput.CustomRenderPass = DepthPass;
	Scene->AddCustomRenderPass(&ViewFamily, PassInput);

	TSet<FPrimitiveComponentId> ComponentsToRenderInColorPass;
	TSet<FPrimitiveComponentId> ComponentsToRenderInDilationPass;
	if (Context.WaterBodies.Num() > 0)
	{
		ComponentsToRenderInColorPass.Reserve(Context.WaterBodies.Num());
		ComponentsToRenderInDilationPass.Reserve(Context.WaterBodies.Num());
		for (const TWeakObjectPtr<UWaterBodyComponent> WaterBodyToRenderPtr : Context.WaterBodies)
		{
			if (UWaterBodyComponent* WaterBodyToRender = WaterBodyToRenderPtr.Get())
			{
				// Perform our own simple culling based on the known Capture bounds:
				const FBox WaterBodyBounds = WaterBodyToRender->Bounds.GetBox();
				if (CaptureBounds.Intersect(FBox2D(FVector2D(WaterBodyBounds.Min), FVector2D(WaterBodyBounds.Max))))
				{
					ComponentsToRenderInColorPass.Add(WaterBodyToRender->GetWaterInfoMeshComponent()->GetPrimitiveSceneId());
					ComponentsToRenderInDilationPass.Add(WaterBodyToRender->GetDilatedWaterInfoMeshComponent()->GetPrimitiveSceneId());
				}
			}
		}
	}
	PassInput.ShowOnlyPrimitives = MoveTemp(ComponentsToRenderInColorPass);
	FWaterInfoRenderingColorPass* ColorPass = new FWaterInfoRenderingColorPass(RenderTargetSize);
	PassInput.CustomRenderPass = ColorPass;
	Scene->AddCustomRenderPass(&ViewFamily, PassInput);

	FUpdateWaterInfoParams Params;
	Params.CaptureZ = ViewLocation.Z;
	Params.WaterHeightExtents = Context.ZoneToRender->GetWaterHeightExtents();
	Params.GroundZMin = Context.ZoneToRender->GetGroundZMin();
	Params.VelocityBlurRadius = FMath::Clamp(Context.ZoneToRender->GetVelocityBlurRadius(), 0, 16);
	Params.WaterZoneExtents = ZoneExtent;

	PassInput.ShowOnlyPrimitives = MoveTemp(ComponentsToRenderInDilationPass);
	FWaterInfoRenderingDilationPass* DilationPass = new FWaterInfoRenderingDilationPass(RenderTargetSize);
	DilationPass->DepthPass = DepthPass;
	DilationPass->ColorPass = ColorPass;
	DilationPass->WaterInfoRenderTarget = Context.TextureRenderTarget->GameThread_GetRenderTargetResource();
	DilationPass->Params = Params;
	DilationPass->RenderTargetArraySlice = TextureArraySlice;
	if (bPerformRenderCapture)
	{
		// End a render capture when this pass runs :
		DilationPass->PerformRenderCapture(FCustomRenderPassBase::ERenderCaptureType::EndCapture);
	}
	PassInput.CustomRenderPass = DilationPass;
	Scene->AddCustomRenderPass(&ViewFamily, PassInput);
}

} // namespace WaterInfo
