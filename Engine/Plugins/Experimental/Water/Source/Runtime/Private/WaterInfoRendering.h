// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtrTemplates.h"

class AWaterZone;
class UWaterBodyComponent;
class FSceneInterface;
class UTextureRenderTarget2DArray;
class UPrimitiveComponent;
class FSceneView;
class FSceneViewFamily;


template <typename KeyType, typename ValueType>
using TWeakObjectPtrKeyMap = TMap<TWeakObjectPtr<KeyType>, ValueType, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<KeyType>, ValueType>>;

namespace UE::WaterInfo
{

struct FRenderingContext
{
	AWaterZone* ZoneToRender = nullptr;
	UTextureRenderTarget2DArray* TextureRenderTarget;
	TArray<TWeakObjectPtr<UWaterBodyComponent>> WaterBodies;
	TArray<TWeakObjectPtr<UPrimitiveComponent>> GroundPrimitiveComponents;
	float CaptureZ;
};

void UpdateWaterInfoRendering2(
	FSceneView& InView, 
	const FRenderingContext& Context,
	int32 RenderTargetArrayLayer,
	const FVector& WaterInfoCenter);

void UpdateWaterInfoRendering_CustomRenderPass(
	FSceneInterface* Scene,
	const FSceneViewFamily& ViewFamily,
	const FRenderingContext& Context,
	int32 TextureArraySlice,
	const FVector& WaterInfoCenter);

const FName& GetWaterInfoDepthPassName();
const FName& GetWaterInfoColorPassName();
const FName& GetWaterInfoDilationPassName();

} // namespace UE::WaterInfo
