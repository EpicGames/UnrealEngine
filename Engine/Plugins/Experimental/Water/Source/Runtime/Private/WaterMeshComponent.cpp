// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterMeshComponent.h"
#include "EngineUtils.h"
#include "MaterialDomain.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/Material.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/StaticMesh.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "WaterBodyComponent.h"
#include "WaterMeshSceneProxy.h"
#include "WaterModule.h"
#include "WaterSplineComponent.h"
#include "WaterSubsystem.h"
#include "WaterUtils.h"
#include "WaterBodyInfoMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterMeshComponent)

/** Scalability CVars*/
static TAutoConsoleVariable<int32> CVarWaterMeshLODCountBias(
	TEXT("r.Water.WaterMesh.LODCountBias"), 0,
	TEXT("This value is added to the LOD Count of each Water Mesh Component. Negative values will lower the quality(fewer and larger water tiles at the bottom level of the water quadtree), higher values will increase quality (more and smaller water tiles at the bottom level of the water quadtree)"),
	ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarWaterMeshTessFactorBias(
	TEXT("r.Water.WaterMesh.TessFactorBias"), 0,
	TEXT("This value is added to the tessellation factor of each Mesh Component. Negative values will lower the overall density/resolution or the vertex grid, higher values will increase the density/resolution "),
	ECVF_Scalability);

static TAutoConsoleVariable<float> CVarWaterMeshLODScaleBias(
	TEXT("r.Water.WaterMesh.LODScaleBias"), 0.0f,
	TEXT("This value is added to the LOD Scale of each Mesh Component. Negative values will lower the overall density/resolution or the vertex grid and make the LODs smaller, higher values will increase the density/resolution and make the LODs larger. Smallest value is -0.5. That will make the inner LOD as tight and optimized as possible"),
	ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarWaterMeshMaxDimensionInTiles(
	TEXT("r.Water.WaterMesh.MaxDimensionInTiles"), 256,
	TEXT("Maximum dimension of a water mesh in tiles (max of either X or Y). If a water mesh attempts to generate a quadtree with more than this many tiles in X or Y, it will be biased back below this threshold via the same process as the LODScaleBias (iteratively dividing tile size by 2 until compliance). Having too many tiles can create very large GPU allocations."),
	ECVF_Scalability);

TAutoConsoleVariable<int32> CVarWaterMeshGPUQuadTree(
	TEXT("r.Water.WaterMesh.GPUQuadTree"),
	0,
	TEXT("Builds the water quadtree on the GPU and does indirect draws of water tiles, driven by the GPU."),
	ECVF_RenderThreadSafe
);

/** Debug CVars */ 
static TAutoConsoleVariable<int32> CVarWaterMeshShowTileGenerationGeometry(
	TEXT("r.Water.WaterMesh.ShowTileGenerationGeometry"),
	0,
	TEXT("This debug option will display the geometry used for intersecting the water grid and generating tiles"),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarWaterMeshForceRebuildMeshPerFrame(
	TEXT("r.Water.WaterMesh.ForceRebuildMeshPerFrame"),
	0,
	TEXT("Force rebuilding the entire mesh each frame"),
	ECVF_Default
);

TAutoConsoleVariable<int32> CVarWaterMeshEnabled(
	TEXT("r.Water.WaterMesh.Enabled"),
	1,
	TEXT("If the water mesh is enabled or disabled. This affects both rendering and the water tile generation"),
	ECVF_RenderThreadSafe
);

extern TAutoConsoleVariable<float> CVarWaterSplineResampleMaxDistance;


// ----------------------------------------------------------------------------------

UWaterMeshComponent::UWaterMeshComponent()
{
	bAutoActivate = true;
	bHasPerInstanceHitProxies = true;

	SetMobility(EComponentMobility::Static);
}

void UWaterMeshComponent::PostLoad()
{
	Super::PostLoad();
}

void UWaterMeshComponent::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	const FVertexFactoryType* WaterVertexFactoryNonIndirectType = GetWaterVertexFactoryType(/*bWithWaterSelectionSupport = */ false, EWaterVertexFactoryDrawMode::NonIndirect);
	const FVertexFactoryType* WaterVertexFactoryIndirectType = GetWaterVertexFactoryType(/*bWithWaterSelectionSupport = */ false, EWaterVertexFactoryDrawMode::Indirect);
	const FVertexFactoryType* WaterVertexFactoryIndirectISRType = GetWaterVertexFactoryType(/*bWithWaterSelectionSupport = */ false, EWaterVertexFactoryDrawMode::IndirectInstancedStereo);
	if (FarDistanceMaterial)
	{
		FMaterialInterfacePSOPrecacheParams& ComponentParams = OutParams[OutParams.AddDefaulted()];
		ComponentParams.Priority = EPSOPrecachePriority::High;
		ComponentParams.MaterialInterface = FarDistanceMaterial;
		ComponentParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(WaterVertexFactoryNonIndirectType));
		ComponentParams.PSOPrecacheParams = BasePrecachePSOParams;
	}
	for (UMaterialInterface* MaterialInterface : UsedMaterials)
	{
		if (MaterialInterface)
		{
			FMaterialInterfacePSOPrecacheParams& ComponentParams = OutParams[OutParams.AddDefaulted()];
			ComponentParams.Priority = EPSOPrecachePriority::High;
			ComponentParams.MaterialInterface = MaterialInterface;
			ComponentParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(WaterVertexFactoryNonIndirectType));
			ComponentParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(WaterVertexFactoryIndirectType));
			ComponentParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(WaterVertexFactoryIndirectISRType));
			ComponentParams.PSOPrecacheParams = BasePrecachePSOParams;
		}
	}
}

void UWaterMeshComponent::PostInitProperties()
{
	Super::PostInitProperties();

	UpdateBounds();
	MarkRenderTransformDirty();
}

FPrimitiveSceneProxy* UWaterMeshComponent::CreateSceneProxy()
{
	// Early out
	if (!bIsEnabled)
	{
		return nullptr;
	}
	
	if (CheckPSOPrecachingAndBoostPriority(EPSOPrecachePriority::Highest) && GetPSOPrecacheProxyCreationStrategy() != EPSOPrecacheProxyCreationStrategy::AlwaysCreate)
	{
		UE_LOGF(LogWater, Verbose, "Skipping CreateSceneProxy for UWaterMeshComponent %ls (Its PSOs are still compiling)", *GetFullName());
		return nullptr;
	}

	return new FWaterMeshSceneProxy(this);
}

void UWaterMeshComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	for (UMaterialInterface* Mat : UsedMaterials)
	{
		if (Mat)
		{
			OutMaterials.Add(Mat);
		}
	}
}

void UWaterMeshComponent::SetMaterial(int32 ElementIndex, UMaterialInterface* Material)
{
	UE_LOGF(LogWater, Warning, "SetMaterial is not compatible with UWaterMeshComponent since all materials on this component are auto-populated from the Water Bodies contained within it.");
}

#if WITH_EDITOR

bool UWaterMeshComponent::ShouldRenderSelected() const
{
	if (bSelectable)
	{
		bool bShouldRender = Super::ShouldRenderSelected();
		if (!bShouldRender)
		{
			if (AWaterZone* Owner = GetOwner<AWaterZone>())
			{
				Owner->ForEachWaterBodyComponent([&bShouldRender](UWaterBodyComponent* WaterBodyComponent)
				{
					check(WaterBodyComponent);
					bShouldRender |= WaterBodyComponent->ShouldRenderSelected();

					// Stop iterating over water body components by returning false as soon as one component says it should be "render selected" :
					return !bShouldRender;
				});
			}
		}

		return bShouldRender;
	}
	return false;
}

#endif // WITH_EDITOR

// Deprecated 5.7
FMaterialRelevance UWaterMeshComponent::GetWaterMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return GetWaterMaterialRelevance(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

FMaterialRelevance UWaterMeshComponent::GetWaterMaterialRelevance(EShaderPlatform InShaderPlatform) const
{
	// Combine the material relevance for all materials.
	FMaterialRelevance Result;
	for (UMaterialInterface* Mat : UsedMaterials)
	{
		Result |= Mat->GetRelevance_Concurrent(InShaderPlatform);
	}

	return Result;
}

FVector2D UWaterMeshComponent::GetGlobalWaterMeshCenter() const
{
	const float LODCountBiasFactor = FMath::Pow(2.0f, (float)CVarWaterMeshLODCountBias.GetValueOnGameThread());
	const float EffectiveTileSize = TileSize / LODCountBiasFactor;
	FVector2D Result = FVector2D(FMath::GridSnap<FVector::FReal>(GetComponentLocation().X, EffectiveTileSize), FMath::GridSnap<FVector::FReal>(GetComponentLocation().Y, EffectiveTileSize));
	return Result;
}

bool UWaterMeshComponent::IsLocalOnlyTessellationEnabled() const
{
	if (const AWaterZone* WaterZone = GetOwner<AWaterZone>())
	{
		return WaterZone->IsLocalOnlyTessellationEnabled();
	}
	return false;
}

void UWaterMeshComponent::SetTileSize(float NewTileSize)
{
	TileSize = NewTileSize;
	MarkWaterMeshGridDirty();
	MarkRenderStateDirty();
}

FIntPoint UWaterMeshComponent::GetExtentInTiles() const
{
	if (const AWaterZone* WaterZone = GetOwner<AWaterZone>(); ensureMsgf(WaterZone != nullptr, TEXT("WaterMeshComponent is owned by an actor that is not a WaterZone. This is not supported!")))
	{
		const float MeshTileSize = TileSize;
		const FVector2D ZoneHalfExtent = FVector2D(WaterZone->GetDynamicWaterInfoExtent()) / 2.0;
		const int32 HalfExtentInTilesX = FMath::RoundUpToPowerOfTwo(ZoneHalfExtent.X / MeshTileSize);
		const int32 HalfExtentInTilesY = FMath::RoundUpToPowerOfTwo(ZoneHalfExtent.Y / MeshTileSize);
		const FIntPoint HalfExtentInTiles = FIntPoint(HalfExtentInTilesX, HalfExtentInTilesY);

		// QuadTreeResolution caches the resolution so it is clearly visible to the user in the details panel. It represents the full extent rather than the half extent
		QuadTreeResolution = HalfExtentInTiles * 2;

		return HalfExtentInTiles;
	}

	return FIntPoint(1, 1);
}

FBoxSphereBounds UWaterMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox NewBounds = FBox(ForceInit);
	
	// With the water quadtree building moved into the scene proxy, the scene proxy is free to create quadtree(s) anywhere within the water zone.
	// To ensure that the scene proxy isn't frustum culled, we need to be conservative and use the water zone bounds instead. The quadtree does
	// fine grained frustum culling internally anyways, so this shouldn't be a performance issue.
	AWaterZone* WaterZone = Cast<AWaterZone>(GetOwner());
	if (WaterZone)
	{
		NewBounds = WaterZone->GetZoneBounds();
	}
	
	// Add the far distance to the bounds if it's valid
	if (FarDistanceMaterial)
	{
		NewBounds = NewBounds.ExpandBy(FVector(FarDistanceMeshExtent, FarDistanceMeshExtent, 0.0f));
	}
	return NewBounds;
}

static bool IsMaterialUsedWithWater(const UMaterialInterface* InMaterial)
{
	return (InMaterial && InMaterial->CheckMaterialUsage_Concurrent(EMaterialUsage::MATUSAGE_Water));
}

void UWaterMeshComponent::RebuildWaterMesh(float InTileSize, const FIntPoint& InExtentInTiles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWaterMeshComponent::RebuildWaterMesh);

	AWaterZone* WaterZone = CastChecked<AWaterZone>(GetOwner());

	bool bAnyWaterMeshesNotReady = false;
	const bool bIsGPUQuadTree = CVarWaterMeshGPUQuadTree.GetValueOnGameThread() != 0;
	
	UMaterialInterface* FarDistanceMaterialInterface = IsMaterialUsedWithWater(FarDistanceMaterial) ? FarDistanceMaterial.Get() : nullptr;
	WaterQuadTreeBuilder.Init(WaterZone->GetZoneBounds2D(), InExtentInTiles, InTileSize, FarDistanceMaterialInterface, FarDistanceMeshExtent, FarDistanceMeshHeightWithoutOcean, bUseFarMeshWithoutOcean, bIsGPUQuadTree);

	UsedMaterials.Empty();

	bool bUsesFarDistanceMaterial = bUseFarMeshWithoutOcean;

	WaterZone->ForEachWaterBodyComponent([&](UWaterBodyComponent* WaterBodyComponent) -> bool
	{
		check(WaterBodyComponent);
		AActor* Actor = WaterBodyComponent->GetOwner();
		check(Actor);

		// Skip invisible water bodies
		if (!WaterBodyComponent->ShouldRender() || !WaterBodyComponent->ShouldGenerateWaterMeshTile())
		{
			return true;
		}

		UWaterBodyInfoMeshComponent* WaterBodyInfoMeshComponent = WaterBodyComponent->GetWaterInfoMeshComponent();
		UStaticMesh* StaticMesh = WaterBodyInfoMeshComponent ? WaterBodyInfoMeshComponent->GetStaticMesh().Get() : nullptr;

		if (StaticMesh == nullptr)
		{
			return true;
		}

		bAnyWaterMeshesNotReady |= StaticMesh->IsCompiling();
		FStaticMeshRenderData* StaticMeshRenderData = StaticMesh->GetRenderData();
		if (!StaticMeshRenderData)
		{
			return true;
		}

		bUsesFarDistanceMaterial = bUsesFarDistanceMaterial || (WaterBodyComponent->GetWaterBodyType() == EWaterBodyType::Ocean);

		auto GetMaterialInterface = [&](UMaterialInstanceDynamic* MID, bool bUseFallback) -> UMaterialInterface*
		{
			UMaterialInterface* MaterialInterface = nullptr;
			if (!MID || !IsMaterialUsedWithWater(MID))
			{
				MaterialInterface = bUseFallback ? UMaterial::GetDefaultMaterial(MD_Surface) : nullptr;
			}
			else
			{
				WaterBodyComponent->SetDynamicParametersOnMID(MID);
				MaterialInterface = MID;
			}
			if (MaterialInterface)
			{
				UsedMaterials.Add(MaterialInterface);
				return MaterialInterface;
			}
			return nullptr;
		};

		const bool bIsRiver = WaterBodyComponent->GetWaterBodyType() == EWaterBodyType::River;

		FWaterQuadTreeBuilder::FWaterBody WaterBody = {};
		WaterBody.Material = GetMaterialInterface(WaterBodyComponent->GetWaterMaterialInstance(), false);
		WaterBody.RiverToLakeMaterial = bIsRiver ? GetMaterialInterface(WaterBodyComponent->GetRiverToLakeTransitionMaterialInstance(), false) : nullptr;
		WaterBody.RiverToOceanMaterial = bIsRiver ? GetMaterialInterface(WaterBodyComponent->GetRiverToOceanTransitionMaterialInstance(), false) : nullptr;
		WaterBody.StaticMeshRenderData = StaticMeshRenderData;
		WaterBody.LocalToWorld = WaterBodyComponent->GetComponentTransform();
		WaterBody.Bounds = WaterBodyComponent->Bounds;
		WaterBody.OverlapMaterialPriority = WaterBodyComponent->GetOverlapMaterialPriority();
		WaterBody.Type = WaterBodyComponent->GetWaterBodyType();
		WaterBody.WaterBodyIndex = WaterBodyComponent->GetWaterBodyIndex();
		WaterBody.SurfaceBaseHeight = WaterBodyComponent->GetComponentLocation().Z;
		WaterBody.MaxWaveHeight = WaterBodyComponent->GetMaxWaveHeight();
#if WITH_WATER_SELECTION_SUPPORT
		WaterBody.HitProxy = new HActor(/*InActor = */Actor, /*InPrimComponent = */nullptr);
		WaterBody.bWaterBodySelected = Actor->IsSelected();
#endif // WITH_WATER_SELECTION_SUPPORT

		if (!bIsGPUQuadTree)
		{
			switch (WaterBody.Type)
			{
			case EWaterBodyType::River:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(River);

				TArray<FBox, TInlineAllocator<16>> Boxes;
				TArray<UPrimitiveComponent*> CollisionComponents = WaterBodyComponent->GetCollisionComponents();
				for (UPrimitiveComponent* Comp : CollisionComponents)
				{
					if (UBodySetup* BodySetup = (Comp != nullptr) ? Comp->GetBodySetup() : nullptr)
					{
						// Go through all sub shapes on the bodysetup to get a tight fit along water body
						for (const FKConvexElem& ConvElem : BodySetup->AggGeom.ConvexElems)
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(Add);

							FBox SubBox = ConvElem.ElemBox.TransformBy(Comp->GetComponentTransform().ToMatrixWithScale());
							SubBox.Max.Z += WaterBodyComponent->GetMaxWaveHeight();

							Boxes.Add(SubBox);
						}
					}
					else
					{
						// fallback on global AABB: 
						FVector Center;
						FVector Extent;
						Actor->GetActorBounds(false, Center, Extent);
						FBox Box(FBox::BuildAABB(Center, Extent));
						Box.Max.Z += WaterBodyComponent->GetMaxWaveHeight();
						Boxes.Add(Box);
					}
				}
				
				for (const FBox& Box : Boxes)
				{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					if (!!CVarWaterMeshShowTileGenerationGeometry.GetValueOnGameThread())
					{
						DrawDebugBox(GetWorld(), Box.GetCenter(), Box.GetExtent(), FColor::Red);
					}
#endif
				}
				WaterBody.RiverBoxes = Boxes;
				break;
			}
			case EWaterBodyType::Lake:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Lake);

				const UWaterSplineComponent* SplineComp = WaterBodyComponent->GetWaterSpline();
				const int32 NumOriginalSplinePoints = SplineComp->GetNumberOfSplinePoints();
				float ConstantZ = SplineComp->GetLocationAtDistanceAlongSpline(0.0f, ESplineCoordinateSpace::World).Z;

				FBox LakeBounds = Actor->GetComponentsBoundingBox(/* bNonColliding = */true);
				LakeBounds.Max.Z += WaterBodyComponent->GetMaxWaveHeight();

				// Skip lakes with less than 3 spline points
				if (NumOriginalSplinePoints < 3)
				{
					return true;
				}

				TArray<TArray<FVector2D>> PolygonBatches;
				// Reuse the convex hulls generated for the physics shape because the work has already been done, but we can fallback to a simple spline evaluation method in case there's no physics:
				bool bUseFallbackMethod = true;

				TArray<UPrimitiveComponent*> CollisionComponents = WaterBodyComponent->GetCollisionComponents();
				if (CollisionComponents.Num() > 0)
				{
					UPrimitiveComponent* Comp = CollisionComponents[0];
					if (UBodySetup* BodySetup = (Comp != nullptr) ? Comp->GetBodySetup() : nullptr)
					{
						FTransform CompTransform = Comp->GetComponentTransform();
						// Go through all sub shapes on the bodysetup to get a tight fit along water body
						for (const FKConvexElem& ConvElem : BodySetup->AggGeom.ConvexElems)
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(Add);

							// Vertex data contains the bottom vertices first, then the top ones :
							int32 TotalNumVertices = ConvElem.VertexData.Num();
							check(TotalNumVertices % 2 == 0);
							int32 NumVertices = TotalNumVertices / 2;
							if (NumVertices > 0)
							{
								bUseFallbackMethod = false;

								// Because the physics shape is made of multiple convex hulls, we cannot simply add their vertices to one big list but have to have 1 batch of polygons per convex hull
								TArray<FVector2D>& Polygon = PolygonBatches.Emplace_GetRef();
								Polygon.SetNum(NumVertices);
								for (int32 i = 0; i < NumVertices; ++i)
								{
									// Gather the top vertices :
									Polygon[i] = FVector2D(CompTransform.TransformPosition(ConvElem.VertexData[NumVertices + i]));
								}
							}
						}
					}
				}

				if (bUseFallbackMethod)
				{
					TArray<FVector> PolyLineVertices;
					SplineComp->ConvertSplineToPolyLine(ESplineCoordinateSpace::World, FMath::Square(CVarWaterSplineResampleMaxDistance.GetValueOnGameThread()), PolyLineVertices);

					TArray<FVector2D>& Polygon = PolygonBatches.Emplace_GetRef();
					Polygon.Reserve(PolyLineVertices.Num());
					Algo::Transform(PolyLineVertices, Polygon, [](const FVector& Vertex) { return FVector2D(Vertex); });
				}

				for (const TArray<FVector2D>& Polygon : PolygonBatches)
				{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					if (!!CVarWaterMeshShowTileGenerationGeometry.GetValueOnGameThread())
					{
						float Z = SplineComp->GetLocationAtDistanceAlongSpline(0.0f, ESplineCoordinateSpace::World).Z;
						int32 NumVertices = Polygon.Num();
						for (int32 i = 0; i < NumVertices; i++)
						{
							const FVector2D& Point0 = Polygon[i];
							const FVector2D& Point1 = Polygon[(i + 1) % NumVertices];
							DrawDebugLine(GetWorld(), FVector(Point0.X, Point0.Y, Z), FVector(Point1.X, Point1.Y, Z), FColor::Green);
						}
					}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				}

				WaterBody.PolygonBounds = LakeBounds;
				WaterBody.PolygonBatches = MoveTemp(PolygonBatches);

				break;
			}
			case EWaterBodyType::Ocean:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Ocean);

				const UWaterSplineComponent* SplineComp = WaterBodyComponent->GetWaterSpline();
				const int32 NumOriginalSplinePoints = SplineComp->GetNumberOfSplinePoints();

				// Skip oceans with less than 3 spline points
				if (NumOriginalSplinePoints < 3)
				{
					return true;
				}

				TArray<FVector2D> Polygon;

				TArray<FVector> PolyLineVertices;
				SplineComp->ConvertSplineToPolyLine(ESplineCoordinateSpace::World, FMath::Square(CVarWaterSplineResampleMaxDistance.GetValueOnGameThread()), PolyLineVertices);

				Polygon.Reserve(PolyLineVertices.Num());
				Algo::Transform(PolyLineVertices, Polygon, [](const FVector& Vertex) { return FVector2D(Vertex); });

				FBox OceanBounds = WaterBodyComponent->Bounds.GetBox();
				OceanBounds.Max.Z += WaterBodyComponent->GetMaxWaveHeight();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (!!CVarWaterMeshShowTileGenerationGeometry.GetValueOnGameThread())
				{
					float Z = SplineComp->GetLocationAtDistanceAlongSpline(0.0f, ESplineCoordinateSpace::World).Z;
					int32 NumVertices = Polygon.Num();
					for (int32 i = 0; i < NumVertices; i++)
					{
						const FVector2D& Point0 = Polygon[i];
						const FVector2D& Point1 = Polygon[(i + 1) % NumVertices];
						DrawDebugLine(GetWorld(), FVector(Point0.X, Point0.Y, Z), FVector(Point1.X, Point1.Y, Z), FColor::Blue);
					}

					DrawDebugBox(GetWorld(), OceanBounds.GetCenter(), OceanBounds.GetExtent(), FColor::Blue);
				}
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)

				WaterBody.PolygonBounds = OceanBounds;
				WaterBody.PolygonBatches.Add(MoveTemp(Polygon));

				break;
			}
			case EWaterBodyType::Transition:
				// Transitions dont require rendering
				break;
			default:
				ensureMsgf(false, TEXT("This water body type is not implemented and will not produce any water tiles. "));
			}
		}

		WaterQuadTreeBuilder.AddWaterBody(MoveTemp(WaterBody));

		return true;
	});

	if (bUsesFarDistanceMaterial && FarDistanceMaterialInterface && (FarDistanceMeshExtent > 0.0f))
	{
		UsedMaterials.Add(FarDistanceMaterial);
	}

	MarkRenderStateDirty();

	// Force another rebuild next frame if water body meshes are still compiling
	if (bIsGPUQuadTree && bAnyWaterMeshesNotReady)
	{
		bNeedsRebuild = true;
	}
}

void UWaterMeshComponent::Update()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWaterMeshComponent::Update);
	bIsEnabled = FWaterUtils::IsWaterMeshEnabled(/*bIsRenderThread = */false) && FApp::CanEverRender();

	// Early out
	if (!bIsEnabled)
	{
		return;
	}

	const int32 NewLODCountBias = CVarWaterMeshLODCountBias.GetValueOnGameThread();
	const int32 NewTessFactorBias = CVarWaterMeshTessFactorBias.GetValueOnGameThread();
	const float NewLODScaleBias = CVarWaterMeshLODScaleBias.GetValueOnGameThread();
	if (bNeedsRebuild 
		|| !!CVarWaterMeshShowTileGenerationGeometry.GetValueOnGameThread() 
		|| !!CVarWaterMeshForceRebuildMeshPerFrame.GetValueOnGameThread() 
		|| (NewLODCountBias != LODCountBiasScalability)
		|| (NewTessFactorBias != TessFactorBiasScalability) 
		|| (NewLODScaleBias != LODScaleBiasScalability))
	{
		LODCountBiasScalability = NewLODCountBias;
		TessFactorBiasScalability = NewTessFactorBias;
		LODScaleBiasScalability = NewLODScaleBias;
		const FIntPoint ExtentInTiles = GetExtentInTiles();

		const int32 MaxDimensionInTiles = CVarWaterMeshMaxDimensionInTiles.GetValueOnGameThread();

		int32 Bias = 0;
		float LODCountBiasFactor;
		do
		{
			LODCountBiasFactor = FMath::Pow(2.0f, (float)LODCountBiasScalability - Bias);
		}
		// Cap the loop at 100 iterations in case users input a bad MaxExtentInTiles.
		// note: the shortcircuiting of the increment of bias in this condition evaluation is import for detecting if the loop falls through or was executed.
		while (ExtentInTiles.GetMax() * LODCountBiasFactor > MaxDimensionInTiles && ++Bias < 100);

		if (Bias != 0)
		{
			UE_LOGF(LogWater, Warning, "Width of water quad tree tiles (%d) for Water Mesh Component (%ls) has exceeded the cap for this platform (%d). Tile sizes have been biased by a factor of %.2f to prevent exceeding the cap. If a high tile count is intended by the user, the scalability cvar `r.Water.WaterMesh.MaxWidthInTiles` should be adjusted accordingly.", ExtentInTiles.GetMax(), *GetPathName(), MaxDimensionInTiles, LODCountBiasFactor);
		}

		RebuildWaterMesh(TileSize / LODCountBiasFactor, FIntPoint(FMath::CeilToInt(ExtentInTiles.X * LODCountBiasFactor), FMath::CeilToInt(ExtentInTiles.Y * LODCountBiasFactor)));
		PrecachePSOs();
		bNeedsRebuild = false;
	}
}

#if WITH_EDITOR
void UWaterMeshComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FProperty* PropertyThatChanged = PropertyChangedEvent.MemberProperty;
	if (PropertyThatChanged)
	{
		const FName PropertyName = PropertyThatChanged->GetFName();

		// Properties that needs the scene proxy to be rebuilt
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaterMeshComponent, LODScale)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UWaterMeshComponent, TessellationFactor)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UWaterMeshComponent, TileSize)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UWaterMeshComponent, ForceCollapseDensityLevel)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UWaterMeshComponent, FarDistanceMaterial)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UWaterMeshComponent, FarDistanceMeshExtent)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UWaterMeshComponent, FarDistanceMeshHeightWithoutOcean)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UWaterMeshComponent, bUseFarMeshWithoutOcean))
		{
			MarkWaterMeshGridDirty();
			MarkRenderStateDirty();
		}
	}
}
#endif

