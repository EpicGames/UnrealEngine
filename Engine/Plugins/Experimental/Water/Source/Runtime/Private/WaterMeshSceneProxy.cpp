// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterMeshSceneProxy.h"
#include "MaterialShared.h"
#include "WaterMeshComponent.h"
#include "WaterZoneActor.h"
#include "WaterViewExtension.h"
#include "Materials/Material.h"
#include "WaterUtils.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneRendering.h"
#include "Materials/MaterialRenderProxy.h"
#include "Math/ColorList.h"
#include "RayTracingInstance.h"
#include "StaticMeshResources.h"
#include "SceneInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "StereoRenderUtils.h"
#include "WaterSubsystem.h"

DECLARE_STATS_GROUP(TEXT("Water Mesh"), STATGROUP_WaterMesh, STATCAT_Advanced);

DECLARE_DWORD_COUNTER_STAT(TEXT("Tiles Drawn"), STAT_WaterTilesDrawn, STATGROUP_WaterMesh);
DECLARE_DWORD_COUNTER_STAT(TEXT("Tiles Occlusion Culled"), STAT_WaterTilesOcclusionCulled, STATGROUP_WaterMesh);
DECLARE_DWORD_COUNTER_STAT(TEXT("Draw Calls"), STAT_WaterDrawCalls, STATGROUP_WaterMesh);
DECLARE_DWORD_COUNTER_STAT(TEXT("Vertices Drawn"), STAT_WaterVerticesDrawn, STATGROUP_WaterMesh);
DECLARE_DWORD_COUNTER_STAT(TEXT("Number Drawn Materials"), STAT_WaterDrawnMats, STATGROUP_WaterMesh);
DECLARE_DWORD_COUNTER_STAT(TEXT("Number Occlusion Queries"), STAT_WaterOcclusionQueries, STATGROUP_WaterMesh);

/** Scalability CVars */
static TAutoConsoleVariable<int32> CVarWaterMeshLODMorphEnabled(
	TEXT("r.Water.WaterMesh.LODMorphEnabled"), 1,
	TEXT("If the smooth LOD morph is enabled. Turning this off may cause slight popping between LOD levels but will skip the calculations in the vertex shader, making it cheaper"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarWaterMeshGPUQuadTreeSuperSampling(
	TEXT("r.Water.WaterMesh.GPUQuadTree.SuperSampling"), 2,
	TEXT("Rasterizes water meshes into the GPU water quadtree at a higher resolution, reducing missing water tile artifacts near the edges of water bodies. Default: 2, Min: 1, Max : 8"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarWaterMeshGPUQuadTreeMultiSampling(
	TEXT("r.Water.WaterMesh.GPUQuadTree.MultiSampling"), 1,
	TEXT("Rasterizes water meshes into the GPU water quadtree with a multisampled rendertarget, reducing missing water tile artifacts near the edges of water bodies. Default: 1, Min: 1, Max : 8"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarWaterMeshGPUQuadTreeNumJitterSamples(
	TEXT("r.Water.WaterMesh.GPUQuadTree.NumJitterSamples"), 4,
	TEXT("Rasterizes water meshes into the GPU water quadtree with multiple jittered draw calls, reducing missing water tile artifacts near the edges of water bodies. Default: 4, Min: 1, Max : 16. 1 disables this feature."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarWaterMeshGPUQuadTreeJitterPattern(
	TEXT("r.Water.WaterMesh.GPUQuadTree.JitterPattern"), 1,
	TEXT("Jitter pattern when using multiple jittered draw calls to rasterize water meshes into the GPU water quadtree. 0: Halton, 1: MSAA. Default 1"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarWaterMeshGPUQuadTreeJitterSampleFootprint(
	TEXT("r.Water.WaterMesh.GPUQuadTree.JitterSampleFootprint"), 1.5f,
	TEXT("Pixel footprint of the jitter sample pattern. Values greater than 1.0 can cause the water mesh to raster into neighboring pixels not normally covered by the mesh. Default: 1.5, Min 0.0"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarWaterMeshGPUQuadTreeConservativeRasterization(
	TEXT("r.Water.WaterMesh.GPUQuadTree.ConservativeRasterization"), 0,
	TEXT("Enables software conservative rasterization for rasterizing water body meshes into the water quadtree. Disables jittered draws. Default: 0"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarWaterMeshGPUQuadTreeNumQuads(
	TEXT("r.Water.WaterMesh.GPUQuadTree.NumQuadsPerTileSide"), 8,
	TEXT("Number of quads per side of each tile mesh used to draw the water surface. A lower number results in more draw calls, a higher number in wasted VS invocations. Default: 8"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarWaterMeshGPUQuadInstanceDataAllocMult(
	TEXT("r.Water.WaterMesh.GPUQuadTree.InstanceDataAllocMult"), 1.0f,
	TEXT("Multiplier to apply to the number of tiles in the quadtree at LOD0. The derived number is how many slots for water quad mesh instance data are allocated. Default: 1.0, Min: 0.0, Max: 8.0"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarWaterMeshOcclusionCullingMaxQueries(
	TEXT("r.Water.WaterMesh.OcclusionCulling.MaxQueries"), 256,
	TEXT("Maximum number of occlusion queries for the CPU water quadtree nodes. Using fewer queries than nodes will result in coarser culling."),
	ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarWaterMeshOcclusionCullingIncludeFarMesh(
	TEXT("r.Water.WaterMesh.OcclusionCulling.IncludeFarMesh"), 1,
	TEXT("When occlusion culling is enabled, always do occlusion queries for the water far mesh, independent of r.Water.WaterMesh.OcclusionCulling.MaxQueries."),
	ECVF_Scalability);

/** Debug CVars */
static TAutoConsoleVariable<int32> CVarWaterMeshShowWireframe(
	TEXT("r.Water.WaterMesh.ShowWireframe"),
	0,
	TEXT("Forces wireframe rendering on for water"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarWaterMeshShowWireframeAtBaseHeight(
	TEXT("r.Water.WaterMesh.ShowWireframeAtBaseHeight"),
	0,
	TEXT("When rendering in wireframe, show the mesh with no displacement"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarWaterMeshEnableRendering(
	TEXT("r.Water.WaterMesh.EnableRendering"),
	1,
	TEXT("Turn off all water rendering from within the scene proxy"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarWaterMeshShowLODLevels(
	TEXT("r.Water.WaterMesh.ShowLODLevels"),
	0,
	TEXT("Shows the LOD levels as concentric squares around the observer position at height 0"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarWaterMeshShowTileBounds(
	TEXT("r.Water.WaterMesh.ShowTileBounds"),
	0,
	TEXT("Shows the tile bounds with optional color modes: 0 is disabled, 1 is by water body type, 2 is by LOD, 3 is by density index"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarWaterMeshShowTileBoundsForeground(
	TEXT("r.Water.WaterMesh.ShowTileBounds.DrawForeground"),
	0,
	TEXT("Shows all tile bounds, even occluded ones by drawing into the foreground"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarWaterMeshPreAllocStagingInstanceMemory(
	TEXT("r.Water.WaterMesh.PreAllocStagingInstanceMemory"),
	0,
	TEXT("Pre-allocates staging instance data memory according to historical max. This reduces the overhead when the array needs to grow but may use more memory"),
	ECVF_RenderThreadSafe);

#if RHI_RAYTRACING
static TAutoConsoleVariable<int32> CVarRayTracingGeometryWater(
	TEXT("r.RayTracing.Geometry.Water"),
	0,
	TEXT("Include water in ray tracing effects (default = 0 (water disabled in ray tracing))"));
#endif

static TAutoConsoleVariable<int32> CVarWaterMeshOcclusionCulling(
	TEXT("r.Water.WaterMesh.OcclusionCulling"),
	0,
	TEXT("Enables occlusion culling for the CPU water quadtree."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarWaterMeshOcclusionCullExpandBoundsAmountXY(
	TEXT("r.Water.WaterMesh.OcclusionCullExpandBoundsAmountXY"), 4800.0f,
	TEXT("Expand the water tile bounds in XY for the purpose of occlusion culling"),
	ECVF_Scalability);

// ----------------------------------------------------------------------------------

template <bool bWithWaterSelectionSupport>
class TWaterVertexFactoryUserDataWrapper : public FOneFrameResource
{
public:
	TWaterMeshUserData<bWithWaterSelectionSupport> UserData;
};

FWaterMeshSceneProxy::FWaterMeshSceneProxy(UWaterMeshComponent* Component)
	: FPrimitiveSceneProxy(Component)
	, MaterialRelevance(Component->GetWaterMaterialRelevance(GetScene().GetShaderPlatform()))
	, WaterQuadTreeBuilder(Component->GetWaterQuadTreeBuilder())
	, bIsLocalOnlyTessellationEnabled(Component->IsLocalOnlyTessellationEnabled())
{
	// Need to enable single pass GetDynamicMeshElements, as the function uses all Views together to compute visible water tiles
	bSinglePassGDME = true;

	// Leaf size * 0.5 equals the tightest possible LOD Scale that doesn't break the morphing. Can be scaled larger
	const float LeafSize = WaterQuadTreeBuilder.GetLeafSize();
	WaterQuadTreeConstants.LODScale = LeafSize * FMath::Max(Component->GetLODScale(), 0.5f);

	int32 NumQuads = (int32)FMath::Pow(2.0f, (float)Component->GetTessellationFactor());
	WaterQuadTreeConstants.NumQuadsLOD0 = NumQuads;
	WaterQuadTreeConstants.NumQuadsPerIndirectDrawTile = FMath::Min((int32)FMath::RoundUpToPowerOfTwo(FMath::Clamp(CVarWaterMeshGPUQuadTreeNumQuads.GetValueOnGameThread(), 2, 128)), WaterQuadTreeConstants.NumQuadsLOD0);

	// Assign the force collapse level if there is one, otherwise leave it at the default
	if (Component->ForceCollapseDensityLevel > -1)
	{
		WaterQuadTreeConstants.ForceCollapseDensityLevel = Component->ForceCollapseDensityLevel;
	}

	const int32 TreeDepth = WaterQuadTreeBuilder.GetTreeDepth();
	WaterQuadTreeConstants.DensityCount = FMath::Min(TreeDepth, (int32)FMath::FloorLog2(WaterQuadTreeConstants.NumQuadsLOD0));


	// bIsGPUQuadTree is constant over the lifetime of this scene proxy, so we know up front if we need the indirect draw vertex factory
	const bool bIsGPUQuadTree = WaterQuadTreeBuilder.IsGPUQuadTree();
	if (bIsGPUQuadTree)
	{
		// Only create one indirect draw vertex factory, depending on whether ISR is enabled
		const UE::StereoRenderUtils::FStereoShaderAspects Aspects(GetScene().GetShaderPlatform());
		if (Aspects.IsInstancedStereoEnabled())
		{
			WaterVertexFactoryIndirectDrawISR = new FWaterVertexFactoryIndirectDrawISRType(GetScene().GetFeatureLevel(), WaterQuadTreeConstants.NumQuadsPerIndirectDrawTile, WaterQuadTreeConstants.NumQuadsLOD0, WaterQuadTreeConstants.DensityCount, LeafSize, WaterQuadTreeConstants.LODScale);
			BeginInitResource(WaterVertexFactoryIndirectDrawISR);
		}
		else
		{
			WaterVertexFactoryIndirectDraw = new FWaterVertexFactoryIndirectDrawType(GetScene().GetFeatureLevel(), WaterQuadTreeConstants.NumQuadsPerIndirectDrawTile, WaterQuadTreeConstants.NumQuadsLOD0, WaterQuadTreeConstants.DensityCount, LeafSize, WaterQuadTreeConstants.LODScale);
			BeginInitResource(WaterVertexFactoryIndirectDraw);
		}
	}

	// We always need the basic CPU-driven rendering vertex factory because the far mesh uses it
	WaterVertexFactories.Reserve(TreeDepth);
	for (int32 i = 0; i < TreeDepth; i++)
	{
		WaterVertexFactories.Add(new FWaterVertexFactoryType(GetScene().GetFeatureLevel(), NumQuads, WaterQuadTreeConstants.NumQuadsLOD0, WaterQuadTreeConstants.DensityCount, LeafSize, WaterQuadTreeConstants.LODScale));
		BeginInitResource(WaterVertexFactories.Last());

		NumQuads /= 2;

		// If LODs become too small, early out
		if (NumQuads <= 1)
		{
			break;
		}
	}

	WaterVertexFactories.Shrink();
	check(WaterQuadTreeConstants.DensityCount == WaterVertexFactories.Num());

#if RHI_RAYTRACING
	RayTracingWaterData.SetNum(WaterQuadTreeConstants.DensityCount);
#endif

	// In case we don't need to support multiple local quadtrees, create a global quadtree right away and initialize occlusion culling bounds.
	// Local quadtrees are incompatible with CPU-based occlusion culling as the number of quadtrees can change during the lifetime of the scene proxy.
	if (!bIsLocalOnlyTessellationEnabled)
	{
		FViewWaterQuadTree& ViewWaterQuadTree = ViewQuadTrees.Add(INDEX_NONE);

		ViewWaterQuadTree.Update(WaterQuadTreeBuilder, Component->GetGlobalWaterMeshCenter());

		const FWaterQuadTree& WaterQuadTree = ViewWaterQuadTree.GetWaterQuadTree();

		// Always do CPU occlusion queries, even if this is a GPU quadtree. The GPU quadtree still potentially uses the far mesh which is CPU driven.
		const int32 MaxQueries = CVarWaterMeshOcclusionCullingMaxQueries.GetValueOnGameThread();
		const bool bIncludeFarMeshOcclusionQueries = CVarWaterMeshOcclusionCullingIncludeFarMesh.GetValueOnGameThread() != 0;
		OcclusionCullingBounds = WaterQuadTree.ComputeNodeBounds(MaxQueries, CVarWaterMeshOcclusionCullExpandBoundsAmountXY.GetValueOnGameThread(), bIncludeFarMeshOcclusionQueries, &OcclusionResultsFarMeshOffset);
		EmptyOcclusionCullingBounds.Add(WaterQuadTree.GetBoundsIncludingFarMesh());
		// If this is a GPU quadtree, the CPU root node will have an invalid bounding box, so derive conservative bounds now
		if (bIsGPUQuadTree && !OcclusionCullingBounds.IsEmpty())
		{
			OcclusionCullingBounds[0] = FBox(FVector(WaterQuadTree.GetTileRegion().Min, ViewWaterQuadTree.GetMinHeight()), FVector(WaterQuadTree.GetTileRegion().Max, ViewWaterQuadTree.GetMaxHeight()));
		}
	}
	else
	{
		EmptyOcclusionCullingBounds.Add(Component->Bounds);

		// Normally the quadtrees are created and updated by the WaterViewExtension. 
		// It can happen that a SceneProxy tries to render before the WaterView updates it, which caused some water flickering.
		// By forcing the creation here, using the quadtree update information stored by the water view extension, we make sure the water data is always valid.
		if (FWaterViewExtension* WaterViewExtension = UWaterSubsystem::GetWaterViewExtension(Component->GetWorld()))
		{
			WaterViewExtension->CreateSceneProxyQuadtrees(this);
		}
	}

	// When using a GPU quadtree, this scene proxy allocates pooled buffers in GetDynamicMeshElements. AllocatePooledBuffer must be called on the renderthread.
	// The callback on GWaterMeshGPUWork seems to be invoked after all GetDynamicMeshElements tasks finish, so there should be no race condition there.
	// There is usually only a single instance of this proxy (in certain cases there might be two), so opting out of parallel GDME shouldn't have much of an impact (?).
	bSupportsParallelGDME = !bIsGPUQuadTree;
}

FWaterMeshSceneProxy::~FWaterMeshSceneProxy()
{
	for (FWaterVertexFactoryType* WaterFactory : WaterVertexFactories)
	{
		WaterFactory->ReleaseResource();
		delete WaterFactory;
	}
	if (WaterVertexFactoryIndirectDraw)
	{
		WaterVertexFactoryIndirectDraw->ReleaseResource();
		delete WaterVertexFactoryIndirectDraw;
	}
	if (WaterVertexFactoryIndirectDrawISR)
	{
		WaterVertexFactoryIndirectDrawISR->ReleaseResource();
		delete WaterVertexFactoryIndirectDrawISR;
	}

#if RHI_RAYTRACING
	for (auto& WaterDataArray : RayTracingWaterData)
	{
		for (auto& WaterRayTracingItem : WaterDataArray)
		{
			WaterRayTracingItem.Geometry.ReleaseResource();
			WaterRayTracingItem.DynamicVertexBuffer.Release();
		}
	}	
#endif
}

void FWaterMeshSceneProxy::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	SceneProxyCreatedFrameNumberRenderThread = GFrameNumberRenderThread;

	// Register a callback with the FWaterViewExtension to be executed every frame before rendering the scene.
	if (WaterQuadTreeBuilder.IsGPUQuadTree())
	{
		FWaterMeshGPUWork::FCallback Callback;
		Callback.Proxy = this;
		Callback.Function = [this](FRDGBuilder& GraphBuilder, bool bDepthBufferIsPopulated)
		{
			for (auto& Pair : ViewQuadTrees)
			{
				Pair.Value.TraverseGPUQuadTree(GraphBuilder, bDepthBufferIsPopulated);
			}
		};
		GWaterMeshGPUWork.Callbacks.Add(MoveTemp(Callback));
	}
}

void FWaterMeshSceneProxy::DestroyRenderThreadResources()
{
	if (WaterQuadTreeBuilder.IsGPUQuadTree())
	{
		GWaterMeshGPUWork.Callbacks.RemoveAllSwap([this](const FWaterMeshGPUWork::FCallback& Callback) { return Callback.Proxy == this; });
	}
}

void FWaterMeshSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Water);
	TRACE_CPUPROFILER_EVENT_SCOPE(FWaterMeshSceneProxy::GetDynamicMeshElements);
	
	if (!HasWaterData() || !FWaterUtils::IsWaterMeshRenderingEnabled(/*bIsRenderThread = */true))
	{
		return;
	}
	
	FRHICommandListBase& RHICmdList = Collector.GetRHICommandList();

	// Handle wireframe views
	FColoredMaterialRenderProxy* WireframeMaterialInstance = nullptr;
	const uint32 WireframeVisibilityMap = GetWireframeVisibilityMapAndMaterial(Views, VisibilityMap, Collector, WireframeMaterialInstance);

	// Get a mapping from view index to quadtree key
	TArray<int32, TInlineAllocator<8>> ViewToQuadTreeKey = GetViewToQuadTreeMapping(Views, VisibilityMap);
	
	// Iterate over all quadtrees and generate draw calls for all views that map to that quadtree.
	for (const auto& Pair : ViewQuadTrees)
	{
		const FViewWaterQuadTree& ViewWaterQuadTree = Pair.Value;
		const FWaterQuadTree& WaterQuadTree = ViewWaterQuadTree.GetWaterQuadTree();

		// Compute a subset mask of VisibilityMap of all views that use this quadtree
		uint32 LocalViewMask = 0;
		bool bEncounteredISRView = false;
		int32 InstanceFactor = 1;
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex) && ViewToQuadTreeKey[ViewIndex] == Pair.Key)
			{
				LocalViewMask |= 1 << ViewIndex;
				const FSceneView* View = Views[ViewIndex];
				if (!bEncounteredISRView && View->IsInstancedStereo() && IStereoRendering::IsAPrimaryView(*View))
				{
					bEncounteredISRView = true;
					InstanceFactor = View->GetStereoPassInstanceFactor();
				}
			}
		}

		// The water render groups we have to render for this batch : 
		const TArray<EWaterMeshRenderGroupType, TInlineAllocator<FWaterVertexFactoryType::NumRenderGroups>> BatchRenderGroups = GetBatchRenderGroups(Views, LocalViewMask);

		const int32 NumWaterMaterials = WaterQuadTree.GetWaterMaterials().Num();
		const int32 NumBuckets = NumWaterMaterials * WaterQuadTreeConstants.DensityCount;
		const int32 NumBucketsIndirect = NumWaterMaterials; // Indirect draws use a single density mesh tile

		if (WaterQuadTree.IsGPUQuadTree() && NumBucketsIndirect > 0)
		{
			const FViewWaterQuadTree::FUserDataAndIndirectArgs WaterMeshUserDataAndIndirectArgs = ViewWaterQuadTree.PrepareGPUQuadTreeForRendering(Views, LocalViewMask, Collector, WaterQuadTreeConstants, BatchRenderGroups, RHICmdList);

			// Go through all buckets and issue one batched draw call per LOD level per material per view
			int32 CompactViewIndex = 0;
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				// when rendering ISR, don't process the instanced view
				if ((LocalViewMask & (1 << ViewIndex)) && (!bEncounteredISRView || Views[ViewIndex]->IsPrimarySceneView()))
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(BucketsPerView);

					const bool bInstancedStereo = Views[ViewIndex]->bIsInstancedStereoEnabled;
					const bool bViewIsWireframe = WireframeVisibilityMap & (1 << ViewIndex);

					for (int32 MaterialIndex = 0; MaterialIndex < NumWaterMaterials; ++MaterialIndex)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(MaterialBucket);
						bool bMaterialDrawn = false;

						// We only render all tiles with one or multiple instances of a single density quad mesh, so no need to create one bucket per density.
						{
							const int32 BucketIndex = MaterialIndex;

							TRACE_CPUPROFILER_EVENT_SCOPE(DensityBucket);

							const FMaterialRenderProxy* MaterialRenderProxy = bViewIsWireframe && WireframeMaterialInstance ? WireframeMaterialInstance : WaterQuadTree.GetWaterMaterials()[MaterialIndex];
							check(MaterialRenderProxy != nullptr);

							bool bUseForDepthPass = false;

							// If there's a valid material, use that to figure out the depth pass status
							if (const FMaterial* BucketMaterial = MaterialRenderProxy->GetMaterialNoFallback(GetScene().GetFeatureLevel()))
							{
								// Preemptively turn off depth rendering for this mesh batch if the material doesn't need it
								bUseForDepthPass = !BucketMaterial->GetShadingModels().HasShadingModel(MSM_SingleLayerWater) && !IsTranslucentOnlyBlendMode(*BucketMaterial);
							}

							bMaterialDrawn = true;
							for (EWaterMeshRenderGroupType RenderGroup : BatchRenderGroups)
							{
								// Set up mesh batch
								FMeshBatch& Mesh = Collector.AllocateMesh();
								Mesh.bWireframe = bViewIsWireframe;
								Mesh.VertexFactory = bInstancedStereo ? (FVertexFactory*)WaterVertexFactoryIndirectDrawISR : (FVertexFactory*)WaterVertexFactoryIndirectDraw;
								Mesh.MaterialRenderProxy = MaterialRenderProxy;
								Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
								Mesh.Type = PT_TriangleList;
								Mesh.DepthPriorityGroup = SDPG_World;
								Mesh.bCanApplyViewModeOverrides = false;
								Mesh.bUseForMaterial = true;
								Mesh.CastShadow = false;
								// Preemptively turn off depth rendering for this mesh batch if the material doesn't need it
								Mesh.bUseForDepthPass = bUseForDepthPass;
								Mesh.bUseAsOccluder = false;

#if WITH_WATER_SELECTION_SUPPORT
								Mesh.bUseSelectionOutline = (RenderGroup == EWaterMeshRenderGroupType::RG_RenderSelectedWaterTilesOnly);
								Mesh.bUseWireframeSelectionColoring = (RenderGroup == EWaterMeshRenderGroupType::RG_RenderSelectedWaterTilesOnly);
#endif // WITH_WATER_SELECTION_SUPPORT

								Mesh.Elements.SetNumZeroed(1);
								{
									TRACE_CPUPROFILER_EVENT_SCOPE_STR("Setup batch element");

									// Set up one mesh batch element
									FMeshBatchElement& BatchElement = Mesh.Elements[0];

									// Set up for indirect draw
									BatchElement.FirstIndex = 0;
									BatchElement.NumPrimitives = 0; // Must be 0 to enable usage of IndirectArgsBuffer
									BatchElement.IndirectArgsBuffer = WaterMeshUserDataAndIndirectArgs.IndirectArgs->GetRHI();
									BatchElement.IndirectArgsOffset = (CompactViewIndex * NumBucketsIndirect + BucketIndex) * sizeof(FRHIDrawIndexedIndirectParameters);
									BatchElement.UserData = (void*)WaterMeshUserDataAndIndirectArgs.UserData[(int32)RenderGroup];
									BatchElement.UserIndex = CompactViewIndex * NumBucketsIndirect + BucketIndex;

									BatchElement.IndexBuffer = bInstancedStereo ? WaterVertexFactoryIndirectDrawISR->IndexBuffer : WaterVertexFactoryIndirectDraw->IndexBuffer;
									BatchElement.PrimitiveIdMode = PrimID_ForceZero;

									// We need the uniform buffer of this primitive because it stores the proper value for the bOutputVelocity flag.
									// The identity primitive uniform buffer simply stores false for this flag which leads to missing motion vectors.
									BatchElement.PrimitiveUniformBuffer = GetUniformBuffer();
								}

								{
									INC_DWORD_STAT(STAT_WaterDrawCalls);
									TRACE_CPUPROFILER_EVENT_SCOPE(Collector.AddMesh);

									Collector.AddMesh(ViewIndex, Mesh);
								}
							}
						}

						INC_DWORD_STAT_BY(STAT_WaterDrawnMats, (int32)bMaterialDrawn);
					}

					++CompactViewIndex;
				}
			}
		}
	
		// Even if we have a GPU quadtree, we still go through the setup for the non-GPU path because the water quadtree might have an ocean far mesh which we render traditionally.
		// If this is a GPU quadtree, traversing it on the CPU will only return the far mesh tiles (if present).

		TArray<FWaterQuadTree::FTraversalOutput, TInlineAllocator<4>> WaterInstanceDataPerView;

		// Gather visible tiles, their lod and materials for all renderable views (skip right view when stereo pair is rendered instanced)
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FSceneView* View = Views[ViewIndex];

			// skip gathering visible tiles from instanced right eye views
			if ((LocalViewMask & (1 << ViewIndex)) && (!bEncounteredISRView || View->IsPrimarySceneView()))
			{
				const FVector ObserverPosition = View->ViewMatrices.GetViewOrigin();

				const FViewWaterQuadTree::FWaterLODParams WaterLODParams = ViewWaterQuadTree.GetWaterLODParams(ObserverPosition, WaterQuadTreeConstants.LODScale);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (CVarWaterMeshShowLODLevels.GetValueOnRenderThread())
				{
					for (int32 i = WaterLODParams.LowestLOD; i < WaterQuadTree.GetTreeDepth(); i++)
					{
						float LODDist = FWaterQuadTree::GetLODDistance(i, WaterQuadTreeConstants.LODScale);
						FVector Orig = FVector(FVector2D(ObserverPosition), WaterLODParams.WaterHeightForLOD);

						DrawCircle(Collector.GetPDI(ViewIndex), Orig, FVector::ForwardVector, FVector::RightVector, GColorList.GetFColorByIndex(i + 1), LODDist, 64, 0);
					}
				}
#endif
				TRACE_CPUPROFILER_EVENT_SCOPE(QuadTreeTraversalPerView);

				FWaterQuadTree::FTraversalOutput& WaterInstanceData = WaterInstanceDataPerView.Emplace_GetRef();
				WaterInstanceData.BucketInstanceCounts.Empty(NumBuckets);
				WaterInstanceData.BucketInstanceCounts.AddZeroed(NumBuckets);
				if (!!CVarWaterMeshPreAllocStagingInstanceMemory.GetValueOnRenderThread())
				{
					WaterInstanceData.StagingInstanceData.Empty(HistoricalMaxViewInstanceCount);
				}

				FWaterQuadTree::FTraversalDesc TraversalDesc;
				TraversalDesc.LowestLOD = WaterLODParams.LowestLOD;
				TraversalDesc.HeightMorph = WaterLODParams.HeightLODFactor;
				TraversalDesc.LODCount = WaterQuadTree.GetTreeDepth();
				TraversalDesc.DensityCount = WaterQuadTreeConstants.DensityCount;
				TraversalDesc.ForceCollapseDensityLevel = WaterQuadTreeConstants.ForceCollapseDensityLevel;
				TraversalDesc.Frustum = View->GetCullingFrustum();
				TraversalDesc.ObserverPosition = ObserverPosition;
				TraversalDesc.PreViewTranslation = View->ViewMatrices.GetPreViewTranslation();
				TraversalDesc.LODScale = WaterQuadTreeConstants.LODScale;
				TraversalDesc.bLODMorphingEnabled = !!CVarWaterMeshLODMorphEnabled.GetValueOnRenderThread();
				TraversalDesc.WaterInfoBounds = WaterQuadTree.GetTileRegion();
				TraversalDesc.OcclusionCullingResults = nullptr;
				TraversalDesc.OcclusionCullingFarMeshOffset = OcclusionResultsFarMeshOffset;
				if (CVarWaterMeshOcclusionCulling.GetValueOnRenderThread() != 0)
				{
					const FOcclusionCullingResults* CullingResults = OcclusionResults.Find(View->GetViewKey());
					if (CullingResults && !CullingResults->Results.IsEmpty())
					{
						TraversalDesc.OcclusionCullingResults = &CullingResults->Results;
					}
				}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				//Debug
				TraversalDesc.DebugPDI = Collector.GetPDI(ViewIndex);
				TraversalDesc.DebugShowTile = CVarWaterMeshShowTileBounds.GetValueOnRenderThread();
				TraversalDesc.bDebugDrawIntoForeground = CVarWaterMeshShowTileBoundsForeground.GetValueOnRenderThread() != 0;
#endif
				WaterQuadTree.BuildWaterTileInstanceData(TraversalDesc, WaterInstanceData);

				HistoricalMaxViewInstanceCount = FMath::Max(HistoricalMaxViewInstanceCount, WaterInstanceData.InstanceCount);
			}
		}

		// Get number of total instances for all views
		int32 TotalInstanceCount = 0;
		for (const FWaterQuadTree::FTraversalOutput& WaterInstanceData : WaterInstanceDataPerView)
		{
			TotalInstanceCount += WaterInstanceData.InstanceCount;
		}

		if (TotalInstanceCount == 0)
		{
			// no instance visible, early exit
			continue;
		}

		ViewWaterQuadTree.GetWaterInstanceDataBuffers()->Lock(RHICmdList, TotalInstanceCount* InstanceFactor);

		int32 InstanceDataOffset = 0;

		// Go through all buckets and issue one batched draw call per LOD level per material per view
		int32 TraversalIndex = 0;
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			// when rendering ISR, don't process the instanced view
			if ((LocalViewMask & (1 << ViewIndex)) && (!bEncounteredISRView || Views[ViewIndex]->IsPrimarySceneView()))
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(BucketsPerView);

				const bool bViewIsWireframe = WireframeVisibilityMap & (1 << ViewIndex);

				FWaterQuadTree::FTraversalOutput& WaterInstanceData = WaterInstanceDataPerView[TraversalIndex];
				TraversalIndex++;

				for (int32 MaterialIndex = 0; MaterialIndex < NumWaterMaterials; ++MaterialIndex)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(MaterialBucket);
					bool bMaterialDrawn = false;

					for (int32 DensityIndex = 0; DensityIndex < WaterQuadTreeConstants.DensityCount; ++DensityIndex)
					{
						const int32 BucketIndex = MaterialIndex * WaterQuadTreeConstants.DensityCount + DensityIndex;
						const int32 InstanceCount = WaterInstanceData.BucketInstanceCounts[BucketIndex];

						if (!InstanceCount)
						{
							continue;
						}

						TRACE_CPUPROFILER_EVENT_SCOPE(DensityBucket);

						const FMaterialRenderProxy* MaterialRenderProxy = bViewIsWireframe && WireframeMaterialInstance ? WireframeMaterialInstance : WaterQuadTree.GetWaterMaterials()[MaterialIndex];
						check(MaterialRenderProxy != nullptr);

						bool bUseForDepthPass = false;

						// If there's a valid material, use that to figure out the depth pass status
						if (const FMaterial* BucketMaterial = MaterialRenderProxy->GetMaterialNoFallback(GetScene().GetFeatureLevel()))
						{
							// Preemptively turn off depth rendering for this mesh batch if the material doesn't need it
							bUseForDepthPass = !BucketMaterial->GetShadingModels().HasShadingModel(MSM_SingleLayerWater) && !IsTranslucentOnlyBlendMode(*BucketMaterial);
						}

						bMaterialDrawn = true;
						for (EWaterMeshRenderGroupType RenderGroup : BatchRenderGroups)
						{
							// Set up mesh batch
							FMeshBatch& Mesh = Collector.AllocateMesh();
							Mesh.bWireframe = bViewIsWireframe;
							Mesh.VertexFactory = WaterVertexFactories[DensityIndex];
							Mesh.MaterialRenderProxy = MaterialRenderProxy;
							Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
							Mesh.Type = PT_TriangleList;
							Mesh.DepthPriorityGroup = SDPG_World;
							Mesh.bCanApplyViewModeOverrides = false;
							Mesh.bUseForMaterial = true;
							Mesh.CastShadow = false;
							// Preemptively turn off depth rendering for this mesh batch if the material doesn't need it
							Mesh.bUseForDepthPass = bUseForDepthPass;
							Mesh.bUseAsOccluder = false;

#if WITH_WATER_SELECTION_SUPPORT
							Mesh.bUseSelectionOutline = (RenderGroup == EWaterMeshRenderGroupType::RG_RenderSelectedWaterTilesOnly);
							Mesh.bUseWireframeSelectionColoring = (RenderGroup == EWaterMeshRenderGroupType::RG_RenderSelectedWaterTilesOnly);
#endif // WITH_WATER_SELECTION_SUPPORT

							Mesh.Elements.SetNumZeroed(1);

							{
								TRACE_CPUPROFILER_EVENT_SCOPE_STR("Setup batch element");

								// Set up one mesh batch element
								FMeshBatchElement& BatchElement = Mesh.Elements[0];

								// Set up for instancing
								//BatchElement.bIsInstancedMesh = true;
								BatchElement.NumInstances = InstanceCount;
								BatchElement.UserData = (void*)ViewWaterQuadTree.GetWaterMeshUserDataBuffers()->GetUserData(RenderGroup);
								BatchElement.UserIndex = InstanceDataOffset * InstanceFactor;

								BatchElement.FirstIndex = 0;
								BatchElement.NumPrimitives = WaterVertexFactories[DensityIndex]->IndexBuffer->GetIndexCount() / 3;
								BatchElement.MinVertexIndex = 0;
								BatchElement.MaxVertexIndex = WaterVertexFactories[DensityIndex]->VertexBuffer->GetVertexCount() - 1;

								BatchElement.IndexBuffer = WaterVertexFactories[DensityIndex]->IndexBuffer;
								BatchElement.PrimitiveIdMode = PrimID_ForceZero;

								// We need the uniform buffer of this primitive because it stores the proper value for the bOutputVelocity flag.
								// The identity primitive uniform buffer simply stores false for this flag which leads to missing motion vectors.
								BatchElement.PrimitiveUniformBuffer = GetUniformBuffer();
							}

							{
								INC_DWORD_STAT_BY(STAT_WaterVerticesDrawn, WaterVertexFactories[DensityIndex]->VertexBuffer->GetVertexCount() * InstanceCount);
								INC_DWORD_STAT(STAT_WaterDrawCalls);
								INC_DWORD_STAT_BY(STAT_WaterTilesDrawn, InstanceCount);

								TRACE_CPUPROFILER_EVENT_SCOPE(Collector.AddMesh);

								Collector.AddMesh(ViewIndex, Mesh);
							}
						}

						// Note : we're repurposing the BucketInstanceCounts array here for storing the actual offset in the buffer. This means that effectively from this point on, BucketInstanceCounts doesn't actually 
						// contain the number of instances anymore : 
						WaterInstanceData.BucketInstanceCounts[BucketIndex] = InstanceDataOffset;
						InstanceDataOffset += InstanceCount;
					}

					INC_DWORD_STAT_BY(STAT_WaterDrawnMats, (int32)bMaterialDrawn);
				}

				const int32 NumStagingInstances = WaterInstanceData.StagingInstanceData.Num();
				for (int32 Idx = 0; Idx < NumStagingInstances; ++Idx)
				{
					const FWaterQuadTree::FStagingInstanceData& Data = WaterInstanceData.StagingInstanceData[Idx];
					const int32 WriteIndex = WaterInstanceData.BucketInstanceCounts[Data.BucketIndex]++;

					for (int32 StreamIdx = 0; StreamIdx < FWaterInstanceDataBuffersType::NumBuffers; ++StreamIdx)
					{
						TArrayView<FVector4f> BufferMemory = ViewWaterQuadTree.GetWaterInstanceDataBuffers()->GetBufferMemory(StreamIdx);
						for (int32 IdxMultipliedInstance = 0; IdxMultipliedInstance < InstanceFactor; ++IdxMultipliedInstance)
						{
							BufferMemory[WriteIndex * InstanceFactor + IdxMultipliedInstance] = Data.Data[StreamIdx];
						}
					}
				}
			}
		}

		ViewWaterQuadTree.GetWaterInstanceDataBuffers()->Unlock(RHICmdList);
	}
}

const TArray<FBoxSphereBounds>* FWaterMeshSceneProxy::GetOcclusionQueries(const FSceneView* View) const
{
	// Local tessellation/multiple dynamic quadtrees are incompatible with CPU occlusion culling
	if (!bIsLocalOnlyTessellationEnabled && CVarWaterMeshOcclusionCulling.GetValueOnRenderThread() != 0)
	{
		return &OcclusionCullingBounds;
	}
	return &EmptyOcclusionCullingBounds;
}

void FWaterMeshSceneProxy::AcceptOcclusionResults(const FSceneView* View, TArray<bool>* Results, int32 ResultsStart, int32 NumResults)
{
	// Don't accept subprimitive occlusion results from a previously-created sceneproxy - the tree may have been different
	if (OcclusionCullingBounds.Num() == NumResults && SceneProxyCreatedFrameNumberRenderThread < GFrameNumberRenderThread)
	{
		// This lock is necessary to guard against access from multiple views.
		OcclusionResultsMutex.Lock();

		uint32 ViewId = View->GetViewKey();
		FOcclusionCullingResults* OldResults = OcclusionResults.Find(ViewId);
		if (OldResults)
		{
			OldResults->FrameNumber = GFrameNumberRenderThread;
			OldResults->Results.Reset();
			OldResults->Results.Append(Results->GetData() + ResultsStart, NumResults);
		}
		else
		{
			// now is a good time to clean up any stale entries
			for (auto Iter = OcclusionResults.CreateIterator(); Iter; ++Iter)
			{
				if (Iter.Value().FrameNumber != GFrameNumberRenderThread)
				{
					Iter.RemoveCurrent();
				}
			}

			FOcclusionCullingResults NewResults;
			NewResults.FrameNumber = GFrameNumberRenderThread;
			NewResults.Results.Append(Results->GetData() + ResultsStart, NumResults);

			OcclusionResults.Add(ViewId, MoveTemp(NewResults));
		}

		OcclusionResultsMutex.Unlock();

		int32 NumCulled = 0;
		for (int32 ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
		{
			NumCulled += (*Results)[ResultsStart + ResultIndex] ? 1 : 0;
		}
		INC_DWORD_STAT_BY(STAT_WaterTilesOcclusionCulled, NumCulled);
		INC_DWORD_STAT_BY(STAT_WaterOcclusionQueries, NumResults);
	}
}

bool FWaterMeshSceneProxy::HasWaterData() const
{
	for (const auto& Pair : ViewQuadTrees)
	{
		if (Pair.Value.GetWaterQuadTree().GetNodeCount() != 0 && WaterQuadTreeConstants.DensityCount != 0)
		{
			return true;
		}
	}
	return false;
}

TArray<EWaterMeshRenderGroupType, TInlineAllocator<FWaterMeshSceneProxy::FWaterVertexFactoryType::NumRenderGroups>> FWaterMeshSceneProxy::GetBatchRenderGroups(const TArray<const FSceneView*>& Views, uint32 VisibilityMap) const
{
	TArray<EWaterMeshRenderGroupType, TInlineAllocator<FWaterVertexFactoryType::NumRenderGroups>> BatchRenderGroups;
	{
		// By default, render all water tiles : 
		BatchRenderGroups.Add(EWaterMeshRenderGroupType::RG_RenderWaterTiles);

#if WITH_WATER_SELECTION_SUPPORT
		bool bHasSelectedInstances = IsSelected();

		// We're using bSinglePassGDME, so we need to check the ViewFamily of all views and can't use the ViewFamily passed into GDME.
		bool bSelectionRenderEnabled = false;
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				bSelectionRenderEnabled |= GIsEditor && Views[ViewIndex]->Family->EngineShowFlags.Selection;
			}
		}

		if (bSelectionRenderEnabled && bHasSelectedInstances)
		{
			// Don't render all in one group: instead, render 2 groups : first, the selected only then, the non-selected only :
			BatchRenderGroups[0] = EWaterMeshRenderGroupType::RG_RenderSelectedWaterTilesOnly;
			BatchRenderGroups.Add(EWaterMeshRenderGroupType::RG_RenderUnselectedWaterTilesOnly);
		}
#endif // WITH_WATER_SELECTION_SUPPORT
	}

	return BatchRenderGroups;
}

uint32 FWaterMeshSceneProxy::GetWireframeVisibilityMapAndMaterial(const TArray<const FSceneView*>& Views, uint32 VisibilityMap, FMeshElementCollector& Collector, FColoredMaterialRenderProxy*& OutMaterialInstance) const
{
	// Figure out which views have wireframe enabled
	uint32 WireframeVisibilityMap = 0;
	if (AllowDebugViewmodes())
	{
		if (CVarWaterMeshShowWireframe.GetValueOnRenderThread() == 1)
		{
			// Force all views to wireframe
			WireframeVisibilityMap = VisibilityMap;
		}
		else
		{
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if ((VisibilityMap & (1 << ViewIndex)) && Views[ViewIndex]->Family->EngineShowFlags.Wireframe)
				{
					WireframeVisibilityMap |= (1 << ViewIndex);
				}
			}
		}
	}

	// Set up wireframe material (if needed)
	OutMaterialInstance = nullptr;
	if (WireframeVisibilityMap && CVarWaterMeshShowWireframeAtBaseHeight.GetValueOnRenderThread() == 1)
	{
		OutMaterialInstance = new FColoredMaterialRenderProxy(
			GEngine->WireframeMaterial ? GEngine->WireframeMaterial->GetRenderProxy() : NULL,
			FColor::Cyan);

		Collector.RegisterOneFrameMaterialProxy(OutMaterialInstance);
	}

	return WireframeVisibilityMap;
}

int32 FWaterMeshSceneProxy::FindBestQuadTreeForView(const FSceneView* View) const
{
	// Do we have a quadtree matching this views PlayerIndex?
	if (ViewQuadTrees.Contains(View->PlayerIndex))
	{
		return View->PlayerIndex;
	}
	// Otherwise just use the closest quadtree.
	else
	{
		const FVector2D ViewPosition2D = FVector2D(View->ViewLocation);
		return FindBestQuadTreeForViewLocation(ViewPosition2D);
	}
}

int32 FWaterMeshSceneProxy::FindBestQuadTreeForViewLocation(const FVector2D& ViewPosition2D) const
{
	double ClosestDistSquared = DBL_MAX;
	int32 ClosestQuadTreeKey = INDEX_NONE;
	for (const auto& Pair : ViewQuadTrees)
	{
		const double DistSquared = FVector2D::DistSquared(Pair.Value.GetWaterQuadTree().GetTileRegion().GetCenter(), ViewPosition2D);
		if (DistSquared < ClosestDistSquared)
		{
			ClosestDistSquared = DistSquared;
			ClosestQuadTreeKey = Pair.Key;
		}
	}
	return ClosestQuadTreeKey;
}

TArray<int32, TInlineAllocator<8>> FWaterMeshSceneProxy::GetViewToQuadTreeMapping(const TArray<const FSceneView*>& Views, uint32 VisibilityMap) const
{
	// Find a quadtree for every view
	TArray<int32, TInlineAllocator<8>> ViewToQuadTreeKey;
	ViewToQuadTreeKey.SetNum(Views.Num());
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			ViewToQuadTreeKey[ViewIndex] = FindBestQuadTreeForView(Views[ViewIndex]);
		}
	}
	return ViewToQuadTreeKey;
}

#if RHI_RAYTRACING
void FWaterMeshSceneProxy::SetupRayTracingInstances(FRHICommandListBase& RHICmdList, int32 NumInstances, uint32 DensityIndex)
{
	TIndirectArray<FRayTracingWaterData>& WaterDataArray = RayTracingWaterData[DensityIndex];

	if (WaterDataArray.Num() > NumInstances)
	{
		for (int32 Item = NumInstances; Item < WaterDataArray.Num(); Item++)
		{
			auto& WaterItem = WaterDataArray[Item];
			WaterItem.Geometry.ReleaseResource();
			WaterItem.DynamicVertexBuffer.Release();
		}
		WaterDataArray.RemoveAt(NumInstances, WaterDataArray.Num() - NumInstances);
	}	

	if (WaterDataArray.Num() < NumInstances)
	{
		FRayTracingGeometryInitializer Initializer;
		static const FName DebugName("FWaterMeshSceneProxy");		
		Initializer.IndexBuffer = WaterVertexFactories[DensityIndex]->IndexBuffer->IndexBufferRHI;
		Initializer.GeometryType = RTGT_Triangles;
		Initializer.bFastBuild = true;
		Initializer.bAllowUpdate = true;
		Initializer.TotalPrimitiveCount = 0;

		WaterDataArray.Reserve(NumInstances);
		const int32 StartIndex = WaterDataArray.Num();

		for (int32 Item = StartIndex; Item < NumInstances; Item++)
		{
			FRayTracingWaterData* WaterData = new FRayTracingWaterData;

			Initializer.DebugName = FName(DebugName, Item);

			WaterData->Geometry.SetInitializer(Initializer);
			WaterData->Geometry.InitResource(RHICmdList);

			WaterDataArray.Add(WaterData);
		}
	}
}

void FWaterMeshSceneProxy::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector)
{
	if (!HasWaterData() || !FWaterUtils::IsWaterMeshRenderingEnabled(/*bIsRenderThread = */true) || !CVarRayTracingGeometryWater.GetValueOnRenderThread())
	{
		return;
	}
	
	TConstArrayView<const FSceneView*> Views = Collector.GetViews();
	const uint32 VisibilityMap = Collector.GetVisibilityMap();

	// RT geometry will be generated based on first active view and then reused for all other views
	// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
	const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
	checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

	const FSceneView& SceneView = *Views[FirstActiveViewIndex]; // TODO: should use view-specific BLAS?
	const FVector ObserverPosition = SceneView.ViewMatrices.GetViewOrigin();

	const int32 QuadTreeKey = FindBestQuadTreeForView(&SceneView);
	const FViewWaterQuadTree* ViewWaterQuadTree = ViewQuadTrees.Find(QuadTreeKey);
	check(ViewWaterQuadTree);
	const FWaterQuadTree& WaterQuadTree = ViewWaterQuadTree->GetWaterQuadTree();

	FViewWaterQuadTree::FWaterLODParams WaterLODParams = ViewWaterQuadTree->GetWaterLODParams(ObserverPosition, WaterQuadTreeConstants.LODScale);

	const int32 NumBuckets = WaterQuadTree.GetWaterMaterials().Num() * WaterQuadTreeConstants.DensityCount;

	FWaterQuadTree::FTraversalOutput WaterInstanceData;
	WaterInstanceData.BucketInstanceCounts.Empty(NumBuckets);
	WaterInstanceData.BucketInstanceCounts.AddZeroed(NumBuckets);

	FWaterQuadTree::FTraversalDesc TraversalDesc;
	TraversalDesc.LowestLOD = WaterLODParams.LowestLOD;
	TraversalDesc.HeightMorph = WaterLODParams.HeightLODFactor;
	TraversalDesc.LODCount = WaterQuadTree.GetTreeDepth();
	TraversalDesc.DensityCount = WaterQuadTreeConstants.DensityCount;
	TraversalDesc.ForceCollapseDensityLevel = WaterQuadTreeConstants.ForceCollapseDensityLevel;
	TraversalDesc.PreViewTranslation = SceneView.ViewMatrices.GetPreViewTranslation();
	TraversalDesc.ObserverPosition = ObserverPosition;
	TraversalDesc.Frustum = FConvexVolume(); // Default volume to disable frustum culling
	TraversalDesc.LODScale = WaterQuadTreeConstants.LODScale;
	TraversalDesc.bLODMorphingEnabled = !!CVarWaterMeshLODMorphEnabled.GetValueOnRenderThread();
	TraversalDesc.WaterInfoBounds = WaterQuadTree.GetTileRegion();

	WaterQuadTree.BuildWaterTileInstanceData(TraversalDesc, WaterInstanceData);

	if (WaterInstanceData.InstanceCount == 0)
	{
		// no instance visible, early exit
		return;
	}

	const int32 NumWaterMaterials = WaterQuadTree.GetWaterMaterials().Num();	

	for (int32 DensityIndex = 0; DensityIndex < WaterQuadTreeConstants.DensityCount; ++DensityIndex)
	{
		int32 DensityInstanceCount = 0;
		for (int32 MaterialIndex = 0; MaterialIndex < NumWaterMaterials; ++MaterialIndex)
		{
			const int32 BucketIndex = MaterialIndex * WaterQuadTreeConstants.DensityCount + DensityIndex;
			const int32 InstanceCount = WaterInstanceData.BucketInstanceCounts[BucketIndex];
			DensityInstanceCount += InstanceCount;
		}

		SetupRayTracingInstances(Collector.GetRHICommandList(), DensityInstanceCount, DensityIndex);
	}

	// Create per-bucket prefix sum and sort instance data so we can easily access per-instance data for each density
	TArray<int32> BucketOffsets;
	BucketOffsets.SetNumZeroed(NumBuckets);

	for (int32 BucketIndex = 1; BucketIndex < NumBuckets; ++BucketIndex)
	{
		BucketOffsets[BucketIndex] = BucketOffsets[BucketIndex - 1] + WaterInstanceData.BucketInstanceCounts[BucketIndex - 1];
	}
	
	WaterInstanceData.StagingInstanceData.StableSort([](const FWaterQuadTree::FStagingInstanceData& Lhs, const FWaterQuadTree::FStagingInstanceData& Rhs)
		{
			return Lhs.BucketIndex < Rhs.BucketIndex;
		});

	FMeshBatch BaseMesh;
	BaseMesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
	BaseMesh.Type = PT_TriangleList;
	BaseMesh.bUseForMaterial = true;
	BaseMesh.CastShadow = false;
	BaseMesh.CastRayTracedShadow = false;
	BaseMesh.SegmentIndex = 0;
	BaseMesh.Elements.AddZeroed();

	for (int32 DensityIndex = 0; DensityIndex < WaterQuadTreeConstants.DensityCount; ++DensityIndex)
	{
		int32 DensityInstanceIndex = 0;
		
		BaseMesh.VertexFactory = WaterVertexFactories[DensityIndex];

		FMeshBatchElement& BatchElement = BaseMesh.Elements[0];

		BatchElement.NumInstances = 1;

		BatchElement.FirstIndex = 0;
		BatchElement.NumPrimitives = WaterVertexFactories[DensityIndex]->IndexBuffer->GetIndexCount() / 3;
		BatchElement.MinVertexIndex = 0;
		BatchElement.MaxVertexIndex = WaterVertexFactories[DensityIndex]->VertexBuffer->GetVertexCount() - 1;

		// Don't use primitive buffer
		BatchElement.IndexBuffer = WaterVertexFactories[DensityIndex]->IndexBuffer;
		BatchElement.PrimitiveIdMode = PrimID_ForceZero;
		BatchElement.PrimitiveUniformBufferResource = &GIdentityPrimitiveUniformBuffer;

		for (int32 MaterialIndex = 0; MaterialIndex < NumWaterMaterials; ++MaterialIndex)
		{
			const int32 BucketIndex = MaterialIndex * WaterQuadTreeConstants.DensityCount + DensityIndex;
			const int32 InstanceCount = WaterInstanceData.BucketInstanceCounts[BucketIndex];

			if (!InstanceCount)
			{
				continue;
			}

			const FMaterialRenderProxy* MaterialRenderProxy = WaterQuadTree.GetWaterMaterials()[MaterialIndex];
			check(MaterialRenderProxy != nullptr);

			BaseMesh.MaterialRenderProxy = MaterialRenderProxy;

			for (int32 InstanceIndex = 0; InstanceIndex < InstanceCount; ++InstanceIndex)
			{
				using FWaterVertexFactoryUserDataWrapperType = TWaterVertexFactoryUserDataWrapper<WITH_WATER_SELECTION_SUPPORT>;
				FWaterVertexFactoryUserDataWrapperType& UserDataWrapper = Collector.AllocateOneFrameResource<FWaterVertexFactoryUserDataWrapperType>();

				const int32 InstanceDataIndex = BucketOffsets[BucketIndex] + InstanceIndex;
				const FWaterQuadTree::FStagingInstanceData& InstanceData = WaterInstanceData.StagingInstanceData[InstanceDataIndex];

				FWaterVertexFactoryRaytracingParameters UniformBufferParams;
				UniformBufferParams.VertexBuffer = WaterVertexFactories[DensityIndex]->VertexBuffer->GetSRV();
				UniformBufferParams.InstanceData0 = InstanceData.Data[0];
				UniformBufferParams.InstanceData1 = InstanceData.Data[1];

				UserDataWrapper.UserData.InstanceDataBuffers = ViewWaterQuadTree->GetWaterMeshUserDataBuffers()->GetUserData(EWaterMeshRenderGroupType::RG_RenderWaterTiles)->InstanceDataBuffers;
				UserDataWrapper.UserData.RenderGroupType = EWaterMeshRenderGroupType::RG_RenderWaterTiles;
				UserDataWrapper.UserData.WaterVertexFactoryRaytracingVFUniformBuffer = FWaterVertexFactoryRaytracingParametersRef::CreateUniformBufferImmediate(UniformBufferParams, UniformBuffer_SingleFrame);
							
				BatchElement.UserData = (void*)&UserDataWrapper.UserData;							

				FRayTracingWaterData& WaterInstanceRayTracingData = RayTracingWaterData[DensityIndex][DensityInstanceIndex++];

				TArray<FMeshBatch>& CollectorMeshBatches = Collector.AllocateMeshBatchArray();
				CollectorMeshBatches.Add(BaseMesh);

				{
					FRayTracingDynamicGeometryUpdateParams UpdateParams;
					UpdateParams.MeshBatchesView = CollectorMeshBatches;
					UpdateParams.bUsingIndirectDraw = false;
					UpdateParams.NumVertices = uint32(WaterVertexFactories[DensityIndex]->VertexBuffer->GetVertexCount());
					UpdateParams.VertexBufferSize = uint32(WaterVertexFactories[DensityIndex]->VertexBuffer->GetVertexCount() * sizeof(FVector3f));
					UpdateParams.NumTriangles = uint32(WaterVertexFactories[DensityIndex]->IndexBuffer->GetIndexCount() / 3);
					UpdateParams.Geometry = &WaterInstanceRayTracingData.Geometry;
					UpdateParams.Buffer = nullptr;
					UpdateParams.bApplyWorldPositionOffset = true;

					Collector.AddRayTracingGeometryUpdate(FirstActiveViewIndex, MoveTemp(UpdateParams));
				}

				FRayTracingInstance RayTracingInstance;
				RayTracingInstance.Geometry = &WaterInstanceRayTracingData.Geometry;
				RayTracingInstance.InstanceTransforms.Add(FMatrix::Identity);
				RayTracingInstance.MaterialsView = CollectorMeshBatches;

				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
				{
					if ((VisibilityMap & (1 << ViewIndex)) == 0)
					{
						continue;
					}

					Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
				}
			}
		}
	}
}
#endif // RHI_RAYTRACING

bool FWaterMeshSceneProxy::CreateViewWaterQuadTree(int32 Key, const FVector2D& CenterPosition)
{
	if (bIsLocalOnlyTessellationEnabled)
	{
		if (!ViewQuadTrees.Contains(Key))
		{
			FViewWaterQuadTree& ViewWaterQuadTree = ViewQuadTrees.Add(Key);
			ViewWaterQuadTree.Update(WaterQuadTreeBuilder, CenterPosition);
			return true;
		}
	}
	return false;
}

bool FWaterMeshSceneProxy::UpdateViewWaterQuadTree(int32 Key, const FVector2D& CenterPosition)
{
	if (bIsLocalOnlyTessellationEnabled)
	{
		if (FViewWaterQuadTree* ViewWaterQuadTree = ViewQuadTrees.Find(Key))
		{
			ViewWaterQuadTree->Update(WaterQuadTreeBuilder, CenterPosition);
			return true;
		}
	}
	return false;
}

void FWaterMeshSceneProxy::DestroyViewWaterQuadTree(int32 Key)
{
	if (bIsLocalOnlyTessellationEnabled)
	{
		ViewQuadTrees.Remove(Key);
	}
}

FPrimitiveViewRelevance FWaterMeshSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View);
	Result.bShadowRelevance = false;
	Result.bDynamicRelevance = true;
	Result.bStaticRelevance = false;
	Result.bRenderInMainPass = ShouldRenderInMainPass();
	Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
	Result.bRenderCustomDepth = ShouldRenderCustomDepth();
	Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
	MaterialRelevance.SetPrimitiveViewRelevance(Result);
	Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
	return Result;
}

bool FWaterMeshSceneProxy::HasSubprimitiveOcclusionQueries() const
{
	return true;
}

#if WITH_WATER_SELECTION_SUPPORT
HHitProxy* FWaterMeshSceneProxy::CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies)
{
	WaterQuadTreeBuilder.GatherHitProxies(OutHitProxies);

	// No default hit proxy.
	return nullptr;
}
#endif // WITH_WATER_SELECTION_SUPPORT

void FViewWaterQuadTree::Update(const FWaterQuadTreeBuilder& Builder, const FVector2D& CenterPosition)
{
	Builder.BuildWaterQuadTree(WaterQuadTree, CenterPosition);
	QuadTreeGPU = {};

	// Initialize Z bounds needed for GPU driven rendering
	if (Builder.IsGPUQuadTree())
	{
		WaterQuadTreeMinHeight = DBL_MAX;
		WaterQuadTreeMaxHeight = -DBL_MAX;

		const TArrayView<const FWaterBodyRenderData> WaterBodyRenderData = WaterQuadTree.GetWaterBodyRenderData();
		for (const FWaterBodyRenderData& RenderData : WaterBodyRenderData)
		{
			WaterQuadTreeMinHeight = FMath::Min(WaterQuadTreeMinHeight, RenderData.BoundsMinZ - RenderData.MaxWaveHeight);
			WaterQuadTreeMaxHeight = FMath::Max(WaterQuadTreeMaxHeight, RenderData.BoundsMaxZ + RenderData.MaxWaveHeight);
		}

		if (WaterQuadTreeMinHeight == DBL_MAX)
		{
			WaterQuadTreeMinHeight = 0.0;
			WaterQuadTreeMaxHeight = 0.0;
		}
		WaterQuadTreeMinHeight -= 1.0;
		WaterQuadTreeMaxHeight += 1.0;
	}

	if (!WaterInstanceDataBuffers || !WaterMeshUserDataBuffers)
	{
		const int32 TotalLeafNodes = Builder.GetMaxLeafCount();
		WaterInstanceDataBuffers = MakeUnique<FWaterInstanceDataBuffersType>(TotalLeafNodes);
		WaterMeshUserDataBuffers = MakeUnique<FWaterMeshUserDataBuffersType>(WaterInstanceDataBuffers.Get());
	}
}

void FViewWaterQuadTree::TraverseGPUQuadTree(FRDGBuilder& GraphBuilder, bool bDepthBufferIsPopulated)
{
	if (bNeedToTraverseGPUQuadTree)
	{
		if (!QuadTreeGPU.IsInitialized())
		{
			BuildGPUQuadTree(GraphBuilder);
		}
		FWaterQuadTreeGPU::FTraverseParams TraverseParams = MoveTemp(WaterQuadTreeGPUTraverseParams);
		TraverseParams.bDepthBufferIsPopulated = bDepthBufferIsPopulated;
		QuadTreeGPU.Traverse(GraphBuilder, TraverseParams);
		bNeedToTraverseGPUQuadTree = false;
		WaterQuadTreeGPUTraverseParams = {};
	}
}

FViewWaterQuadTree::FUserDataAndIndirectArgs FViewWaterQuadTree::PrepareGPUQuadTreeForRendering(const TArray<const FSceneView*>& Views, uint32 VisibilityMap, FMeshElementCollector& Collector, const FWaterQuadTreeConstants& QuadTreeConstants, const TArrayView<const EWaterMeshRenderGroupType>& BatchRenderGroups, FRHICommandListBase& RHICmdList) const
{
	bool bEncounteredISRView = false;
	TArray<const FSceneView*> VisibleViews;

	// Gather per view data for all renderable views
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FSceneView* View = Views[ViewIndex];
		if (!bEncounteredISRView && View->IsInstancedStereo() && IStereoRendering::IsAPrimaryView(*View))
		{
			bEncounteredISRView = true;
		}

		// skip gathering visible tiles from instanced right eye views
		if ((VisibilityMap & (1 << ViewIndex)) && (!bEncounteredISRView || View->IsPrimarySceneView()))
		{
			VisibleViews.Add(View);
		}
	}

	const int32 NumWaterMaterials = WaterQuadTree.GetWaterMaterials().Num();
	const int32 NumBucketsIndirect = NumWaterMaterials; // Indirect draws use a single density mesh tile
	const int32 NumVisibleViews = VisibleViews.Num();
	const int32 NumIndirectDrawCalls = NumBucketsIndirect * NumVisibleViews;
	const int32 LeafCountUpperBound = WaterQuadTree.GetMaxLeafCount() * NumVisibleViews;
	const int32 NumInstanceDataSlots = FMath::Max(1, static_cast<int32>(LeafCountUpperBound * FMath::Clamp(CVarWaterMeshGPUQuadInstanceDataAllocMult.GetValueOnRenderThread(), 0.0f, 8.0f)));

	// Allocate buffers for the indirect draws
	struct FIndirectDrawResources
	{
		TRefCountPtr<FRDGPooledBuffer> IndirectArgs;
		TRefCountPtr<FRDGPooledBuffer> InstanceDataOffsets;
		TRefCountPtr<FRDGPooledBuffer> InstanceData0;
		TRefCountPtr<FRDGPooledBuffer> InstanceData1;
		TRefCountPtr<FRDGPooledBuffer> InstanceData2;
		TRefCountPtr<FRDGPooledBuffer> InstanceData3;
	};
	FIndirectDrawResources& IndirectDrawResources = Collector.AllocateOneFrameResource<FIndirectDrawResources>();
	IndirectDrawResources.IndirectArgs = AllocatePooledBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDrawIndexedIndirectParameters>(FMath::Max(1, NumIndirectDrawCalls)), TEXT("WaterQuadTree.IndirectArgsBuffer"));
	IndirectDrawResources.InstanceDataOffsets = AllocatePooledBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), FMath::Max(1, NumIndirectDrawCalls)), TEXT("WaterQuadTree.InstanceDataOffsetsBuffer"));
	IndirectDrawResources.InstanceData0 = AllocatePooledBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), FMath::Max(1, NumInstanceDataSlots)), TEXT("WaterQuadTree.InstanceDataBuffer0"));
	IndirectDrawResources.InstanceData1 = AllocatePooledBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), FMath::Max(1, NumInstanceDataSlots)), TEXT("WaterQuadTree.InstanceDataBuffer1"));
	IndirectDrawResources.InstanceData2 = AllocatePooledBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), FMath::Max(1, NumInstanceDataSlots)), TEXT("WaterQuadTree.InstanceDataBuffer2"));
	if (WITH_WATER_SELECTION_SUPPORT != 0)
	{
		IndirectDrawResources.InstanceData3 = AllocatePooledBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), FMath::Max(1, NumInstanceDataSlots)), TEXT("WaterQuadTree.InstanceDataBuffer3"));
	}


	// Traverse quadtree. We actually only set up the parameters here; the actual traversal is done later in the frame in a callback invoked from FWaterViewExtension::PreRenderBasePass_RenderThread.
	// That callback accesses these params, so we need to ensure that they are still valid at that point.
	{
		WaterQuadTreeGPUTraverseParams = {};
		WaterQuadTreeGPUTraverseParams.OutIndirectArgsBuffer = IndirectDrawResources.IndirectArgs;
		WaterQuadTreeGPUTraverseParams.OutInstanceDataOffsetsBuffer = IndirectDrawResources.InstanceDataOffsets;
		WaterQuadTreeGPUTraverseParams.OutInstanceData0Buffer = IndirectDrawResources.InstanceData0;
		WaterQuadTreeGPUTraverseParams.OutInstanceData1Buffer = IndirectDrawResources.InstanceData1;
		WaterQuadTreeGPUTraverseParams.OutInstanceData2Buffer = IndirectDrawResources.InstanceData2;
		WaterQuadTreeGPUTraverseParams.OutInstanceData3Buffer = IndirectDrawResources.InstanceData3;
		WaterQuadTreeGPUTraverseParams.Views = MoveTemp(VisibleViews);
		WaterQuadTreeGPUTraverseParams.QuadTreePosition = FVector(WaterQuadTree.GetTileRegion().Min, WaterQuadTreeMinHeight);
		WaterQuadTreeGPUTraverseParams.CullingBounds = WaterQuadTree.GetTileRegion().ShiftBy(-WaterQuadTree.GetTileRegion().Min);
		WaterQuadTreeGPUTraverseParams.NumDensities = QuadTreeConstants.DensityCount;
		WaterQuadTreeGPUTraverseParams.NumMaterials = WaterQuadTree.GetWaterMaterials().Num();
		WaterQuadTreeGPUTraverseParams.NumQuadsLOD0 = QuadTreeConstants.NumQuadsLOD0;
		WaterQuadTreeGPUTraverseParams.NumQuadsPerTileSide = QuadTreeConstants.NumQuadsPerIndirectDrawTile;
		WaterQuadTreeGPUTraverseParams.ForceCollapseDensityLevel = QuadTreeConstants.ForceCollapseDensityLevel;
		WaterQuadTreeGPUTraverseParams.LeafSize = WaterQuadTree.GetLeafSize();
		WaterQuadTreeGPUTraverseParams.LODScale = QuadTreeConstants.LODScale;
		WaterQuadTreeGPUTraverseParams.DebugShowTile = CVarWaterMeshShowTileBounds.GetValueOnRenderThread();
		WaterQuadTreeGPUTraverseParams.bWithWaterSelectionSupport = WITH_WATER_SELECTION_SUPPORT != 0;
		WaterQuadTreeGPUTraverseParams.bLODMorphingEnabled = !!CVarWaterMeshLODMorphEnabled.GetValueOnRenderThread();

		bNeedToTraverseGPUQuadTree = true;
	}

	FUserDataAndIndirectArgs Result;
	Result.IndirectArgs = IndirectDrawResources.IndirectArgs;

	using FWaterVertexFactoryUserDataWrapperType = TWaterVertexFactoryUserDataWrapper<WITH_WATER_SELECTION_SUPPORT>;
	FWaterVertexFactoryUserDataWrapperType* UserDataWrappers[3] = {};
	for (EWaterMeshRenderGroupType RenderGroup : BatchRenderGroups)
	{
		FWaterVertexFactoryUserDataWrapperType& UserDataWrapper = Collector.AllocateOneFrameResource<FWaterVertexFactoryUserDataWrapperType>();
		UserDataWrapper.UserData.RenderGroupType = RenderGroup;
		UserDataWrapper.UserData.QuadTreePosition = FVector(WaterQuadTree.GetTileRegion().Min, WaterQuadTreeMinHeight);
		UserDataWrapper.UserData.CaptureDepthRange = WaterQuadTreeMaxHeight - WaterQuadTreeMinHeight;
		UserDataWrapper.UserData.IndirectInstanceData0 = IndirectDrawResources.InstanceData0->GetRHI();
		UserDataWrapper.UserData.IndirectInstanceData1 = IndirectDrawResources.InstanceData1->GetRHI();
		UserDataWrapper.UserData.IndirectInstanceData2 = IndirectDrawResources.InstanceData2->GetRHI();
		UserDataWrapper.UserData.IndirectInstanceData3 = (WITH_WATER_SELECTION_SUPPORT != 0) ? IndirectDrawResources.InstanceData3->GetRHI() : nullptr;
		UserDataWrapper.UserData.IndirectInstanceDataOffsetsSRV = IndirectDrawResources.InstanceDataOffsets->GetSRV(RHICmdList, FRHIBufferSRVCreateInfo(PF_R32_UINT));
		UserDataWrapper.UserData.IndirectInstanceData0SRV = IndirectDrawResources.InstanceData0->GetSRV(RHICmdList, FRHIBufferSRVCreateInfo(PF_R32_UINT));
		UserDataWrapper.UserData.IndirectInstanceData1SRV = IndirectDrawResources.InstanceData1->GetSRV(RHICmdList, FRHIBufferSRVCreateInfo(PF_R32_UINT));
		UserDataWrapper.UserData.IndirectInstanceData2SRV = IndirectDrawResources.InstanceData2->GetSRV(RHICmdList, FRHIBufferSRVCreateInfo(PF_R32_UINT));
		UserDataWrapper.UserData.IndirectInstanceData3SRV = (WITH_WATER_SELECTION_SUPPORT != 0) ? IndirectDrawResources.InstanceData3->GetSRV(RHICmdList, FRHIBufferSRVCreateInfo(PF_R32_UINT)) : nullptr;

		Result.UserData[(int32)RenderGroup] = &UserDataWrapper.UserData;
	}

	return Result;
}

FViewWaterQuadTree::FWaterLODParams FViewWaterQuadTree::GetWaterLODParams(const FVector& Position, float LODScale) const
{
	float WaterHeightForLOD = 0.0f;
	WaterQuadTree.QueryInterpolatedTileBaseHeightAtLocation(FVector2D(Position), WaterHeightForLOD);

	// Need to let the lowest LOD morph globally towards the next LOD. When the LOD is done morphing, simply clamp the LOD in the LOD selection to effectively promote the lowest LOD to the same LOD level as the one above
	float DistToWater = FMath::Abs(Position.Z - WaterHeightForLOD) / LODScale;
	DistToWater = FMath::Max(DistToWater - 2.0f, 0.0f);
	DistToWater *= 2.0f;

	// Clamp to WaterTileQuadTree.GetLODCount() - 1.0f prevents the last LOD to morph
	const float FloatLOD = FMath::Clamp(FMath::Log2(DistToWater), 0.0f, WaterQuadTree.GetTreeDepth() - 1.0f);

	FWaterLODParams WaterLODParams;
	WaterLODParams.HeightLODFactor = FMath::Frac(FloatLOD);
	WaterLODParams.LowestLOD = FMath::Clamp(FMath::FloorToInt(FloatLOD), 0, WaterQuadTree.GetTreeDepth() - 1);
	WaterLODParams.WaterHeightForLOD = WaterHeightForLOD;

	return WaterLODParams;
}

void FViewWaterQuadTree::BuildGPUQuadTree(FRDGBuilder& GraphBuilder)
{
	auto BuildOrthoMatrix = [](float InOrthoWidth, float InOrthoHeight, float DepthRange)
	{
		const FMatrix::FReal OrthoWidth = InOrthoWidth / 2.0f;
		const FMatrix::FReal OrthoHeight = InOrthoHeight / 2.0f;

		const FMatrix::FReal NearPlane = 0.f;
		const FMatrix::FReal FarPlane = DepthRange;

		const FMatrix::FReal ZScale = 1.0f / (FarPlane - NearPlane);
		const FMatrix::FReal ZOffset = 0;

		return FReversedZOrthoMatrix(
			OrthoWidth,
			OrthoHeight,
			ZScale,
			ZOffset
		);
	};

	// Create GPU array of water body render data
	const TArrayView<const FWaterBodyRenderData> WaterBodyRenderData = WaterQuadTree.GetWaterBodyRenderData();
	TArray<FWaterQuadTreeGPU::FWaterBodyRenderDataGPU> WaterBodyRenderDataGPU;
	{
		WaterBodyRenderDataGPU.Reserve(WaterBodyRenderData.Num());

		for (const FWaterBodyRenderData& RenderData : WaterBodyRenderData)
		{
			FColor HitProxyColor = FColor(0, 0, 0, 0);
#if WITH_WATER_SELECTION_SUPPORT
			if (RenderData.HitProxy)
			{
				HitProxyColor = RenderData.HitProxy->Id.GetColor();
				HitProxyColor.A = RenderData.bWaterBodySelected ? 255 : 0;
			}
#endif

			FWaterQuadTreeGPU::FWaterBodyRenderDataGPU& RenderDataGPU = WaterBodyRenderDataGPU.AddDefaulted_GetRef();
			RenderDataGPU.WaterBodyIndex = RenderData.WaterBodyIndex;
			RenderDataGPU.MaterialIndex = RenderData.MaterialIndex;
			RenderDataGPU.RiverToLakeMaterialIndex = RenderData.RiverToLakeMaterialIndex;
			RenderDataGPU.RiverToOceanMaterialIndex = RenderData.RiverToOceanMaterialIndex;
			RenderDataGPU.WaterBodyType = RenderData.WaterBodyType;
			RenderDataGPU.HitProxyColorAndIsSelected = HitProxyColor.Bits;
			RenderDataGPU.SurfaceBaseHeight = float(RenderData.SurfaceBaseHeight - WaterQuadTreeMinHeight);
			RenderDataGPU.MaxWaveHeight = RenderData.MaxWaveHeight;
		}
	}

	const FIntPoint Resolution = WaterQuadTree.GetResolution();
	const FBox2D TileRegion = WaterQuadTree.GetTileRegion();
	const float LeafSize = WaterQuadTree.GetLeafSize();

	const FVector2D Center = TileRegion.Min + FVector2D(Resolution) * (double)LeafSize * 0.5;
	const FVector QuadTreeCorner = FVector(TileRegion.Min, WaterQuadTreeMinHeight); // Make everything relative to this to avoid LWC precision issues
	const float WaterQuadTreeDepthRange = WaterQuadTreeMaxHeight - WaterQuadTreeMinHeight;
	const float RcpWaterQuadTreeDepthRange = 1.0f / WaterQuadTreeDepthRange;

	// Create view/projection matrices
	const FVector ViewLocation = FVector(Center, WaterQuadTreeMaxHeight) - QuadTreeCorner;
	const FVector LookAt = ViewLocation - FVector(0.0, 0.0, 1.0);
	const FMatrix44f ViewMatrix = FMatrix44f(FLookAtMatrix(ViewLocation, LookAt, FVector(0.f, -1.f, 0.f)));
	const FMatrix44f ProjectionMatrix = FMatrix44f(BuildOrthoMatrix(Resolution.X * LeafSize, Resolution.Y * LeafSize, WaterQuadTreeDepthRange));
	const FMatrix44f ViewProjection = ViewMatrix * ProjectionMatrix;

	bool bAllDrawsAreConservativeRasterCompatible = true;

	// Create array of water body draws
	TArray<FWaterQuadTreeGPU::FDraw> Draws;
	{
		const TArray<FWaterBodyQuadTreeRasterInfo>& WaterBodyRasterInfos = WaterQuadTree.GetWaterBodyRasterInfos();
		for (const FWaterBodyQuadTreeRasterInfo& Info : WaterBodyRasterInfos)
		{
			const FStaticMeshLODResources& LODResource = Info.RenderData->LODResources[0];
			for (const FStaticMeshSection& MeshSection : LODResource.Sections)
			{
				if (LODResource.IndexBuffer.GetRHI())
				{
					// Make relative to quadtree to avoid LWC precision issues
					FTransform LocalToQuadTreeWorld = Info.LocalToWorld;
					LocalToQuadTreeWorld.SetTranslation(LocalToQuadTreeWorld.GetTranslation() - QuadTreeCorner);

					const FWaterBodyRenderData& WBRenderData = WaterBodyRenderData[Info.WaterBodyRenderDataIndex];

					FWaterQuadTreeGPU::FDraw& Draw = Draws.AddDefaulted_GetRef();
					Draw.Transform = FMatrix44f(LocalToQuadTreeWorld.ToMatrixWithScale()) * ViewProjection;
					Draw.VertexBuffer = LODResource.VertexBuffers.PositionVertexBuffer.GetRHI();
					Draw.TexCoordBuffer = LODResource.VertexBuffers.StaticMeshVertexBuffer.TexCoordVertexBuffer.GetRHI();
					Draw.IndexBuffer = LODResource.IndexBuffer.GetRHI();
					Draw.FirstIndex = MeshSection.FirstIndex;
					Draw.NumPrimitives = MeshSection.NumTriangles;
					Draw.BaseVertexIndex = 0;
					Draw.NumVertices = MeshSection.MaxVertexIndex - MeshSection.MinVertexIndex + 1;
					Draw.WaterBodyRenderDataIndex = Info.WaterBodyRenderDataIndex;
					Draw.Priority = Info.Priority;
					Draw.MinZ = float(WBRenderData.BoundsMinZ - WaterQuadTreeMinHeight) * RcpWaterQuadTreeDepthRange;
					Draw.MaxZ = float(WBRenderData.BoundsMaxZ + WBRenderData.MaxWaveHeight - WaterQuadTreeMinHeight) * RcpWaterQuadTreeDepthRange;
					Draw.MaxWaveHeight = WBRenderData.MaxWaveHeight * RcpWaterQuadTreeDepthRange;
					Draw.bIsRiver = Info.bIsRiver;

					// Conservative raster mesh has 3 unique vertices per triangle and 3 UV channels with 32bit float UVs.
					bAllDrawsAreConservativeRasterCompatible = bAllDrawsAreConservativeRasterCompatible
						&& LODResource.VertexBuffers.PositionVertexBuffer.GetNumVertices() == LODResource.IndexBuffer.GetNumIndices()
						&& (LODResource.VertexBuffers.PositionVertexBuffer.GetNumVertices() % 3) == 0
						&& LODResource.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() == 3
						&& LODResource.VertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs();
				}
			}
		}
	}

	FWaterQuadTreeGPU::FInitParams Params;
	Params.WaterBodyRenderData = WaterBodyRenderDataGPU;
	Params.RequestedQuadTreeResolution = Resolution;
	Params.SuperSamplingFactor = FMath::Clamp(CVarWaterMeshGPUQuadTreeSuperSampling.GetValueOnRenderThread(), 1, 8);
	Params.NumMSAASamples = FMath::Clamp(CVarWaterMeshGPUQuadTreeMultiSampling.GetValueOnRenderThread(), 1, 8);
	Params.NumJitterSamples = FMath::Clamp(CVarWaterMeshGPUQuadTreeNumJitterSamples.GetValueOnRenderThread(), 1, 16);
	Params.JitterSampleFootprint = FMath::Max(CVarWaterMeshGPUQuadTreeJitterSampleFootprint.GetValueOnRenderThread(), 0.0f);
	Params.CaptureDepthRange = WaterQuadTreeDepthRange;
	Params.bUseMSAAJitterPattern = CVarWaterMeshGPUQuadTreeJitterPattern.GetValueOnRenderThread() == 1;
	Params.bUseConservativeRasterization = bAllDrawsAreConservativeRasterCompatible && CVarWaterMeshGPUQuadTreeConservativeRasterization.GetValueOnRenderThread() != 0;

	QuadTreeGPU.Init(GraphBuilder, Params, Draws);
}
