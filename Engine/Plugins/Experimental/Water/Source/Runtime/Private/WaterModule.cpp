// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterModule.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"

//////////////////////////////////////////////////////////////////////////
// FStateMachineModule

DEFINE_LOG_CATEGORY(LogWater);


class FWaterModule : public IWaterModuleInterface
{
public:
	virtual void StartupModule() override
	{
		FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("Water"))->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(TEXT("/Plugin/Water"), PluginShaderDir);
	}

	virtual void ShutdownModule() override
	{
	}

#if WITH_EDITOR
	virtual void SetWaterEditorServices(IWaterEditorServices* InWaterEditorServices) { WaterEditorServices = InWaterEditorServices; }
	virtual IWaterEditorServices* GetWaterEditorServices() const { return WaterEditorServices; }
#endif // WITH_EDITOR
private:
#if WITH_EDITORONLY_DATA
	IWaterEditorServices* WaterEditorServices = nullptr;
#endif // WITH_EDITORONLY_DATA
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FWaterModule, Water);

