// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterQuadTree.h"
#include "Materials/MaterialInterface.h"
#include "Math/ColorList.h"
#include "PrimitiveDrawingUtils.h"

#if WITH_WATER_SELECTION_SUPPORT
#endif // WITH_WATER_SELECTION_SUPPORT

void FWaterQuadTree::FNode::AddNodeForRender(const FNodeData& InNodeData, const FWaterBodyRenderData& InWaterBodyRenderData, int32 InDensityLevel, int32 InLODLevel, const FTraversalDesc& InTraversalDesc, FTraversalOutput& Output) const
{
	int32 MaterialIndex = InWaterBodyRenderData.MaterialIndex;
	uint32 NodeWaterBodyIndex = (uint32)InWaterBodyRenderData.WaterBodyIndex;
	int32 TileDebugID = InWaterBodyRenderData.WaterBodyType;

	// The base height of this tile comes either the top of the bounding box (for rivers) or the given base height (lakes and ocean)
	double BaseHeight = InWaterBodyRenderData.IsRiver() ? Bounds.Max.Z : InWaterBodyRenderData.SurfaceBaseHeight;

	// If there's a transition water body
	if (TransitionWaterBodyIndex > 0)
	{
		const FWaterBodyRenderData& TransitionWaterBodyRenderData = InNodeData.WaterBodyRenderData[TransitionWaterBodyIndex];
		
		// Only rivers can have transitions set up and rivers can't have custom transitions to other rivers
		check(InWaterBodyRenderData.IsRiver());
		check(!TransitionWaterBodyRenderData.IsRiver());

		if (TransitionWaterBodyRenderData.IsLake())
		{
			if (InWaterBodyRenderData.RiverToLakeMaterial)
			{
				MaterialIndex = InWaterBodyRenderData.RiverToLakeMaterialIndex;
			}
			else
			{
				MaterialIndex = InWaterBodyRenderData.MaterialIndex;
			}

			NodeWaterBodyIndex = (uint32)TransitionWaterBodyRenderData.WaterBodyIndex;
			BaseHeight = TransitionWaterBodyRenderData.SurfaceBaseHeight;
			TileDebugID = 3;
		}
		if (TransitionWaterBodyRenderData.IsOcean())
		{
			if (InWaterBodyRenderData.RiverToOceanMaterial)
			{
				MaterialIndex = InWaterBodyRenderData.RiverToOceanMaterialIndex;
			}
			else
			{
				MaterialIndex = InWaterBodyRenderData.MaterialIndex;
			}

			NodeWaterBodyIndex = (uint32)TransitionWaterBodyRenderData.WaterBodyIndex;
			BaseHeight = TransitionWaterBodyRenderData.SurfaceBaseHeight;
			TileDebugID = 4;
		}
	}

	const float BaseHeightTWS = BaseHeight + InTraversalDesc.PreViewTranslation.Z;

	const int32 DensityIndex = FMath::Min(InDensityLevel, InTraversalDesc.DensityCount - 1);
	const int32 BucketIndex = MaterialIndex * InTraversalDesc.DensityCount + DensityIndex;
	
	++Output.BucketInstanceCounts[BucketIndex];

	const FVector TranslatedWorldPosition(Bounds.GetCenter() + InTraversalDesc.PreViewTranslation);
	const FVector2D Scale(Bounds.GetSize());
	FStagingInstanceData& StagingData = Output.StagingInstanceData[Output.StagingInstanceData.AddUninitialized()];

	// Add the data to the bucket
	StagingData.BucketIndex = BucketIndex;
	StagingData.Data[0].X = TranslatedWorldPosition.X;
	StagingData.Data[0].Y = TranslatedWorldPosition.Y;
	StagingData.Data[0].Z = BaseHeightTWS;
	StagingData.Data[0].W = *(float*)&NodeWaterBodyIndex;

	// Lowest LOD isn't always 0, this increases with the height distance 
	const bool bIsLowestLOD = (InLODLevel == InTraversalDesc.LowestLOD);

	// Only allow a tile to morph if it's not the last density level and not the last LOD level, sicne there is no next level to morph to
	const uint32 bShouldMorph = (InTraversalDesc.bLODMorphingEnabled && (DensityIndex != InTraversalDesc.DensityCount - 1)) ? 1 : 0;
	// Tiles can morph twice to be able to morph between 3 LOD levels. Next to last density level can only morph once
	const uint32 bCanMorphTwice = (DensityIndex < InTraversalDesc.DensityCount - 2) ? 1 : 0;

	// Pack some of the data to save space. LOD level in the lower 8 bits and then bShouldMorph in the 9th bit and bCanMorphTwice in the 10th bit
	const uint32 BitPackedChannel = ((uint32)(InLODLevel) & 0xFF) | (bShouldMorph << 8) | (bCanMorphTwice << 9);

	// Should morph
	StagingData.Data[1].X = *(float*)&BitPackedChannel;
	StagingData.Data[1].Y = bIsLowestLOD ? InTraversalDesc.HeightMorph : 0.0f;
	StagingData.Data[1].Z = Scale.X;
	StagingData.Data[1].W = Scale.Y;

#if WITH_WATER_SELECTION_SUPPORT
	// Instance Hit Proxy ID
	FLinearColor HitProxyColor = InWaterBodyRenderData.HitProxy->Id.GetColor().ReinterpretAsLinear();
	StagingData.Data[2].X = HitProxyColor.R;
	StagingData.Data[2].Y = HitProxyColor.G;
	StagingData.Data[2].Z = HitProxyColor.B;
	StagingData.Data[2].W = InWaterBodyRenderData.bWaterBodySelected ? 1.0f : 0.0f;
#endif // WITH_WATER_SELECTION_SUPPORT

	++Output.InstanceCount;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Debug drawing
	if (InTraversalDesc.DebugShowTile != 0)
	{
		FColor Color;
		if (InTraversalDesc.DebugShowTile == 1)
		{
			static FColor WaterTypeColor[] = { FColor::Red, FColor::Green, FColor::Blue, FColor::Yellow, FColor::Purple };
			Color = WaterTypeColor[TileDebugID];
		}
		else if (InTraversalDesc.DebugShowTile == 2)
		{
			Color = GColorList.GetFColorByIndex(InLODLevel + 1);
		}
		else if (InTraversalDesc.DebugShowTile == 3)
		{

			Color = GColorList.GetFColorByIndex(DensityIndex + 1);
		}

		DrawWireBox(InTraversalDesc.DebugPDI, Bounds.ExpandBy(FVector(-20.0f, -20.0f, 0.0f)), Color, InTraversalDesc.bDebugDrawIntoForeground ? SDPG_Foreground : SDPG_World);
	}
#endif
}

bool FWaterQuadTree::FNode::CanRender(int32 InDensityLevel, int32 InForceCollapseDensityLevel, const FWaterBodyRenderData& InWaterBodyRenderData) const
{
	//check(InWaterBodyRenderData.Material);

	// Can render if the density level is (in addition to same water bodies in all descendants) either above the force collapse level or if the subtree is complete
	return InWaterBodyRenderData.Material && IsSubtreeSameWaterBody && ((InDensityLevel > InForceCollapseDensityLevel) || HasCompleteSubtree);
}

// Same as SelectLOD, but we recurse only inside the same LOD, so no need to do bounds checks
void FWaterQuadTree::FNode::SelectLODRefinement(const FNodeData& InNodeData, int32 DensityLevel, int32 InLODLevel, const FTraversalDesc& InTraversalDesc, FTraversalOutput& Output) const
{
	const FWaterBodyRenderData& WaterBodyRenderData = InNodeData.WaterBodyRenderData[WaterBodyIndex];
	const FVector CenterPosition = Bounds.GetCenter();
	const FVector Extent = Bounds.GetExtent();
	const FBox2D Bounds2D = FBox2D(FVector2D(Bounds.Min), FVector2D(Bounds.Max));

	// Early out on frustum culling 
	check(InTraversalDesc.WaterInfoBounds.bIsValid);
	if (InTraversalDesc.Frustum.IntersectBox(CenterPosition, Extent) && Bounds2D.Intersect(InTraversalDesc.WaterInfoBounds))
	{
		// Occlusion culling
		if (InTraversalDesc.OcclusionCullingResults 
			&& (BreadthFirstIndex < (uint32)InTraversalDesc.OcclusionCullingFarMeshOffset) 
			&& InTraversalDesc.OcclusionCullingResults->IsValidIndex(BreadthFirstIndex)
			&& (*InTraversalDesc.OcclusionCullingResults)[BreadthFirstIndex])
		{
			return;
		}

		// This LOD can represent all its leaf nodes, simply add node
		if (CanRender(DensityLevel, InTraversalDesc.ForceCollapseDensityLevel, WaterBodyRenderData))
		{
			AddNodeForRender(InNodeData, WaterBodyRenderData, DensityLevel, InLODLevel, InTraversalDesc, Output);
		}
		else
		{
			// If not, we need to recurse down the children until we find one that can be rendered
			for (int32 ChildIndex : Children)
			{
				if (ChildIndex > 0)
				{
					InNodeData.Nodes[ChildIndex].SelectLODRefinement(InNodeData, DensityLevel + 1, InLODLevel, InTraversalDesc, Output);
				}
			}
		}
	}
}

void FWaterQuadTree::FNode::SelectLOD(const FNodeData& InNodeData, int32 InLODLevel, const FTraversalDesc& InTraversalDesc, FTraversalOutput& Output) const
{
	const FWaterBodyRenderData& WaterBodyRenderData = InNodeData.WaterBodyRenderData[WaterBodyIndex];
	const FVector CenterPosition = Bounds.GetCenter();
	const FVector Extent = Bounds.GetExtent();
	const FBox2D Bounds2D = FBox2D(FVector2D(Bounds.Min), FVector2D(Bounds.Max));

	// Early out on frustum culling 
	check(InTraversalDesc.WaterInfoBounds.bIsValid);
	if (!InTraversalDesc.Frustum.IntersectBox(CenterPosition, Extent) || !Bounds2D.Intersect(InTraversalDesc.WaterInfoBounds))
	{
		// Handled
		return;
	}

	// Occlusion culling
	if (InTraversalDesc.OcclusionCullingResults
		&& (BreadthFirstIndex < (uint32)InTraversalDesc.OcclusionCullingFarMeshOffset)
		&& InTraversalDesc.OcclusionCullingResults->IsValidIndex(BreadthFirstIndex)
		&& (*InTraversalDesc.OcclusionCullingResults)[BreadthFirstIndex])
	{
		return;
	}

	// Distance to tile (if 0, position is inside quad)
	const float ClosestDistanceToTile = FMath::Sqrt(Bounds2D.ComputeSquaredDistanceToPoint(FVector2D(InTraversalDesc.ObserverPosition)));

	// If quad is outside this LOD range, it belongs to the LOD above, assume it fits in that LOD and drill down to find renderable nodes
	if (ClosestDistanceToTile > GetLODDistance(InLODLevel, InTraversalDesc.LODScale))
	{
		// This node is capable of representing all its leaf nodes, so just submit this node
		if (CanRender(0, InTraversalDesc.ForceCollapseDensityLevel, WaterBodyRenderData))
		{
			AddNodeForRender(InNodeData, WaterBodyRenderData, 1, InLODLevel + 1, InTraversalDesc, Output);
		}
		else
		{
			// If not, we need to recurse down the children until we find one that can be rendered
			for (int32 ChildIndex : Children)
			{
				if (ChildIndex > 0)
				{
					InNodeData.Nodes[ChildIndex].SelectLODRefinement(InNodeData, 2, InLODLevel + 1, InTraversalDesc, Output);
				}
			}
		}

		// Handled
		return;
	}

	// Last LOD, simply add node
	if (InLODLevel == 0)
	{
		if (CanRender(0, InTraversalDesc.ForceCollapseDensityLevel, WaterBodyRenderData))
		{
			AddNodeForRender(InNodeData, WaterBodyRenderData, 0, InLODLevel, InTraversalDesc, Output);
		}
	}
	else
	{
		// This quad is fully inside its LOD (also qualifies if it's simply the lowest LOD)
		if (ClosestDistanceToTile > GetLODDistance(InLODLevel - 1, InTraversalDesc.LODScale) || InLODLevel == InTraversalDesc.LowestLOD)
		{
			// This node is capable of representing all its leaf nodes, so just submit this node
			if (CanRender(0, InTraversalDesc.ForceCollapseDensityLevel, WaterBodyRenderData))
			{
				AddNodeForRender(InNodeData, WaterBodyRenderData, 0, InLODLevel, InTraversalDesc, Output);
			}
			else
			{
				// If not, we need to recurse down the children until we find one that can be rendered
				for (int32 ChildIndex : Children)
				{
					if (ChildIndex > 0)
					{
						InNodeData.Nodes[ChildIndex].SelectLODRefinement(InNodeData, 1, InLODLevel, InTraversalDesc, Output);
					}
				}
			}
		}
		else
		{
			// If this node has a complete subtree it will not contain any actual children, they are implicit to save memory so we generate them here
			if (HasCompleteSubtree && IsSubtreeSameWaterBody)
			{
				FNode ChildNode;
				const FVector HalfBoundSize(Extent.X, Extent.Y, Extent.Z*2.0f);
				const FVector HalfOffsets[] = { {0.0f, 0.0f, 0.0f}, {1.0f, 0.0f, 0.0f} , {0.0f, 1.0f, 0.0f} , {1.0f, 1.0f, 0.0f} };
				for (int i = 0; i < 4; i++)
				{
					const FVector ChildMin = Bounds.Min + HalfBoundSize * HalfOffsets[i];
					const FVector ChildMax = ChildMin + HalfBoundSize;
					const FBox ChildBounds(ChildMin, ChildMax);

					// Create a temporary node to traverse
					ChildNode.HasCompleteSubtree = 1;
					ChildNode.IsSubtreeSameWaterBody = 1;
					ChildNode.TransitionWaterBodyIndex = TransitionWaterBodyIndex;
					ChildNode.WaterBodyIndex = WaterBodyIndex;
					ChildNode.Bounds = ChildBounds;

					ChildNode.SelectLOD(InNodeData, InLODLevel - 1, InTraversalDesc, Output);
				}
			}
			else
			{
				for (int32 ChildIndex : Children)
				{
					if (ChildIndex > 0)
					{
						InNodeData.Nodes[ChildIndex].SelectLOD(InNodeData, InLODLevel - 1, InTraversalDesc, Output);
					}
				}
			}
		}
	}
}

void FWaterQuadTree::FNode::AddNodes(FNodeData& InNodeData, const FBox& InMeshBounds, const FBox& InWaterBodyBounds, uint32 InWaterBodyIndex, int32 InLODLevel, uint32 InParentIndex)
{
	const FWaterBodyRenderData& InWaterBody = InNodeData.WaterBodyRenderData[InWaterBodyIndex];
	const FWaterBodyRenderData& ThisWaterBody = InNodeData.WaterBodyRenderData[WaterBodyIndex];

	Bounds.Max.Z = FMath::Max(Bounds.Max.Z, InWaterBodyBounds.Max.Z);
	Bounds.Min.Z = FMath::Min(Bounds.Min.Z, InWaterBodyBounds.Min.Z);

	// Check is this node should be marked for material overlap 
	if (InWaterBody.IsRiver() && ((ThisWaterBody.IsLake() && InWaterBody.RiverToLakeMaterial) || (ThisWaterBody.IsOcean() && InWaterBody.RiverToOceanMaterial)))
	{
		// If the incoming water body is a river with a transition, and if the existing water body is either ocean or lake, we set transition water body index, but only if the new water body has a higher priority
		if ((TransitionWaterBodyIndex == 0) || (ThisWaterBody.Priority >= InNodeData.WaterBodyRenderData[TransitionWaterBodyIndex].Priority))
		{
			TransitionWaterBodyIndex = (uint16)WaterBodyIndex;
		}
	}
	else if (ThisWaterBody.IsRiver() && ((InWaterBody.IsLake() && ThisWaterBody.RiverToLakeMaterial) || (InWaterBody.IsOcean() && ThisWaterBody.RiverToOceanMaterial)))
	{
		// If the existing water body is a river with a transition, and if the incoming water body is either ocean or lake, we set transition water body index, but only if the new water body has a higher priority
		if (TransitionWaterBodyIndex == 0 || InWaterBody.Priority >= InNodeData.WaterBodyRenderData[TransitionWaterBodyIndex].Priority)
		{
			TransitionWaterBodyIndex = (uint16)InWaterBodyIndex;
		}
	}

	// Assign the render data here (based on priority)
	if (InWaterBody.Priority >= ThisWaterBody.Priority)
	{
		WaterBodyIndex = InWaterBodyIndex;

		// Cache whether or not this node has a material
		HasMaterial = InNodeData.WaterBodyRenderData[WaterBodyIndex].Material != nullptr;
	}

	// Reset the flags before going through the children. These flags will be turned off by recursion if the state changes
	// Setting them here ensures leaf nodes are marked as complete subtrees, allowing them to be further implicitly subdivided
	IsSubtreeSameWaterBody = 1;
	HasCompleteSubtree = 1;

	// This is a leaf node, stop here
	if (InLODLevel == 0)
	{
		return;
	}

	FVector2D HalfBoundSize = FVector2D(Bounds.GetSize()) * 0.5f;

	FNode PrevChildNode = InNodeData.Nodes[0];
	const FVector2D HalfOffsets[] = { {0.0f, 0.0f}, {1.0f, 0.0f} , {0.0f, 1.0f} , {1.0f, 1.0f} };
	for (int32 i = 0; i < 4; i++)
	{
		if (Children[i] > 0)
		{
			if (InNodeData.Nodes[Children[i]].Bounds.IntersectXY(InWaterBodyBounds))
			{
				InNodeData.Nodes[Children[i]].AddNodes(InNodeData, InMeshBounds, InWaterBodyBounds, InWaterBodyIndex, InLODLevel - 1, Children[i]);
			}
		}
		else
		{
			// Check if this child needs to be created. If yes, initialize it with the depth bounds of InBounds
			const FVector ChildMin(FVector2D(Bounds.Min) + HalfBoundSize * HalfOffsets[i], InWaterBodyBounds.Min.Z);
			const FVector ChildMax(FVector2D(ChildMin) + HalfBoundSize, InWaterBodyBounds.Max.Z);
			const FBox ChildBounds(ChildMin, ChildMax);

			if (ChildBounds.IntersectXY(InWaterBodyBounds) && ChildBounds.IntersectXY(InMeshBounds))
			{
				// All nodes have been allocated upfront, no reallocation should occur : 
				check(InNodeData.Nodes.Num() < InNodeData.Nodes.Max());
				Children[i] = InNodeData.Nodes.Emplace();
				InNodeData.Nodes[Children[i]].Bounds = ChildBounds; 
				InNodeData.Nodes[Children[i]].ParentIndex = InParentIndex;
				InNodeData.Nodes[Children[i]].AddNodes(InNodeData, InMeshBounds, InWaterBodyBounds, InWaterBodyIndex, InLODLevel - 1, Children[i]);
			}
		}

		if (Children[i] > 0)
		{
			const FNode& ChildNode = InNodeData.Nodes[Children[i]];

			// If INVALID_PARENT, compare against current since there are no previous children
			PrevChildNode = (PrevChildNode.ParentIndex == INVALID_PARENT ? ChildNode : PrevChildNode);

			// If the child doesn't have a subtree with same water bodies, then this node doesn't either
			if (ChildNode.IsSubtreeSameWaterBody == 0 || !ChildNode.CanMerge(PrevChildNode))
			{
				IsSubtreeSameWaterBody = 0;
			}

			PrevChildNode = ChildNode;

			if (ChildNode.HasCompleteSubtree == 0)
			{
				HasCompleteSubtree = 0;
			}
		}
		else
		{
			// If the child isn't allocated, this can not be a complete subtree. If an internal node doesn't have a complete subtree but has the same waterbody, that means it can be forcefully rendered
			HasCompleteSubtree = 0;
		}
	}
}

bool FWaterQuadTree::FNode::QueryBaseHeightAtLocation(const FNodeData& InNodeData, const FVector2D& InWorldLocationXY, float& OutHeight) const
{
	// Early out if subtree is complete and of same waterbody. 
	// Note: Since we prune the quadtree of anything below this condition, it means there are no more granular nodes to fetch below this. In theory we could skip the pruning and have slightly more accurate height sampling, since rivers might have leaf nodes with individual bounds.
	// Same condition as leaf nodes
	if (HasCompleteSubtree && IsSubtreeSameWaterBody)
	{
		// Return "accurate" base height when there's a valid sample
		OutHeight = InNodeData.WaterBodyRenderData[WaterBodyIndex].IsRiver() ? Bounds.Max.Z : InNodeData.WaterBodyRenderData[WaterBodyIndex].SurfaceBaseHeight;

		return true;
	}

	for (int32 ChildIndex : Children)
	{
		if (ChildIndex > 0)
		{
			const FNode& ChildNode = InNodeData.Nodes[ChildIndex];
			const FBox ChildBounds = ChildNode.Bounds;

			// Check if point is inside (or on the Min edges) of the child bounds
			if ((InWorldLocationXY.X >= ChildBounds.Min.X) && (InWorldLocationXY.X < ChildBounds.Max.X)
				&& (InWorldLocationXY.Y >= ChildBounds.Min.Y) && (InWorldLocationXY.Y < ChildBounds.Max.Y))
			{
				return ChildNode.QueryBaseHeightAtLocation(InNodeData, InWorldLocationXY, OutHeight);
			}
		}
	}

	// Return regular base height when there's not valid sample
	OutHeight = InNodeData.WaterBodyRenderData[WaterBodyIndex].SurfaceBaseHeight;

	// Point is not in any of these children, return false
	return false;
}

bool FWaterQuadTree::FNode::QueryBoundsAtLocation(const FNodeData& InNodeData, const FVector2D& InWorldLocationXY, FBox& OutBounds) const
{
	OutBounds = Bounds;

	int32 ChildCount = 0;
	for (int32 ChildIndex : Children)
	{
		if (ChildIndex > 0)
		{
			ChildCount++;
			const FNode& ChildNode = InNodeData.Nodes[ChildIndex];
			const FBox ChildBounds = ChildNode.Bounds;

			// Check if point is inside (or on the Min edges) of the child bounds
			if ((InWorldLocationXY.X >= ChildBounds.Min.X) && (InWorldLocationXY.X < ChildBounds.Max.X)
				&& (InWorldLocationXY.Y >= ChildBounds.Min.Y) && (InWorldLocationXY.Y < ChildBounds.Max.Y))
			{
				return ChildNode.QueryBoundsAtLocation(InNodeData, InWorldLocationXY, OutBounds);
			}
		}
	}

	// No children, this is a leaf node, return true. Otherwise reaching here means none of the children contain the sampling location, so return false
	return ChildCount == 0;
}

void FWaterQuadTree::InitTree(const FBox2D& InBounds, float InTileSize, FIntPoint InExtentInTiles, bool bInIsGPUQuadTree)
{
	ensure(InBounds.GetArea() > 0.0f);
	ensure(InTileSize > 0.0f);
	ensure(InExtentInTiles.X > 0);
	ensure(InExtentInTiles.Y > 0);

	bIsGPUQuadTree = bInIsGPUQuadTree;

	FarMeshData.Clear();

	// Maximum number of allocated leaf nodes for this config
	MaxLeafCount = InExtentInTiles.X*InExtentInTiles.Y*4;
	LeafSize = InTileSize;
	ExtentInTiles = InExtentInTiles;

	// Calculate the depth of the tree. This also corresponds to the LOD count. 0 means root is leaf node
	// Find a pow2 tile resolution that contains the user defined extent in tiles
	const int32 MaxDim = (int32)FMath::Max(InExtentInTiles.X * 2, InExtentInTiles.Y * 2);
	const float RootDim = (float)FMath::RoundUpToPowerOfTwo(MaxDim);

	TileRegion = InBounds;

	WaterBodyRasterInfos.Reset();
	BreadthFirstOrder.Reset();

	// Allocate theoretical max, shrink later in Lock()
	// This is so that the node array doesn't move in memory while inserting
	if (!bIsGPUQuadTree)
	{
		NodeData.Nodes.Empty((float)(FMath::Square(RootDim) * 4) / 3.0f);
	}
	else
	{
		NodeData.Nodes.Empty(1);
	}

	// Add defaulted water body render data to slot 0. This is the "null" render data, pointed to by all newly created nodes. Has lowest priority so it will always be overwritten
	NodeData.WaterBodyRenderData.Empty(1);
	NodeData.WaterBodyRenderData.AddDefaulted();

	ensure(NodeData.Nodes.Num() == 0);

	// Add the root node at slot 0
	NodeData.Nodes.Emplace();

	const float RootWorldSize = RootDim * InTileSize;

	TreeDepth = (int32)FMath::Log2(RootDim);

	// Init root node bounds with invalid Z since that will be updated as nodes are added to the tree
	NodeData.Nodes[0].Bounds = FBox(FVector(TileRegion.Min, TNumericLimits<float>::Max()), FVector(TileRegion.Min + FVector2D(RootWorldSize, RootWorldSize), TNumericLimits<float>::Lowest()));

	ensure(NodeData.Nodes.Num() == 1);

	bIsReadOnly = false;
}

void FWaterQuadTree::Unlock(bool bPruneRedundantNodes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Unlock);

	if (bPruneRedundantNodes)
	{
		auto SwapRemove = [&](int32 NodeIndex, int32 EndIndex)
		{
			if (NodeIndex != EndIndex)
			{
				// Swap to back. All the children of this node would have already been removed (or didn't exist to begin with), so don't care about those
				NodeData.Nodes.SwapMemory(NodeIndex, EndIndex);

				// Patch up the newly moved good node (parent and children)
				FNode& MovedNode = NodeData.Nodes[NodeIndex];
				FNode& MovedNodeParent = NodeData.Nodes[MovedNode.ParentIndex];

				for (int32 i = 0; i < 4; i++)
				{
					if (MovedNode.Children[i] > 0)
					{
						NodeData.Nodes[MovedNode.Children[i]].ParentIndex = NodeIndex;
					}

					if (MovedNodeParent.Children[i] == EndIndex)
					{
						MovedNodeParent.Children[i] = NodeIndex;
					}
				}
			}
		};

		// Remove redundant nodes
		// Remove from the back, since all removalbe children are further back than their parent in the node list and we want to remove bottom-up
		int32 EndIndex = NodeData.Nodes.Num() - 1;
		for (int NodeIndex = EndIndex; NodeIndex > 0; NodeIndex--)
		{
			FNode& ParentNode = NodeData.Nodes[NodeData.Nodes[NodeIndex].ParentIndex];

			// Parent has complete subtree of the same water body, this node is redundant
			if (ParentNode.HasCompleteSubtree && ParentNode.IsSubtreeSameWaterBody)
			{
				// Delete all children (not strictly necessary, but now we don't leave any dangling/incorrect child pointers around)
				FMemory::Memzero(&ParentNode.Children, sizeof(uint32) * 4);

				SwapRemove(NodeIndex, EndIndex);

				// Move back one step down
				EndIndex--;
			}
			else if (!NodeData.Nodes[NodeIndex].HasMaterial && NodeData.Nodes[NodeIndex].HasCompleteSubtree && NodeData.Nodes[NodeIndex].IsSubtreeSameWaterBody)
			{
				for (int32 i = 0; i < 4; i++)
				{
					if (ParentNode.Children[i] == NodeIndex)
					{
						ParentNode.Children[i] = 0;
					}
				}

				SwapRemove(NodeIndex, EndIndex);

				// Move back one step down
				EndIndex--;
			}
		}

		NodeData.Nodes.SetNum(EndIndex + 1);
	}

	// Compute breadth first indices
	{
		class FQueue
		{
		public:
			explicit FQueue(int32 InCapacity) 
			{ 
				Capacity = InCapacity;
				Queue.SetNumUninitialized(Capacity);
			}

			void Add(int32 Element)
			{
				check(NumElements < Capacity);
				Queue[(Front + NumElements) % Capacity] = Element;
				++NumElements;
			}

			int32 Pop() 
			{ 
				check(NumElements > 0);
				const int32 Result = Queue[Front]; 
				Front = (Front + 1) % Capacity;
				--NumElements;
				return Result;
			}

			bool IsEmpty() const { return NumElements == 0; }
		private:
			TArray<int32> Queue;
			int32 Capacity = 0;
			int32 Front = 0;
			int32 NumElements = 0;
		};
		
		FQueue Queue(FMath::Max(NodeData.Nodes.Num(), 1));
		uint32 NextBreadthFirstIndex = 0;
		
		check(BreadthFirstOrder.IsEmpty());
		BreadthFirstOrder.Reserve(NodeData.Nodes.Num());

		Queue.Add(0);

		while (!Queue.IsEmpty())
		{
			const int32 NodeIndex = Queue.Pop();
			FNode& Node = NodeData.Nodes[NodeIndex];
			Node.BreadthFirstIndex = NextBreadthFirstIndex++;
			BreadthFirstOrder.Add(NodeIndex);

			for (uint32 ChildIndex : Node.Children)
			{
				if (ChildIndex > 0 && ensure(NodeData.Nodes.IsValidIndex(ChildIndex)))
				{
					Queue.Add(ChildIndex);
				}
			}
		}
	}

	bIsReadOnly = true;
}

void FWaterQuadTree::AddWaterTilesInsideBounds(const FBox& InBounds, uint32 InWaterBodyIndex)
{
	check(!bIsReadOnly);
	check(!bIsGPUQuadTree);
	NodeData.Nodes[0].AddNodes(NodeData, FBox(FVector(TileRegion.Min, 0.0f), FVector(TileRegion.Max, 0.0f)),  InBounds, InWaterBodyIndex, TreeDepth, 0);
}

void FWaterQuadTree::AddOcean(const TArray<FVector2D>& InPoly, const FBox& InOceanBounds, uint32 InWaterBodyIndex)
{
	check(!bIsReadOnly);
	check(!bIsGPUQuadTree);
	const FBox2D OceanBounds(FVector2D(InOceanBounds.Min), FVector2D(InOceanBounds.Max));
	AddOceanRecursive(InPoly, OceanBounds, FVector2D(InOceanBounds.Min.Z, InOceanBounds.Max.Z), true, TreeDepth * 2, InWaterBodyIndex);
}

void FWaterQuadTree::AddLake(const TArray<FVector2D>& InPoly, const FBox& InLakeBounds, uint32 InWaterBodyIndex)
{
	check(!bIsReadOnly);
	check(!bIsGPUQuadTree);
	const FBox2D LakeBounds(FVector2D(NodeData.Nodes[0].Bounds.Min), FVector2D(NodeData.Nodes[0].Bounds.Max));
	AddLakeRecursive(InPoly, LakeBounds, FVector2D(InLakeBounds.Min.Z, InLakeBounds.Max.Z), true, TreeDepth * 2, InWaterBodyIndex);
}

void FWaterQuadTree::AddFarMesh(FMaterialRenderProxy* InFarMeshMaterial, const FBox2D& InInnerRegion, double InFarDistanceMeshExtent, double InFarDistanceMeshHeight)
{
	// Checking for not being read only here to keep things consistent with the other Add functions. In reality the FarMesh isn't added to the QuadTree itself, so it could technically be done whenever.
	ensure(!bIsReadOnly);
	ensure(InFarMeshMaterial);

	// Early out when there would be no far mesh rendering anyway
	if (InFarMeshMaterial == nullptr || InFarDistanceMeshExtent <= 0.0)
	{
		return;
	}

	// Far mesh is always 8 tiles around the quadtree region (marked as Q in diagram below) 
	//  _ _ _
	// |_|_|_|
	// |_|Q|_|
	// |_|_|_|
	FarMeshData.InstanceData.SetNum(8);
	FarMeshData.Material = InFarMeshMaterial;
	
	{
		// Center position of the total far mesh bounds
		const FVector FarMeshWorldCenter = FVector(InInnerRegion.GetCenter(), InFarDistanceMeshHeight);
		// 3D Extents from the center of the far mesh bounds to the edge. Bounds are 2cm tall (1cm above, 1cm below plane)
		const FVector FarMeshTotalExtent = FVector(InInnerRegion.GetExtent() + InFarDistanceMeshExtent, 1.0);
		FarMeshData.FarMeshBounds = FBox(FarMeshWorldCenter - FarMeshTotalExtent, FarMeshWorldCenter + FarMeshTotalExtent);
	}

	const FVector2D WaterCenter = InInnerRegion.GetCenter();
	const FVector2D WaterExtents = InInnerRegion.GetExtent();
	const FVector2D WaterSize = InInnerRegion.GetSize();
	const FVector2D TileOffets[] = { {-1.0, 1.0}, {0.0, 1.0}, {1.0, 1.0}, {1.0, 0.0}, {1.0, -1.0}, {0.0, -1.0}, {-1.0, -1.0}, {-1.0, 0.0} };

	for (int32 i = 0; i < 8; i++)
	{
		const FVector2D TilePos = WaterCenter + TileOffets[i] * (WaterExtents + 0.5 * InFarDistanceMeshExtent);
		FVector2D TileScale;
		TileScale.X = (TileOffets[i].X == 0.0) ? WaterSize.X : InFarDistanceMeshExtent;
		TileScale.Y = (TileOffets[i].Y == 0.0) ? WaterSize.Y : InFarDistanceMeshExtent;

		FarMeshData.InstanceData[i].WorldPosition = FVector(TilePos, InFarDistanceMeshHeight);
		FarMeshData.InstanceData[i].Scale = FVector2f(TileScale);
	}
}

void FWaterQuadTree::BuildMaterialIndices()
{
	int32 NextIdx = 0;
	TMap<FMaterialRenderProxy*, int32> MatToIdxMap;

	auto GetMatIdx = [&NextIdx, &MatToIdxMap](FMaterialRenderProxy* MaterialRenderProxy)
	{
		if (!MaterialRenderProxy)
		{
			return (int32)INDEX_NONE;
		}
		const int32* Found = MatToIdxMap.Find(MaterialRenderProxy);
		if (!Found)
		{
			Found = &MatToIdxMap.Add(MaterialRenderProxy, NextIdx++);
		}
		return *Found;
	};

	for (int32 Idx = 0; Idx < NodeData.WaterBodyRenderData.Num(); ++Idx)
	{
		FWaterBodyRenderData& Data = NodeData.WaterBodyRenderData[Idx];
		Data.MaterialIndex = GetMatIdx(Data.Material);
		Data.RiverToLakeMaterialIndex = GetMatIdx(Data.RiverToLakeMaterial);
		Data.RiverToOceanMaterialIndex = GetMatIdx(Data.RiverToOceanMaterial);
	}

	// Special case handling for Far Mesh
	FarMeshData.MaterialIndex = GetMatIdx(FarMeshData.Material);

	WaterMaterials.Empty(MatToIdxMap.Num());
	WaterMaterials.AddUninitialized(MatToIdxMap.Num());

	for (TMap<FMaterialRenderProxy*, int32>::TConstIterator It(MatToIdxMap); It; ++It)
	{
		WaterMaterials[It->Value] = It->Key;
	}
}

void FWaterQuadTree::BuildWaterTileInstanceData(const FTraversalDesc& InTraversalDesc, FTraversalOutput& Output) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BuildWaterTileInstanceData);
	check(bIsReadOnly);

	if (!bIsGPUQuadTree)
	{
		NodeData.Nodes[0].SelectLOD(NodeData, TreeDepth, InTraversalDesc, Output);
	}

	// Append Far Mesh tiles
	if (FarMeshData.InstanceData.Num() > 0 && FarMeshData.MaterialIndex != INDEX_NONE)
	{
		const int32 FarMeshTileCount = FarMeshData.InstanceData.Num();
		ensure(FarMeshTileCount == 8);

		// Bucket index calculation is MaterialIndex*DensityCount+CurrentDensity. Since far mesh doesn't have any Density(aka LOD) steps and should render only using a 2 triangle quad, we enter it only into the last Density bucket (this always corresponds to a 2 triangle quad). 
		const int32 BucketIndex = FarMeshData.MaterialIndex * InTraversalDesc.DensityCount + (InTraversalDesc.DensityCount - 1);
		
		for (int32 i = 0; i < FarMeshTileCount; i++)
		{
			// Frustum culling
			if (!InTraversalDesc.Frustum.IntersectBox(FarMeshData.InstanceData[i].WorldPosition, FVector(FarMeshData.InstanceData[i].Scale.X * 0.5, FarMeshData.InstanceData[i].Scale.Y * 0.5, 1.0)))
			{
				continue;
			}

			// Occlusion culling
			if (InTraversalDesc.OcclusionCullingResults 
				&& InTraversalDesc.OcclusionCullingResults->IsValidIndex(InTraversalDesc.OcclusionCullingFarMeshOffset + i)
				&& (*InTraversalDesc.OcclusionCullingResults)[InTraversalDesc.OcclusionCullingFarMeshOffset + i])
			{
				continue;
			}

			++Output.BucketInstanceCounts[BucketIndex];
			++Output.InstanceCount;

			// Build instance data
			// Transform worldposition to Translated World Position
			const FVector TranslatedWorldPosition(FarMeshData.InstanceData[i].WorldPosition + InTraversalDesc.PreViewTranslation);

			FStagingInstanceData& StagingInstanceData = Output.StagingInstanceData[Output.StagingInstanceData.AddUninitialized()];
			StagingInstanceData.BucketIndex = BucketIndex;
			StagingInstanceData.Data[0] = FVector4f(FVector4(TranslatedWorldPosition, 0.0));
			StagingInstanceData.Data[1] = FVector4f(FVector2f::ZeroVector, FarMeshData.InstanceData[i].Scale);
#if WITH_WATER_SELECTION_SUPPORT
			StagingInstanceData.Data[2] = FHitProxyId::InvisibleHitProxyId.GetColor().ReinterpretAsLinear();
#endif // WITH_WATER_SELECTION_SUPPORT

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			// Debug drawing
			if (InTraversalDesc.DebugShowTile != 0)
			{
				const FBox DebugBounds = FBox(FarMeshData.InstanceData[i].WorldPosition - FVector(FarMeshData.InstanceData[i].Scale.X * 0.5, FarMeshData.InstanceData[i].Scale.Y * 0.5, 1.0), FarMeshData.InstanceData[i].WorldPosition + FVector(FarMeshData.InstanceData[i].Scale.X * 0.5, FarMeshData.InstanceData[i].Scale.Y * 0.5, 1.0));
				DrawWireBox(InTraversalDesc.DebugPDI, DebugBounds.ExpandBy(FVector(-20.0f, -20.0f, 0.0f)), FColor::Orange, InTraversalDesc.bDebugDrawIntoForeground ? SDPG_Foreground : SDPG_World);
			}
#endif
		}
	}
}

bool FWaterQuadTree::QueryInterpolatedTileBaseHeightAtLocation(const FVector2D& InWorldLocationXY, float& OutHeight) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWaterQuadTree::QueryInterpolatedTileBaseHeightAtLocation);

	// Figure out what 4 samples to take
	// Sample point grid is aligned with center of leaf node tiles. So offset the grid negative half a leaf tile
	const FVector2D SampleGridWorldPosition(GetTileRegion().Min - FVector2D(GetLeafSize() * 0.5f));
	const FVector2D CornerSampleGridPosition(InWorldLocationXY - SampleGridWorldPosition);
	const FVector2D NormalizedGridPosition(CornerSampleGridPosition / GetLeafSize());
	const FVector2D CornerSampleWorldPosition00 = FVector2D(FMath::Floor(NormalizedGridPosition.X), FMath::Floor(NormalizedGridPosition.Y)) * GetLeafSize() + SampleGridWorldPosition;
	
	// 4 world positions to use for sampling
	FVector2D CornerSampleWorldPositions[] =
	{
		CornerSampleWorldPosition00 + FVector2D(0.0f, 0.0f),
		CornerSampleWorldPosition00 + FVector2D(GetLeafSize(), 0.0f),
		CornerSampleWorldPosition00 + FVector2D(0.0f, GetLeafSize()),
		CornerSampleWorldPosition00 + FVector2D(GetLeafSize(), GetLeafSize())
	};

	// Sample 4 locations
	float HeightSamples[4] = {0.0f, 0.0f, 0.0f, 0.0f};
	int32 NumValidSamples = 0;
	for(int32 i = 0; i < 4; i++)
	{
		if (QueryTileBaseHeightAtLocation(CornerSampleWorldPositions[i], HeightSamples[i]))
		{
			NumValidSamples++;
		}
	}

	// Return bilinear interpolated value
	OutHeight = FMath::BiLerp(HeightSamples[0], HeightSamples[1], HeightSamples[2], HeightSamples[3], FMath::Frac(NormalizedGridPosition.X), FMath::Frac(NormalizedGridPosition.Y));

	return NumValidSamples == 4;
}

bool FWaterQuadTree::QueryTileBaseHeightAtLocation(const FVector2D& InWorldLocationXY, float& OutWorldHeight) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWaterQuadTree::QueryTileBaseHeightAtLocation);
	if (GetNodeCount() > 0)
	{
		check(bIsReadOnly);
		return NodeData.Nodes[0].QueryBaseHeightAtLocation(NodeData, InWorldLocationXY, OutWorldHeight);
	}
	
	OutWorldHeight = 0.0f;
	return false;
}

bool FWaterQuadTree::QueryTileBoundsAtLocation(const FVector2D& InWorldLocationXY, FBox& OutWorldBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWaterQuadTree::QueryTileBoundsAtLocation);
	if (GetNodeCount() > 0)
	{
		check(bIsReadOnly);
		return NodeData.Nodes[0].QueryBoundsAtLocation(NodeData, InWorldLocationXY, OutWorldBounds);
	}

	OutWorldBounds = FBox(ForceInit);
	return false;
}

#if WITH_WATER_SELECTION_SUPPORT
void FWaterQuadTree::GatherHitProxies(TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) const
{
	for(const FWaterBodyRenderData& WaterBodyRenderData : NodeData.WaterBodyRenderData)
	{
		OutHitProxies.Add(WaterBodyRenderData.HitProxy);
	}
}
#endif //WITH_WATER_SELECTION_SUPPORT

TArray<FBoxSphereBounds> FWaterQuadTree::ComputeNodeBounds(int32 MaxNumBounds, float OcclusionCullExpandBoundsAmountXY, bool bIncludeFarMeshTiles, int32* OutFarMeshOffset) const
{
	const int32 NumNodes = NodeData.Nodes.Num();
	const int32 ClampedNumNodes = (MaxNumBounds > 0) ? FMath::Min(MaxNumBounds, NumNodes) : NumNodes;
	const int32 NumFarMeshTiles = FarMeshData.InstanceData.Num();
	const int32 NumToReserve = ClampedNumNodes + (bIncludeFarMeshTiles ? NumFarMeshTiles : 0);

	TArray<FBoxSphereBounds> Result;
	Result.Reserve(NumToReserve);

	// Quadtree tiles
	for (int32 i = 0; i < ClampedNumNodes; ++i)
	{
		// Expand bounds by given amount, only in XY to reduce occlusion 
		const FBox Bounds = NodeData.Nodes[BreadthFirstOrder[i]].Bounds;
		Result.Add(Bounds.ExpandBy(FVector(OcclusionCullExpandBoundsAmountXY, OcclusionCullExpandBoundsAmountXY, 0.0)));
	}

	*OutFarMeshOffset = Result.Num();

	// Far mesh tiles
	if (bIncludeFarMeshTiles)
	{
		for (int32 i = 0; i < NumFarMeshTiles; ++i)
		{
			const FFarMeshData::FFarMeshInstanceData& InstanceData = FarMeshData.InstanceData[i];
			const FVector Extent = FVector(InstanceData.Scale.X * 0.5, InstanceData.Scale.Y * 0.5, 1.0);
			const FBox Bounds = FBox(InstanceData.WorldPosition - Extent, InstanceData.WorldPosition + Extent).ExpandBy(FVector(OcclusionCullExpandBoundsAmountXY, OcclusionCullExpandBoundsAmountXY, 0.0));
			Result.Add(Bounds);
		}
	}

	return Result;
}

/** Split a 2D polygon with a 2D line. Return both polygons */
static void SplitPolyWithLine(const TArray<FVector2D>& InPoly, const FVector2D& LinePoint, const FVector2D& LineNormal, TArray<FVector2D>& OutPoly0, TArray<FVector2D>& OutPoly1)
{
	// Make 2D line.
	FVector2D Normal2D = LineNormal;
	FVector2D Base2D = LinePoint;

	int32 NumPVerts = InPoly.Num();

	// Calculate distance of verts from clipping line
	TArray<double> PlaneDist;
	PlaneDist.AddZeroed(NumPVerts);
	for (int32 i = 0; i < NumPVerts; i++)
	{
		const FVector2D PointDiff = InPoly[i] - LinePoint;
		PlaneDist[i] = LineNormal.X > 0.0 ? PointDiff.X : PointDiff.Y;
	}

	for (int32 ThisVert = 0; ThisVert < NumPVerts; ThisVert++)
	{
		// Vert is on positive side of line, add to Poly0
		if (PlaneDist[ThisVert] > 0.0)
		{
			OutPoly0.Add(InPoly[ThisVert]);
		}
		else
		{
			OutPoly1.Add(InPoly[ThisVert]);
		}

		// If start and next vert are on opposite sides, add intersection
		int32 NextVert = (ThisVert + 1) % NumPVerts;

		if (PlaneDist[ThisVert] * PlaneDist[NextVert] < 0.0)
		{
			// Find distance along edge that plane is
			double Alpha = -PlaneDist[ThisVert] / (PlaneDist[NextVert] - PlaneDist[ThisVert]);
			FVector2D NewVertPos = FMath::Lerp(InPoly[ThisVert], InPoly[NextVert], Alpha);

			// Save vert
			OutPoly0.Add(NewVertPos);
			OutPoly1.Add(NewVertPos);
		}
	}
}

static double CalcPoly2DArea(const TArray<FVector2D>& InPoly)
{
	double ResultArea = 0.0;
	for (int i = 0, j = InPoly.Num() - 1; i < InPoly.Num(); j = i++)
	{
		const FVector2D& Vert0 = InPoly[i];
		const FVector2D& Vert1 = InPoly[j];
		ResultArea += Vert1.X * Vert0.Y - Vert0.X * Vert1.Y;
	}
	return ResultArea * 0.5;
}

void FWaterQuadTree::AddOceanRecursive(const TArray<FVector2D>& InPoly, const FBox2D& InBox, const FVector2D& InZBounds, bool HSplit, int32 InDepth, uint32 InWaterBodyIndex)
{
	// Some value to guard against false positives, based on the max area
	const double BoxArea = InBox.GetArea();
	const double AreaEpsilon = BoxArea * 0.0001;
	const FVector2D LeafSizeShrink(LeafSize * 0.25, LeafSize * 0.25);

	//We've reached the bottom, figure out if this poly is filling out its box
	if (InDepth == 0)
	{
		// If the area is smaller than the box, then there is room for water and we add this as a water tile
		if (CalcPoly2DArea(InPoly) < (BoxArea - AreaEpsilon))
		{
			FBox TileBounds(FVector(InBox.Min + LeafSizeShrink, InZBounds.X), FVector(InBox.Max - LeafSizeShrink, InZBounds.Y));
			AddWaterTilesInsideBounds(TileBounds, InWaterBodyIndex);
		}

		return;
	}

	// The two resulting polys from the split
	TArray<FVector2D> Poly0;
	TArray<FVector2D> Poly1;

	// Line point, alway middle of box
	const FVector2D LinePoint = InBox.GetCenter();

	// If horizontal split, create horizontal line
	const FVector2D LineNormal = HSplit ? FVector2D(0.0f, 1.0f) : FVector2D(1.0f, 0.0f);

	// Split
	SplitPolyWithLine(InPoly, LinePoint, LineNormal, Poly0, Poly1);

	// Recurse split the two new polys if they have any significant land area
	// Poly0 is the positive box (on the positive side of the line normal)
	const FBox2D HalfBox0(InBox.Min + InBox.GetExtent() * LineNormal, InBox.Max);
	if (CalcPoly2DArea(Poly0) > AreaEpsilon)
	{
		AddOceanRecursive(Poly0, HalfBox0, InZBounds, !HSplit, InDepth - 1, InWaterBodyIndex);
	}
	else
	{
		// No more verts in this half box, mark as water
		FBox TileBounds(FVector(HalfBox0.Min + LeafSizeShrink, InZBounds.X), FVector(HalfBox0.Max - LeafSizeShrink, InZBounds.Y));
		AddWaterTilesInsideBounds(TileBounds, InWaterBodyIndex);
	}

	// Poly1 is the negative box (on the negative side of the line normal)
	const FBox2D HalfBox1(InBox.Min, InBox.Max - InBox.GetExtent() * LineNormal);
	if (CalcPoly2DArea(Poly1) > AreaEpsilon)
	{
		AddOceanRecursive(Poly1, HalfBox1, InZBounds, !HSplit, InDepth - 1, InWaterBodyIndex);
	}
	else
	{
		// No more verts in this half box, mark as water
		FBox TileBounds(FVector(HalfBox1.Min + LeafSizeShrink, InZBounds.X), FVector(HalfBox1.Max - LeafSizeShrink, InZBounds.Y));
		AddWaterTilesInsideBounds(TileBounds, InWaterBodyIndex);
	}
}

void FWaterQuadTree::AddLakeRecursive(const TArray<FVector2D>& InPoly, const FBox2D& InBox, const FVector2D& InZBounds, bool HSplit, int32 InDepth, uint32 InWaterBodyIndex)
{
	// Some value to guard against false positives, based on the max area
	const double BoxArea = InBox.GetArea();
	const double AreaEpsilon = BoxArea * 0.0001;
	const FVector2D LeafSizeShrink(LeafSize * 0.01, LeafSize * 0.01);

	//We've reached the bottom, figure out if this poly is filling out its box
	if (InDepth == 0)
	{
		// If there is a valid lake poly area, we want to generate water for that
		if (CalcPoly2DArea(InPoly) > 0)
		{
			FBox TileBounds(FVector(InBox.Min + LeafSizeShrink, InZBounds.X), FVector(InBox.Max - LeafSizeShrink, InZBounds.Y));
			AddWaterTilesInsideBounds(TileBounds, InWaterBodyIndex);
		}

		return;
	}

	// The two resulting polys from the split
	TArray<FVector2D> Poly0;
	TArray<FVector2D> Poly1;

	// Line point, alway middle of box
	const FVector2D LinePoint = InBox.GetCenter();

	// If horizontal split, create horizontal line
	const FVector2D LineNormal = HSplit ? FVector2D(0.0f, 1.0f) : FVector2D(1.0f, 0.0f);

	// Split
	SplitPolyWithLine(InPoly, LinePoint, LineNormal, Poly0, Poly1);

	// Recurse split the two new polys if they have any significant lake poly area
	// Poly0 is the positive box (on the positive side of the line normal)
	const FBox2D HalfBox0(InBox.Min + InBox.GetExtent() * LineNormal, InBox.Max);
	if (CalcPoly2DArea(Poly0) > (BoxArea * 0.5 - AreaEpsilon))
	{
		// This halfbox is filled with lake poly, mark as water
		FBox TileBounds(FVector(HalfBox0.Min + LeafSizeShrink, InZBounds.X), FVector(HalfBox0.Max - LeafSizeShrink, InZBounds.Y));
		AddWaterTilesInsideBounds(TileBounds, InWaterBodyIndex);
	}
	else if (CalcPoly2DArea(Poly0) > 0)
	{
		AddLakeRecursive(Poly0, HalfBox0, InZBounds, !HSplit, InDepth - 1, InWaterBodyIndex);
	}

	// Poly1 is the negative box (on the negative side of the line normal)
	const FBox2D HalfBox1(InBox.Min, InBox.Max - InBox.GetExtent() * LineNormal);
	if (CalcPoly2DArea(Poly1) > (BoxArea * 0.5 - AreaEpsilon))
	{
		// This halfbox is filled with lake poly, mark as water
		FBox TileBounds(FVector(HalfBox1.Min + LeafSizeShrink, InZBounds.X), FVector(HalfBox1.Max - LeafSizeShrink, InZBounds.Y));
		AddWaterTilesInsideBounds(TileBounds, InWaterBodyIndex);
	}
	else if (CalcPoly2DArea(Poly1) > 0)
	{
		AddLakeRecursive(Poly1, HalfBox1, InZBounds, !HSplit, InDepth - 1, InWaterBodyIndex);
	}
}
