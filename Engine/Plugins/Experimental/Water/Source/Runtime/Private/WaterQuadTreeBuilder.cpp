// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterQuadTreeBuilder.h"
#include "WaterQuadTree.h"
#include "WaterBodyTypes.h"
#include "WaterModule.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "MaterialDomain.h"

void FWaterQuadTreeBuilder::Init(const FBox2D& InWaterZoneBounds2D, const FIntPoint& InExtentInTiles, float InTileSize, const UMaterialInterface* InFarDistanceMaterial, float InFarDistanceMeshExtent, double InDefaultFarDistanceMeshHeight, bool bInUseFarMeshWithoutOcean, bool bInIsGPUQuadTree)
{
	WaterBodies.Reset();
	WaterZoneBounds2D = InWaterZoneBounds2D;
	ExtentInTiles = InExtentInTiles;
	TileSize = InTileSize;
	FarDistanceMaterial = InFarDistanceMaterial;
	FarDistanceMeshExtent = InFarDistanceMeshExtent;
	DefaultFarDistanceMeshHeight = InDefaultFarDistanceMeshHeight;
	bUseFarMeshWithoutOcean = bInUseFarMeshWithoutOcean;
	bIsGPUQuadTree = bInIsGPUQuadTree;

	const int32 MaxDim = (int32)FMath::Max(InExtentInTiles.X * 2, InExtentInTiles.Y * 2);
	const float RootDim = (float)FMath::RoundUpToPowerOfTwo(MaxDim);
	TreeDepth = (int32)FMath::Log2(RootDim);
}

void FWaterQuadTreeBuilder::AddWaterBody(const FWaterBody& WaterBody)
{
	WaterBodies.Add(WaterBody);
}

bool FWaterQuadTreeBuilder::BuildWaterQuadTree(FWaterQuadTree& WaterQuadTree, const FVector2D& GridPosition) const
{
	const FVector2D WorldExtent = FVector2D(TileSize * ExtentInTiles.X, TileSize * ExtentInTiles.Y);
	const FBox2D WaterWorldBox = FBox2D(-WorldExtent + GridPosition, WorldExtent + GridPosition);

	// If the dynamic bounds is outside the full bounds of the water mesh, we shouldn't regenerate the quadtree
	if (!(WaterWorldBox.GetArea() > 0.f))
	{
		return false;
	}

	// This resets the tree to an initial state, ready for node insertion
	WaterQuadTree.InitTree(WaterWorldBox, TileSize, ExtentInTiles, bIsGPUQuadTree);

	// Will be updated with the ocean min bound, to be used to place the far mesh just under the ocean to avoid seams
	double FarMeshHeight = DefaultFarDistanceMeshHeight;
	// Only use a far mesh when there is an ocean in the zone.
	bool bHasOcean = false;
	
	// Min and max user defined priority range. (Input also clamped on OverlapMaterialPriority in AWaterBody)
	constexpr int32 MinWaterBodyPriority = -8192;
	constexpr int32 MaxWaterBodyPriority = 8191;
	constexpr int32 GPUQuadTreeMaxNumPriorities = 8;

	// The GPU quadtree only supports 8 different priority values, so we need to remap priorities into that space.
	// Fortunately, rivers and non-river water bodies are rendered into their own "priority space", so we don't need to worry about
	// moving river priorities into their own range.
	TArray<int16> SortedPriorities;
	if (bIsGPUQuadTree)
	{
		for (const FWaterBody& WaterBody : WaterBodies)
		{
			// Don't process water bodies which have their spline outside of this water mesh
			const FBox WaterBodyBounds = WaterBody.Bounds.GetBox();
			if (!WaterBodyBounds.IntersectXY(FBox(FVector(WaterWorldBox.Min, 0.0), FVector(WaterWorldBox.Max, 0.0))))
			{
				continue;
			}

			const int16 Priority = static_cast<int16>(FMath::Clamp(WaterBody.OverlapMaterialPriority, MinWaterBodyPriority, MaxWaterBodyPriority));
			SortedPriorities.AddUnique(Priority);
		}

		SortedPriorities.Sort();

		if (SortedPriorities.Num() > GPUQuadTreeMaxNumPriorities)
		{
			UE_LOGF(LogWater, Warning, "WaterZone has more unique water body priorities (%i) than can be supported with GPU driven water quadtree rendering (%i)!", SortedPriorities.Num(), GPUQuadTreeMaxNumPriorities);
		}
	}

	for (const FWaterBody& WaterBody : WaterBodies)
	{
		// Don't process water bodies which have their spline outside of this water mesh
		const FBox WaterBodyBounds = WaterBody.Bounds.GetBox();
		if (!WaterBodyBounds.IntersectXY(FBox(FVector(WaterWorldBox.Min, 0.0), FVector(WaterWorldBox.Max, 0.0))))
		{
			continue;
		}

		FWaterBodyRenderData RenderData;
		RenderData.Material = WaterBody.Material ? WaterBody.Material->GetRenderProxy() : nullptr;
		RenderData.RiverToLakeMaterial = WaterBody.RiverToLakeMaterial ? WaterBody.RiverToLakeMaterial->GetRenderProxy() : nullptr;
		RenderData.RiverToOceanMaterial = WaterBody.RiverToOceanMaterial ? WaterBody.RiverToOceanMaterial->GetRenderProxy() : nullptr;
		RenderData.Priority = static_cast<int16>(FMath::Clamp(WaterBody.OverlapMaterialPriority, MinWaterBodyPriority, MaxWaterBodyPriority));
		RenderData.WaterBodyIndex = static_cast<int16>(WaterBody.WaterBodyIndex);
		RenderData.SurfaceBaseHeight = WaterBody.SurfaceBaseHeight;
		RenderData.MaxWaveHeight = WaterBody.MaxWaveHeight;
		RenderData.BoundsMinZ = WaterBody.Bounds.GetBox().Min.Z;
		RenderData.BoundsMaxZ = WaterBody.Bounds.GetBox().Max.Z;
		RenderData.WaterBodyType = static_cast<int8>(WaterBody.Type);
#if WITH_WATER_SELECTION_SUPPORT
		RenderData.HitProxy = WaterBody.HitProxy;
		RenderData.bWaterBodySelected = WaterBody.bWaterBodySelected;
#endif // WITH_WATER_SELECTION_SUPPORT

		if (RenderData.RiverToLakeMaterial || RenderData.RiverToOceanMaterial)
		{
			// Move rivers up to it's own priority space, so that they always have precedence if they have transitions and that they only compare agains other rivers with transitions
			RenderData.Priority += (MaxWaterBodyPriority - MinWaterBodyPriority) + 1;
		}

		const uint32 WaterBodyRenderDataIndex = WaterQuadTree.AddWaterBodyRenderData(RenderData);

		if (bIsGPUQuadTree)
		{
			// On the GPU path, we only submit FWaterBodyQuadTreeRasterInfo for the water quadtree to be rasterized on the GPU. In particular, we need the static mesh, a transform and a priority/WaterBodyRenderData index.
			const int16 ClampedPriority = static_cast<int16>(FMath::Clamp(WaterBody.OverlapMaterialPriority, MinWaterBodyPriority, MaxWaterBodyPriority));

			FWaterBodyQuadTreeRasterInfo RasterInfo;
			RasterInfo.LocalToWorld = WaterBody.LocalToWorld;
			RasterInfo.RenderData = WaterBody.StaticMeshRenderData;
			RasterInfo.WaterBodyRenderDataIndex = WaterBodyRenderDataIndex;
			RasterInfo.Priority = FMath::Clamp(SortedPriorities.IndexOfByKey(ClampedPriority), 0, GPUQuadTreeMaxNumPriorities - 1);
			RasterInfo.bIsRiver = WaterBody.Type == EWaterBodyType::River;

			WaterQuadTree.AddWaterBodyRasterInfo(RasterInfo);

			if (WaterBody.Type == EWaterBodyType::Ocean)
			{
				// Place far mesh height just below the ocean level
				FarMeshHeight = RenderData.SurfaceBaseHeight - RenderData.MaxWaveHeight;
				bHasOcean = true;
			}
		}
		else
		{
			switch (WaterBody.Type)
			{
			case EWaterBodyType::River:
			{
				for (const FBox& Box : WaterBody.RiverBoxes)
				{
					WaterQuadTree.AddWaterTilesInsideBounds(Box, WaterBodyRenderDataIndex);
				}
				break;
			}
			case EWaterBodyType::Lake:
			{
				for (const TArray<FVector2D>& Polygon : WaterBody.PolygonBatches)
				{
					WaterQuadTree.AddLake(Polygon, WaterBody.PolygonBounds, WaterBodyRenderDataIndex);
				}
				break;
			}
			case EWaterBodyType::Ocean:
			{
				check(WaterBody.PolygonBatches.Num() == 1);
				WaterQuadTree.AddOcean(WaterBody.PolygonBatches[0], WaterBody.PolygonBounds, WaterBodyRenderDataIndex);

				// Place far mesh height just below the ocean level
				FarMeshHeight = RenderData.SurfaceBaseHeight - WaterBody.MaxWaveHeight;
				bHasOcean = true;

				break;
			}
			case EWaterBodyType::Transition:
				// Transitions dont require rendering
				break;
			default:
				ensureMsgf(false, TEXT("This water body type is not implemented and will not produce any water tiles. "));
			}
		}
	}

	// Build the far distance mesh instances if needed
	if (FarDistanceMaterial && (bHasOcean || bUseFarMeshWithoutOcean) && (FarDistanceMeshExtent > 0.0f))
	{
		// Far Mesh should stitch to the edge of the water zone
		const FBox2D FarMeshBounds = WaterZoneBounds2D;
		WaterQuadTree.AddFarMesh(FarDistanceMaterial->GetRenderProxy(), FarMeshBounds, FarDistanceMeshExtent, FarMeshHeight);
	}

	WaterQuadTree.Unlock(true);

	WaterQuadTree.BuildMaterialIndices();

	return true;
}

#if WITH_WATER_SELECTION_SUPPORT
void FWaterQuadTreeBuilder::GatherHitProxies(TArray<TRefCountPtr<HHitProxy>>& OutHitProxies) const
{
	for (const FWaterBody& WaterBody : WaterBodies)
	{
		OutHitProxies.Add(WaterBody.HitProxy);
	}
}
#endif // WITH_WATER_SELECTION_SUPPORT

bool FWaterQuadTreeBuilder::IsGPUQuadTree() const
{
	return bIsGPUQuadTree;
}

float FWaterQuadTreeBuilder::GetLeafSize() const
{
	return TileSize;
}

int32 FWaterQuadTreeBuilder::GetMaxLeafCount() const
{
	return ExtentInTiles.X * ExtentInTiles.Y * 4;
}

int32 FWaterQuadTreeBuilder::GetTreeDepth() const
{
	return TreeDepth;
}

FIntPoint FWaterQuadTreeBuilder::GetResolution() const
{
	return ExtentInTiles * 2;
}
