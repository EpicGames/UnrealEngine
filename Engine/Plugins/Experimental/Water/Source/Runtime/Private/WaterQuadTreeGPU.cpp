// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterQuadTreeGPU.h"

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "RenderUtils.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHIStaticStates.h"
#include "RHIFeatureLevel.h"
#include "PipelineStateCache.h"
#include "PixelShaderUtils.h"
#include "SystemTextures.h"
#include "ShaderPrintParameters.h"
#include "SceneRendering.h"
#include "Math/Halton.h"
#include "HZB.h"

DECLARE_GPU_STAT(FWaterQuadTreeGPU_Init);
DECLARE_GPU_STAT(FWaterQuadTreeGPU_Traverse);

int32 GWaterQuadTreeParallelPrefixSum = 1;
static FAutoConsoleVariableRef CVarWaterQuadTreeParallelPrefixSum(
	TEXT("r.Water.WaterMesh.GPUQuadTree.ParallelPrefixSum"),
	GWaterQuadTreeParallelPrefixSum,
	TEXT("Enables using a parallel prefix sum algorithm for the water quadtree indirect draw call pipeline."),
	ECVF_RenderThreadSafe);

float GWaterQuadTreeZBoundsPadding = 200.0f;
static FAutoConsoleVariableRef CVarWaterQuadTreeZBoundsPadding(
	TEXT("r.Water.WaterMesh.GPUQuadTree.ZBoundsPadding"),
	GWaterQuadTreeZBoundsPadding,
	TEXT("Amount of padding to apply to the Z bounds of each quadtree node."
		" Necessary for sloped rivers, whose complete Z bounds can be underestimated due to using non-conservative rasterization to build the quadtree."),
	ECVF_RenderThreadSafe);

int32 GWaterQuadTreeOcclusionCulling = 3;
static FAutoConsoleVariableRef CVarWaterQuadTreeOcclusionCulling(
	TEXT("r.Water.WaterMesh.GPUQuadTree.OcclusionCulling"),
	GWaterQuadTreeOcclusionCulling,
	TEXT("0: Disabled, 1: HZB Occlusion Queries, 2: Pixel Precise Raster Queries, 3: HZB + Pixel Precise Raster Queries"),
	ECVF_RenderThreadSafe);

class FWaterQuadTreeVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterQuadTreeVS);
	SHADER_USE_PARAMETER_STRUCT(FWaterQuadTreeVS, FGlobalShader);

	class FRasterMode : SHADER_PERMUTATION_SPARSE_INT("RASTER_MODE", 0, 1, 2);
	using FPermutationDomain = TShaderPermutationDomain<FRasterMode>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FMatrix44f, Transform)
		SHADER_PARAMETER(FVector2f, JitterScale)
		SHADER_PARAMETER(FVector2f, HalfPixelSize)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};
IMPLEMENT_GLOBAL_SHADER(FWaterQuadTreeVS, "/Plugin/Water/Private/WaterQuadTreeVertexShader.usf", "Main", SF_Vertex);

class FWaterQuadTreePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterQuadTreePS);
	SHADER_USE_PARAMETER_STRUCT(FWaterQuadTreePS, FGlobalShader);

	class FClipConservativeTriangle : SHADER_PERMUTATION_BOOL("CLIP_CONSERVATIVE_TRIANGLE");
	using FPermutationDomain = TShaderPermutationDomain<FClipConservativeTriangle>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, WaterBodyRenderDataIndex)
		SHADER_PARAMETER(int32, Priority)
		SHADER_PARAMETER(float, WaterBodyMinZ)
		SHADER_PARAMETER(float, WaterBodyMaxZ)
		SHADER_PARAMETER(float, MaxWaveHeight)
		SHADER_PARAMETER(uint32, bIsRiver)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};
IMPLEMENT_GLOBAL_SHADER(FWaterQuadTreePS, "/Plugin/Water/Private/WaterQuadTreePixelShader.usf", "Main", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FWaterQuadTreeParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FWaterQuadTreeVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FWaterQuadTreePS::FParameters, PS)
END_SHADER_PARAMETER_STRUCT()

class FWaterQuadTreeMergePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterQuadTreeMergePS);
	SHADER_USE_PARAMETER_STRUCT(FWaterQuadTreeMergePS, FGlobalShader);

	class FNumMSAASamples : SHADER_PERMUTATION_SPARSE_INT("NUM_MSAA_SAMPLES", 1, 2, 4, 8);

	using FPermutationDomain = TShaderPermutationDomain<FNumMSAASamples>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMS, WaterBodyRasterTextureMS)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMS, ZBoundsRasterTextureMS)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WaterBodyRasterTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ZBoundsRasterTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, WaterBodyRenderData)
		SHADER_PARAMETER(int32, SuperSamplingFactor)
		SHADER_PARAMETER(float, RcpCaptureDepthRange)
		SHADER_PARAMETER(float, ZBoundsPadding)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};
IMPLEMENT_GLOBAL_SHADER(FWaterQuadTreeMergePS, "/Plugin/Water/Private/WaterQuadTreeMerge.usf", "Main", SF_Pixel);

class FWaterQuadTreeBuildPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterQuadTreeBuildPS);
	SHADER_USE_PARAMETER_STRUCT(FWaterQuadTreeBuildPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, QuadTreeTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, WaterZBoundsTexture)
		SHADER_PARAMETER(int32, InputMipLevelIndex)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};
IMPLEMENT_GLOBAL_SHADER(FWaterQuadTreeBuildPS, "/Plugin/Water/Private/WaterQuadTreeBuild.usf", "Main", SF_Pixel);

class FWaterQuadTreeInitializeIndirectArgsCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterQuadTreeInitializeIndirectArgsCS);
	SHADER_USE_PARAMETER_STRUCT(FWaterQuadTreeInitializeIndirectArgsCS, FGlobalShader);

	class FPreciseOcclusionQueries : SHADER_PERMUTATION_BOOL("PRECISE_OCCLUSION_QUERIES");
	using FPermutationDomain = TShaderPermutationDomain<FPreciseOcclusionQueries>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OcclusionQueryArgs)
		SHADER_PARAMETER(uint32, NumDrawBuckets)
		SHADER_PARAMETER(uint32, NumViews)
		SHADER_PARAMETER(uint32, NumQuads)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INITIALIZE_INDIRECT_ARGS"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FWaterQuadTreeInitializeIndirectArgsCS, "/Plugin/Water/Private/WaterQuadTreeDraws.usf", "MainCS", SF_Compute);

class FWaterQuadTreeClearPerViewBuffersCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterQuadTreeClearPerViewBuffersCS);
	SHADER_USE_PARAMETER_STRUCT(FWaterQuadTreeClearPerViewBuffersCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, BucketCounts)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, PackedNodes)
		SHADER_PARAMETER(uint32, NumDrawBuckets)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("CLEAR_PER_VIEW_BUFFERS"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FWaterQuadTreeClearPerViewBuffersCS, "/Plugin/Water/Private/WaterQuadTreeDraws.usf", "MainCS", SF_Compute);


class FWaterQuadTreeTraverseCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterQuadTreeTraverseCS);
	SHADER_USE_PARAMETER_STRUCT(FWaterQuadTreeTraverseCS, FGlobalShader);

	class FPreciseOcclusionQueries : SHADER_PERMUTATION_BOOL("PRECISE_OCCLUSION_QUERIES");
	using FPermutationDomain = TShaderPermutationDomain<FPreciseOcclusionQueries>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, PackedNodes)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OcclusionQueryBoxes)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OcclusionVisibility)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OcclusionQueryArgs)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, QuadTreeTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WaterZBoundsTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, WaterBodyRenderData)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
		SHADER_PARAMETER(FVector4f, CullingBoundsAABB)
		SHADER_PARAMETER(FVector3f, QuadTreePosition)
		SHADER_PARAMETER(FVector3f, ObserverPosition)
		SHADER_PARAMETER(uint32, QuadTreeResolutionX)
		SHADER_PARAMETER(uint32, QuadTreeResolutionY)
		SHADER_PARAMETER(uint32, ViewIndex)
		SHADER_PARAMETER(float, LeafSize)
		SHADER_PARAMETER(float, LODScale)
		SHADER_PARAMETER(float, CaptureDepthRange)
		SHADER_PARAMETER(int32, ForceCollapseDensityLevel)
		SHADER_PARAMETER(uint32, NumLODs)
		SHADER_PARAMETER(uint32, NumDispatchedThreads)
		SHADER_PARAMETER(uint32, bHZBOcclusionCullingEnabled)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("QUAD_TREE_TRAVERSE"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FWaterQuadTreeTraverseCS, "/Plugin/Water/Private/WaterQuadTreeDraws.usf", "MainCS", SF_Compute);

class FWaterQuadTreeOcclusionQueryVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterQuadTreeOcclusionQueryVS);
	SHADER_USE_PARAMETER_STRUCT(FWaterQuadTreeOcclusionQueryVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, OcclusionQueryBoxes)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("OCCLUSION_QUERY_RASTER_VS"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FWaterQuadTreeOcclusionQueryVS, "/Plugin/Water/Private/WaterQuadTreeDraws.usf", "MainVS", SF_Vertex);

class FWaterQuadTreeOcclusionQueryPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterQuadTreeOcclusionQueryPS);
	SHADER_USE_PARAMETER_STRUCT(FWaterQuadTreeOcclusionQueryPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, Visibility)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("OCCLUSION_QUERY_RASTER_PS"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FWaterQuadTreeOcclusionQueryPS, "/Plugin/Water/Private/WaterQuadTreeDraws.usf", "MainPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FWaterQuadTreeOcclusionQueryParameters, )
	RDG_BUFFER_ACCESS(IndirectDrawArgsBuffer, ERHIAccess::IndirectArgs)
	SHADER_PARAMETER_STRUCT_INCLUDE(FWaterQuadTreeOcclusionQueryVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FWaterQuadTreeOcclusionQueryPS::FParameters, PS)
END_SHADER_PARAMETER_STRUCT()

class FWaterQuadTreeBucketCountsCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterQuadTreeBucketCountsCS);
	SHADER_USE_PARAMETER_STRUCT(FWaterQuadTreeBucketCountsCS, FGlobalShader);

	class FPreciseOcclusionQueries : SHADER_PERMUTATION_BOOL("PRECISE_OCCLUSION_QUERIES");
	using FPermutationDomain = TShaderPermutationDomain<FPreciseOcclusionQueries>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, BucketCounts)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, QuadTreeTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, WaterBodyRenderData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PackedNodes)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, OcclusionResults)
		SHADER_PARAMETER(uint32, NumDispatchedThreads)
		SHADER_PARAMETER(uint32, NumDensities)
		SHADER_PARAMETER(uint32, NumQuadsLOD0)
		SHADER_PARAMETER(uint32, NumQuadsPerDraw)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPUTE_BUCKET_COUNTS"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FWaterQuadTreeBucketCountsCS, "/Plugin/Water/Private/WaterQuadTreeDraws.usf", "MainCS", SF_Compute);

class FWaterQuadTreeBucketPrefixSumCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterQuadTreeBucketPrefixSumCS);
	SHADER_USE_PARAMETER_STRUCT(FWaterQuadTreeBucketPrefixSumCS, FGlobalShader);

	class FParallelPrefixSum : SHADER_PERMUTATION_BOOL("PARALLEL_PREFIX_SUM");
	using FPermutationDomain = TShaderPermutationDomain<FParallelPrefixSum>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, BucketPrefixSums)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, BucketCounts)
		SHADER_PARAMETER(uint32, NumBuckets)
		SHADER_PARAMETER(uint32, OutputOffset)
		SHADER_PARAMETER(uint32, bWriteTotalSumAtBufferEnd)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPUTE_BUCKET_PREFIX_SUM"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FWaterQuadTreeBucketPrefixSumCS, "/Plugin/Water/Private/WaterQuadTreeDraws.usf", "MainCS", SF_Compute);

class FWaterQuadTreeGenerateInstanceDataCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterQuadTreeGenerateInstanceDataCS);
	SHADER_USE_PARAMETER_STRUCT(FWaterQuadTreeGenerateInstanceDataCS, FGlobalShader);

	class FPreciseOcclusionQueries : SHADER_PERMUTATION_BOOL("PRECISE_OCCLUSION_QUERIES");
	using FPermutationDomain = TShaderPermutationDomain<FPreciseOcclusionQueries>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, InstanceData0)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, InstanceData1)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, InstanceData2)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, InstanceData3)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, QuadTreeTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WaterZBoundsTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, WaterBodyRenderData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PackedNodes)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InstanceDataOffsets)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, OcclusionResults)
		SHADER_PARAMETER(FVector3f, QuadTreePosition)
		SHADER_PARAMETER(FVector3f, ObserverPosition)
		SHADER_PARAMETER(uint32, QuadTreeResolutionX)
		SHADER_PARAMETER(uint32, QuadTreeResolutionY)
		SHADER_PARAMETER(uint32, NumDensities)
		SHADER_PARAMETER(uint32, NumMaterials)
		SHADER_PARAMETER(uint32, NumDispatchedThreads)
		SHADER_PARAMETER(uint32, BucketIndexOffset)
		SHADER_PARAMETER(uint32, NumLODs)
		SHADER_PARAMETER(uint32, NumQuadsLOD0)
		SHADER_PARAMETER(uint32, NumQuadsPerDraw)
		SHADER_PARAMETER(float, LeafSize)
		SHADER_PARAMETER(float, LODScale)
		SHADER_PARAMETER(float, CaptureDepthRange)
		SHADER_PARAMETER(uint32, StereoPassInstanceFactor)
		SHADER_PARAMETER(uint32, bWithWaterSelectionSupport)
		SHADER_PARAMETER(uint32, bLODMorphingEnabled)
		SHADER_PARAMETER(uint32, bInstancedStereoRendering)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("GENERATE_INSTANCE_DATA"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FWaterQuadTreeGenerateInstanceDataCS, "/Plugin/Water/Private/WaterQuadTreeDraws.usf", "MainCS", SF_Compute);

class FWaterQuadTreeDebugCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterQuadTreeDebugCS);
	SHADER_USE_PARAMETER_STRUCT(FWaterQuadTreeDebugCS, FGlobalShader);

	class FPreciseOcclusionQueries : SHADER_PERMUTATION_BOOL("PRECISE_OCCLUSION_QUERIES");
	using FPermutationDomain = TShaderPermutationDomain<FPreciseOcclusionQueries>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, QuadTreeTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WaterZBoundsTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, WaterBodyRenderData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PackedNodes)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, OcclusionResults)
		SHADER_PARAMETER(FVector3f, QuadTreePosition)
		SHADER_PARAMETER(uint32, NumDispatchedThreads)
		SHADER_PARAMETER(float, LeafSize)
		SHADER_PARAMETER(float, CaptureDepthRange)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShaderPrint::IsSupported(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("DEBUG_SHOW_TILES"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FWaterQuadTreeDebugCS, "/Plugin/Water/Private/WaterQuadTreeDraws.usf", "MainCS", SF_Compute);


class FJitterOffsetVertexBuffer : public FVertexBuffer
{
public:
	static constexpr int32 NumHaltonSamples = 16;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		// Create the texture RHI.
		const int32 NumMSAASamples = 2 + 4 + 8 + 16;
		const int32 NumFloats = 2 * (NumHaltonSamples + NumMSAASamples);

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<float>(TEXT("JitterOffsetVertexBuffer"), NumFloats)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.SetInitActionInitializer()
			.DetermineInitialState();

		TRHIBufferInitializer<float> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
		float* BufferData = Initializer.GetWritableData();
		
		// Halton
		for (int32 i = 0; i < NumHaltonSamples; ++i)
		{
			BufferData[i * 2 + 0] = Halton(i + 1, 2) - 0.5f;
			BufferData[i * 2 + 1] = Halton(i + 1, 3) - 0.5f;
		}
		BufferData += NumHaltonSamples * 2;

		// MSAA
		{
			const float Offsets[] = 
			{
				/*2x*/ 4, 4, -4, -4, 
				/*4x*/ -2, -6, 6, -2, -6, 2, 2, 6, 
				/*8x*/ 1, -3, -1, 3, 5, 1, -3, -5, -5, 5, -7, -1, 3, 7, 7, -7, 
				/*16x*/ 1, 1, -1, -3, -3, 2, 4, -1, -5, -2, 2, 5, 5, 3, 3, -5, 2, 6, 0, -7, -4, -6, -6, 4, -8, 0, 7, -4, 6, 7, -7, -8
			};

			for (int32 i = 0; i < NumMSAASamples; ++i)
			{
				// Remap from (-8) - 7 to (-0.5) - 0.5
				BufferData[i * 2 + 0] = Offsets[i * 2 + 0] / 16.0f;
				BufferData[i * 2 + 1] = Offsets[i * 2 + 1] / 16.0f;
			}
		}

		VertexBufferRHI = Initializer.Finalize();
	}

	static constexpr int32 GetMSAADataOffset(int32 NumSamples)
	{
		int32 SampleOffset = NumHaltonSamples;
		SampleOffset += NumSamples > 2 ? 2 : 0;
		SampleOffset += NumSamples > 4 ? 4 : 0;
		SampleOffset += NumSamples > 8 ? 8 : 0;
		return SampleOffset * 2 * sizeof(float);
	}
};

TGlobalResource<FJitterOffsetVertexBuffer> GJitterOffsetVertexBuffer;

class FVector3AndInstancedVector2VertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FVertexDeclarationElementList Elements;
		Elements.Add(FVertexElement(0, 0, VET_Float3, 0, sizeof(FVector3f)));
		Elements.Add(FVertexElement(1, 0, VET_Float2, 1, sizeof(FVector2f), true /*bInUseInstanceIndex*/));
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}
	virtual void ReleaseRHI() override
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

TGlobalResource<FVector3AndInstancedVector2VertexDeclaration> GVector3AndInstancedVector2VertexDeclaration;


class FVector3AndThreeVector2VertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FVertexDeclarationElementList Elements;
		Elements.Add(FVertexElement(0, 0, VET_Float3, 0, sizeof(FVector3f)));
		Elements.Add(FVertexElement(1, 0, VET_Float2, 1, sizeof(FVector2f) * 3));
		Elements.Add(FVertexElement(1, 8, VET_Float2, 2, sizeof(FVector2f) * 3));
		Elements.Add(FVertexElement(1, 16, VET_Float2, 3, sizeof(FVector2f) * 3));
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}
	virtual void ReleaseRHI() override
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

TGlobalResource<FVector3AndThreeVector2VertexDeclaration> GVector3AndAndThreeVector2VertexDeclaration;


void FWaterQuadTreeGPU::Init(FRDGBuilder& GraphBuilder, const FInitParams& Params, TArray<FDraw>& Draws)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWaterQuadTreeGPU::Init);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, FWaterQuadTreeGPU_Init, "FWaterQuadTreeGPU::Init");

	// Create resources
	{
		check(!bInitialized);
		check(!QuadTreeTexture);
		check(!WaterZBoundsTexture);
		check(!WaterBodyRenderDataBuffer);

		// Make sure the quadtree is a power of two
		const FIntPoint ResolutionRaw = Params.RequestedQuadTreeResolution;
		const FIntPoint ResolutionPow2 = FIntPoint(FMath::RoundUpToPowerOfTwo(ResolutionRaw.X), FMath::RoundUpToPowerOfTwo(ResolutionRaw.Y));
		const int32 MaxDim = (int32)FMath::Max(ResolutionPow2.X, ResolutionPow2.Y);
		const int32 NumMipLevels = (int32)FMath::FloorLog2(MaxDim) + 1;

		FRHITextureCreateDesc QuadTreeTextureCreateDesc = FRHITextureCreateDesc::Create2D(TEXT("WaterQuadTree.QuadTree"), ResolutionPow2, PF_B8G8R8A8)
			.SetNumMips(NumMipLevels)
			.SetFlags(TexCreate_RenderTargetable | TexCreate_ShaderResource);
		QuadTreeTexture = GraphBuilder.RHICmdList.CreateTexture(QuadTreeTextureCreateDesc);

		FRHITextureCreateDesc ZBoundsTextureCreateDesc = FRHITextureCreateDesc::Create2D(TEXT("WaterQuadTree.ZBoundsTexture"), ResolutionPow2, PF_A2B10G10R10)
			.SetNumMips(NumMipLevels)
			.SetFlags(TexCreate_RenderTargetable | TexCreate_ShaderResource);
		WaterZBoundsTexture = GraphBuilder.RHICmdList.CreateTexture(ZBoundsTextureCreateDesc);

		WaterBodyRenderDataBuffer = GraphBuilder.ConvertToExternalBuffer(CreateStructuredBuffer<FWaterBodyRenderDataGPU>(GraphBuilder, TEXT("WaterQuadTree.WaterBodyRenderData"), Params.WaterBodyRenderData));

		check(Params.CaptureDepthRange > 0.0f);
		CaptureDepthRange = Params.CaptureDepthRange;
		bInitialized = true;
	}

	auto GetNumMSAASamples = [](int32 RequestedNumSamples)
	{
		int32 Result = 16;
		Result = RequestedNumSamples < 16 ? 8 : Result;
		Result = RequestedNumSamples < 8 ? 4 : Result;
		Result = RequestedNumSamples < 4 ? 2 : Result;
		Result = RequestedNumSamples < 2 ? 1 : Result;
		return Result;
	};

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

	FRDGTexture* QuadTreeTextureRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(QuadTreeTexture, TEXT("WaterQuadTree.QuadTree")));
	FRDGTexture* WaterZBoundsTextureRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(WaterZBoundsTexture, TEXT("WaterQuadTree.ZBoundsTexture")));

	FRDGBuffer* WaterBodyRenderDataBufferRDG = GraphBuilder.RegisterExternalBuffer(WaterBodyRenderDataBuffer);
	FRDGBufferSRV* WaterBodyRenderDataBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(WaterBodyRenderDataBufferRDG));

	const FIntPoint QuadTreeResolution = QuadTreeTextureRDG->Desc.Extent;
	// Not necessarily a power of two or a multiple of the quadtree LOD0 resolution. We rely on Texture.Load() to return 0 for out of bounds accesses.
	// Padding this texture is not required as we will not need to create a mip chain for it; but rather use it to initialize mip0 of our quadtree texture.
	const FIntPoint RasterResolution = Params.RequestedQuadTreeResolution * Params.SuperSamplingFactor;
	const int32 NumMipLevels = QuadTreeTextureRDG->Desc.NumMips;
	const uint8 NumMSAASamples = FMath::Min(GetNumMSAASamples(Params.NumMSAASamples), 8);
	const int32 NumJitterSamples = Params.bUseMSAAJitterPattern ? GetNumMSAASamples(Params.NumJitterSamples) : FMath::Clamp(Params.NumJitterSamples, 1, 16);

	FRDGTextureDesc WaterBodyRasterTextureDesc = FRDGTextureDesc::Create2D(RasterResolution, PF_A2B10G10R10, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource, 1, NumMSAASamples);
	FRDGTexture* WaterBodyRasterTexture = GraphBuilder.CreateTexture(WaterBodyRasterTextureDesc, TEXT("WaterQuadTree.WaterBodyRasterTexture"));
	FRDGTextureDesc ZBoundsRasterTextureDesc = FRDGTextureDesc::Create2D(RasterResolution, PF_A2B10G10R10, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource, 1, NumMSAASamples);
	FRDGTexture* ZBoundsRasterTexture = GraphBuilder.CreateTexture(ZBoundsRasterTextureDesc, TEXT("WaterQuadTree.ZBoundsRasterTexture"));


	// Raster water body meshes
	{
		enum class ERasterMode { Regular = 0, Jittered = 1, Conservative = 2 };
		const ERasterMode RasterMode = Params.bUseConservativeRasterization ? ERasterMode::Conservative : NumJitterSamples > 1 ? ERasterMode::Jittered : ERasterMode::Regular;

		FWaterQuadTreeVS::FPermutationDomain VSPermutationDomain;
		VSPermutationDomain.Set<FWaterQuadTreeVS::FRasterMode>(static_cast<int32>(RasterMode));
		TShaderMapRef<FWaterQuadTreeVS> VertexShader(ShaderMap, VSPermutationDomain);
		
		FWaterQuadTreePS::FPermutationDomain PSPermutationDomain;
		PSPermutationDomain.Set<FWaterQuadTreePS::FClipConservativeTriangle>(RasterMode == ERasterMode::Conservative);
		TShaderMapRef<FWaterQuadTreePS> PixelShader(ShaderMap, PSPermutationDomain);

		FWaterQuadTreeParameters* PassParameters = GraphBuilder.AllocParameters<FWaterQuadTreeParameters>();
		PassParameters->PS.RenderTargets[0] = FRenderTargetBinding(WaterBodyRasterTexture, ERenderTargetLoadAction::EClear);
		PassParameters->PS.RenderTargets[1] = FRenderTargetBinding(ZBoundsRasterTexture, ERenderTargetLoadAction::EClear);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("WaterQuadTreeRaster"),
			PassParameters,
			ERDGPassFlags::Raster,
			[PassParameters, RasterResolution, NumJitterSamples, RasterMode, bMSAAPattern = Params.bUseMSAAJitterPattern, JitterFootprint = Params.JitterSampleFootprint, Draws = MoveTemp(Draws), VertexShader, PixelShader](FRHICommandList& RHICmdList)
			{
				RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, RasterResolution.X, RasterResolution.Y, 1.0f);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.BlendState = TStaticBlendState<
					CW_RGBA, BO_Max, BF_One, BF_One, BO_Max, BF_One, BF_One,
					CW_RGBA, BO_Max, BF_One, BF_One, BO_Max, BF_One, BF_One>::GetRHI(); // MAX blending
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				switch (RasterMode)
				{
				case ERasterMode::Regular: GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector3(); break;
				case ERasterMode::Jittered: GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GVector3AndInstancedVector2VertexDeclaration.VertexDeclarationRHI; break;
				case ERasterMode::Conservative: GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GVector3AndAndThreeVector2VertexDeclaration.VertexDeclarationRHI; break;
				default: checkNoEntry(); break;
				}

				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				const FVector2f HalfPixelSize = FVector2f(1.0f / RasterResolution.X, 1.0f / RasterResolution.Y);
				const FVector2f JitterScale = JitterFootprint * FVector2f(2.0f / RasterResolution.X, 2.0f / RasterResolution.Y);
				if (RasterMode == ERasterMode::Jittered)
				{
					const int32 JitterVertexBufferOffset = bMSAAPattern ? FJitterOffsetVertexBuffer::GetMSAADataOffset(NumJitterSamples) : 0;
					RHICmdList.SetStreamSource(1, GJitterOffsetVertexBuffer.GetRHI(), JitterVertexBufferOffset);
				}

				for (const FDraw& Draw : Draws)
				{
					PassParameters->VS.Transform = Draw.Transform;
					PassParameters->VS.JitterScale = JitterScale;
					PassParameters->VS.HalfPixelSize = HalfPixelSize;
					PassParameters->PS.WaterBodyRenderDataIndex = Draw.WaterBodyRenderDataIndex;
					PassParameters->PS.Priority = Draw.Priority;
					PassParameters->PS.WaterBodyMinZ = Draw.MinZ;
					PassParameters->PS.WaterBodyMaxZ = Draw.MaxZ;
					PassParameters->PS.MaxWaveHeight = Draw.MaxWaveHeight;
					PassParameters->PS.bIsRiver = Draw.bIsRiver;

					SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);
					RHICmdList.SetStreamSource(0, Draw.VertexBuffer, 0);

					switch (RasterMode)
					{
					case ERasterMode::Regular: // Fallthrough
					case ERasterMode::Jittered:
					{
						const uint32 NumInstances = (RasterMode == ERasterMode::Jittered) ? NumJitterSamples : 1;
						RHICmdList.DrawIndexedPrimitive(Draw.IndexBuffer, Draw.BaseVertexIndex, 0, Draw.NumVertices, Draw.FirstIndex, Draw.NumPrimitives, NumInstances);
						break;
					}
					case ERasterMode::Conservative:
					{
						RHICmdList.SetStreamSource(1, Draw.TexCoordBuffer, 0);
						check((Draw.NumVertices % 3) == 0);
						RHICmdList.DrawPrimitive(Draw.BaseVertexIndex, Draw.NumVertices / 3, 1);
						break;
					}
					default: checkNoEntry(); break;
					}
				}
			});
	}

	// Merge river and non-river water bodies and downsample to quad tree LOD0 resolution
	{
		FWaterQuadTreeMergePS::FPermutationDomain PermutationDomain;
		PermutationDomain.Set<FWaterQuadTreeMergePS::FNumMSAASamples>(NumMSAASamples);
		TShaderMapRef<FWaterQuadTreeMergePS> PixelShader(ShaderMap, PermutationDomain);

		FWaterQuadTreeMergePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterQuadTreeMergePS::FParameters>();
		if (NumMSAASamples > 1)
		{
			PassParameters->WaterBodyRasterTextureMS = WaterBodyRasterTexture;
			PassParameters->ZBoundsRasterTextureMS = ZBoundsRasterTexture;
		}
		else
		{
			PassParameters->WaterBodyRasterTexture = WaterBodyRasterTexture;
			PassParameters->ZBoundsRasterTexture = ZBoundsRasterTexture;
		}
		PassParameters->WaterBodyRenderData = WaterBodyRenderDataBufferSRV;
		PassParameters->SuperSamplingFactor = Params.SuperSamplingFactor;
		PassParameters->RcpCaptureDepthRange = 1.0f / Params.CaptureDepthRange;
		PassParameters->ZBoundsPadding = GWaterQuadTreeZBoundsPadding / Params.CaptureDepthRange;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(QuadTreeTextureRDG, ERenderTargetLoadAction::ENoAction, 0);
		PassParameters->RenderTargets[1] = FRenderTargetBinding(WaterZBoundsTextureRDG, ERenderTargetLoadAction::ENoAction, 0);

		const FIntRect MergeViewport(0, 0, QuadTreeResolution.X, QuadTreeResolution.Y);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("WaterQuadTreeMerge"),
			PixelShader,
			PassParameters,
			MergeViewport);
	}

	// Build the mip chain
	{
		TShaderMapRef<FWaterQuadTreeBuildPS> PixelShader(ShaderMap);

		const bool bSupportsTextureViews = GRHISupportsTextureViews;
		FIntRect BuildViewport(0, 0, QuadTreeResolution.X, QuadTreeResolution.Y);

		for (int32 MipLevel = 1; MipLevel < NumMipLevels; ++MipLevel)
		{
			FRDGTextureSRVDesc QuadTreeSourceSRVDesc = FRDGTextureSRVDesc::CreateForMipLevel(QuadTreeTextureRDG, MipLevel - 1);
			FRDGTextureSRVDesc WaterZBoundsSourceSRVDesc = FRDGTextureSRVDesc::CreateForMipLevel(WaterZBoundsTextureRDG, MipLevel - 1);
			if (!bSupportsTextureViews)
			{
				QuadTreeSourceSRVDesc = FRDGTextureSRVDesc::Create(QuadTreeTextureRDG);
				WaterZBoundsSourceSRVDesc = FRDGTextureSRVDesc::Create(WaterZBoundsTextureRDG);
			}

			FWaterQuadTreeBuildPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterQuadTreeBuildPS::FParameters>();
			PassParameters->QuadTreeTexture = GraphBuilder.CreateSRV(QuadTreeSourceSRVDesc);
			PassParameters->WaterZBoundsTexture = GraphBuilder.CreateSRV(WaterZBoundsSourceSRVDesc);
			PassParameters->InputMipLevelIndex = bSupportsTextureViews ? 0 : (MipLevel - 1);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(QuadTreeTextureRDG, ERenderTargetLoadAction::ENoAction, MipLevel);
			PassParameters->RenderTargets[1] = FRenderTargetBinding(WaterZBoundsTextureRDG, ERenderTargetLoadAction::ENoAction, MipLevel);

			BuildViewport = BuildViewport / 2;

			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				ShaderMap,
				RDG_EVENT_NAME("WaterQuadTreeBuild"),
				PixelShader,
				PassParameters,
				BuildViewport);
		}
	}
}

void FWaterQuadTreeGPU::Traverse(FRDGBuilder& GraphBuilder, const FTraverseParams& Params) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWaterQuadTreeGPU::Traverse);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, FWaterQuadTreeGPU_Traverse, "FWaterQuadTreeGPU::Traverse");

	const uint32 NumViews = Params.Views.Num();

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

	FRDGTexture* QuadTreeTextureRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(QuadTreeTexture, TEXT("WaterQuadTree.QuadTree")));
	FRDGTexture* WaterZBoundsTextureRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(WaterZBoundsTexture, TEXT("WaterQuadTree.WaterSurfaceHeight")));
	FRDGBuffer* WaterBodyRenderDataBufferRDG = GraphBuilder.RegisterExternalBuffer(WaterBodyRenderDataBuffer);
	FRDGBuffer* IndirectArgsBuffer = GraphBuilder.RegisterExternalBuffer(Params.OutIndirectArgsBuffer);
	FRDGBuffer* InstanceDataOffsetsBuffer = GraphBuilder.RegisterExternalBuffer(Params.OutInstanceDataOffsetsBuffer);
	FRDGBuffer* InstanceData0Buffer = GraphBuilder.RegisterExternalBuffer(Params.OutInstanceData0Buffer);
	FRDGBuffer* InstanceData1Buffer = GraphBuilder.RegisterExternalBuffer(Params.OutInstanceData1Buffer);
	FRDGBuffer* InstanceData2Buffer = GraphBuilder.RegisterExternalBuffer(Params.OutInstanceData2Buffer);
	FRDGBuffer* InstanceData3Buffer = Params.bWithWaterSelectionSupport ? 
		GraphBuilder.RegisterExternalBuffer(Params.OutInstanceData3Buffer) : 
		GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1), TEXT("WaterQuadTree.InstanceData3Dummy"));
	
	FRDGBufferUAV* IndirectArgsBufferUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(IndirectArgsBuffer, PF_R32_UINT));
	FRDGBufferSRV* WaterBodyRenderDataBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(WaterBodyRenderDataBufferRDG));

	const EOcclusionQueryMode OcclusionQueryMode = static_cast<EOcclusionQueryMode>(GWaterQuadTreeOcclusionCulling);
	const bool bHZBOcclusionQueries = OcclusionQueryMode == EOcclusionQueryMode::HZB || OcclusionQueryMode == EOcclusionQueryMode::HZBAndPixelPrecise;
	const bool bPixelPreciseOcclusionQueries = OcclusionQueryMode == EOcclusionQueryMode::PixelPrecise || OcclusionQueryMode == EOcclusionQueryMode::HZBAndPixelPrecise;
	
	FRDGBuffer* OcclusionQueryIndirectArgsBuffer = nullptr;
	FRDGBufferUAV* OcclusionQueryIndirectArgsBufferUAV = nullptr;
	if (bPixelPreciseOcclusionQueries)
	{
		OcclusionQueryIndirectArgsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDrawIndexedIndirectParameters>(NumViews), TEXT("WaterQuadTree.OcclusionQueryIndirectArgs"));
		OcclusionQueryIndirectArgsBufferUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(OcclusionQueryIndirectArgsBuffer, PF_R32_UINT));
	}

	const uint32 NumBucketsPerView = Params.NumMaterials; // The GPU-driven water rendering path only uses a single density mesh tile to draw everything, so NumBuckets is equal to NumMaterials.
	const uint32 NumBucketsTotal = NumBucketsPerView * NumViews;
	const FIntPoint QuadTreeResolution = QuadTreeTextureRDG->Desc.Extent;

	// Initialize indirect args
	{
		FWaterQuadTreeInitializeIndirectArgsCS::FPermutationDomain PermutationDomain;
		PermutationDomain.Set<FWaterQuadTreeInitializeIndirectArgsCS::FPreciseOcclusionQueries>(bPixelPreciseOcclusionQueries);
		TShaderMapRef<FWaterQuadTreeInitializeIndirectArgsCS> ComputeShader(ShaderMap, PermutationDomain);

		FWaterQuadTreeInitializeIndirectArgsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterQuadTreeInitializeIndirectArgsCS::FParameters>();
		PassParameters->IndirectArgs = IndirectArgsBufferUAV;
		PassParameters->OcclusionQueryArgs = OcclusionQueryIndirectArgsBufferUAV;
		PassParameters->NumDrawBuckets = Params.NumMaterials;
		PassParameters->NumViews = NumViews;
		PassParameters->NumQuads = Params.NumQuadsPerTileSide;
		
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("WaterQuadTreeInitIndirectArgs"), ComputeShader, PassParameters, FComputeShaderUtils::GetGroupCount(NumBucketsTotal, 64));
	}

	// Iterate over all views for which to create indirect water draws
	for (uint32 ViewIndex = 0; ViewIndex < NumViews; ++ViewIndex)
	{
		const FSceneView* View = Params.Views[ViewIndex];
		const FViewInfo* ViewInfo = View->bIsViewInfo ? reinterpret_cast<const FViewInfo*>(View) : nullptr;

		const FVector PreViewTranslation = View->ViewMatrices.GetPreViewTranslation();
		const FVector3f QuadTreePositionTranslatedWorldSpace = FVector3f(Params.QuadTreePosition + PreViewTranslation);
		const FVector3f ObserverPositionTranslatedWorldSpace = FVector3f(View->ViewMatrices.GetViewOrigin() + PreViewTranslation);

		const uint32 MaxNumDraws = QuadTreeResolution.X * QuadTreeResolution.Y;
		FRDGBuffer* PackedNodes = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateByteAddressDesc((1 + MaxNumDraws) * sizeof(uint32)), TEXT("WaterQuadTree.PackedNodes"));
		FRDGBuffer* BucketCounts = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateByteAddressDesc(FMath::Max(1u, NumBucketsPerView) * sizeof(uint32)), TEXT("WaterQuadTree.DrawBucketCounts"));
		FRDGBufferUAV* PackedNodesUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(PackedNodes, PF_R32_UINT));
		FRDGBufferUAV* BucketCountsUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(BucketCounts, PF_R32_UINT));
		FRDGBufferSRV* PackedNodesSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(PackedNodes, PF_R32_UINT));
		FRDGBufferSRV* BucketCountsSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(BucketCounts, PF_R32_UINT));

		const bool bPixelPreciseOQForThisView = bPixelPreciseOcclusionQueries && ViewInfo && Params.bDepthBufferIsPopulated;

		FRDGBuffer* OcclusionQueryBoxes = nullptr;
		FRDGBuffer* OcclusionQueryResults = nullptr;
		FRDGBufferUAV* OcclusionQueryBoxesUAV = nullptr;
		FRDGBufferUAV* OcclusionQueryResultsUAV = nullptr;
		FRDGBufferSRV* OcclusionQueryBoxesSRV = nullptr;
		FRDGBufferSRV* OcclusionQueryResultsSRV = nullptr;
		if (bPixelPreciseOQForThisView)
		{
			OcclusionQueryBoxes = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(float) * 4, MaxNumDraws * 2), TEXT("WaterQuadTree.OcclusionQueryBoxes"));
			OcclusionQueryResults = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), MaxNumDraws), TEXT("WaterQuadTree.OcclusionQueryResults"));
			OcclusionQueryBoxesUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(OcclusionQueryBoxes, PF_A32B32G32R32F));
			OcclusionQueryResultsUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(OcclusionQueryResults, PF_R32_UINT));
			OcclusionQueryBoxesSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(OcclusionQueryBoxes, PF_A32B32G32R32F));
			OcclusionQueryResultsSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(OcclusionQueryResults, PF_R32_UINT));
		}

		// Clear bucket counts buffer and packed nodes counter
		{
			TShaderMapRef<FWaterQuadTreeClearPerViewBuffersCS> ComputeShader(ShaderMap);

			FWaterQuadTreeClearPerViewBuffersCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterQuadTreeClearPerViewBuffersCS::FParameters>();
			PassParameters->BucketCounts = BucketCountsUAV;
			PassParameters->PackedNodes = PackedNodesUAV;
			PassParameters->NumDrawBuckets = NumBucketsPerView;

			FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("WaterQuadTreeClearPerViewBuffers"), ComputeShader, PassParameters, FComputeShaderUtils::GetGroupCount(NumBucketsPerView, 64));
		}

		// Traverse quadtree
		{
			FWaterQuadTreeTraverseCS::FPermutationDomain PermutationDomain;
			PermutationDomain.Set<FWaterQuadTreeTraverseCS::FPreciseOcclusionQueries>(bPixelPreciseOQForThisView);
			TShaderMapRef<FWaterQuadTreeTraverseCS> ComputeShader(ShaderMap, PermutationDomain);

			const uint32 NumMipLevels = QuadTreeTextureRDG->Desc.NumMips;
			uint32 NumTexelsTotal = 0;
			for (uint32 MipLevelIndex = 0; MipLevelIndex < NumMipLevels; ++MipLevelIndex)
			{
				const FIntPoint MipExtent = FIntPoint(FMath::Max(1, QuadTreeTextureRDG->Desc.Extent.X >> MipLevelIndex), FMath::Max(1, QuadTreeTextureRDG->Desc.Extent.Y >> MipLevelIndex));
				NumTexelsTotal += MipExtent.X * MipExtent.Y;
			}

			FWaterQuadTreeTraverseCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterQuadTreeTraverseCS::FParameters>();
			PassParameters->View = GetShaderBinding(View->ViewUniformBuffer);
			PassParameters->PackedNodes = PackedNodesUAV;
			PassParameters->OcclusionQueryBoxes = OcclusionQueryBoxesUAV;
			PassParameters->OcclusionVisibility = OcclusionQueryResultsUAV;
			PassParameters->OcclusionQueryArgs = OcclusionQueryIndirectArgsBufferUAV;
			PassParameters->QuadTreeTexture = QuadTreeTextureRDG;
			PassParameters->WaterZBoundsTexture = WaterZBoundsTextureRDG;
			PassParameters->WaterBodyRenderData = WaterBodyRenderDataBufferSRV;

			// HZB
			// These parameters should be filled with GetHZBParameters() or GetDummyHZBParameters(), but these functions are not exposed as public function.
			// This is why these parameters are setup manually.
			{
				PassParameters->HZBParameters.HZBTexture = (ViewInfo && ViewInfo->HZB) ? ViewInfo->HZB : GSystemTextures.GetBlackDummy(GraphBuilder);
				PassParameters->HZBParameters.HZBSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				PassParameters->HZBParameters.HZBSize = ViewInfo ? FVector2f(ViewInfo->HZBMipmap0Size) : FVector2f::ZeroVector;
				PassParameters->HZBParameters.HZBViewSize = ViewInfo ? FVector2f(ViewInfo->ViewRect.Size()) : FVector2f::ZeroVector;
			}
			
			PassParameters->CullingBoundsAABB = FVector4f(Params.CullingBounds.Min.X, Params.CullingBounds.Min.Y, Params.CullingBounds.Max.X, Params.CullingBounds.Max.Y);
			PassParameters->QuadTreePosition = QuadTreePositionTranslatedWorldSpace;
			PassParameters->ObserverPosition = ObserverPositionTranslatedWorldSpace;
			PassParameters->QuadTreeResolutionX = QuadTreeResolution.X;
			PassParameters->QuadTreeResolutionY = QuadTreeResolution.Y;
			PassParameters->ViewIndex = ViewIndex;
			PassParameters->LeafSize = Params.LeafSize;
			PassParameters->LODScale = Params.LODScale;
			PassParameters->CaptureDepthRange = CaptureDepthRange;
			PassParameters->ForceCollapseDensityLevel = -1; // Params.ForceCollapseDensityLevel; // TODO: Properly implement support for this parameter
			PassParameters->NumLODs = NumMipLevels;
			PassParameters->NumDispatchedThreads = NumTexelsTotal;
			PassParameters->bHZBOcclusionCullingEnabled = bHZBOcclusionQueries && (ViewInfo && ViewInfo->HZB);

			FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("WaterQuadTreeTraverse(View: %i)", ViewIndex), ComputeShader, PassParameters, FComputeShaderUtils::GetGroupCount(NumTexelsTotal, 64));
		}

		// Raster occlusion queries
		if (bPixelPreciseOQForThisView)
		{
			TShaderMapRef<FWaterQuadTreeOcclusionQueryVS> VertexShader(ShaderMap);
			TShaderMapRef<FWaterQuadTreeOcclusionQueryPS> PixelShader(ShaderMap);

			FRDGTextureRef DepthTexture = GetIfProduced(ViewInfo->GetSceneTextures().Depth.Target);
			check(DepthTexture);

			FWaterQuadTreeOcclusionQueryParameters* PassParameters = GraphBuilder.AllocParameters<FWaterQuadTreeOcclusionQueryParameters>();
			PassParameters->IndirectDrawArgsBuffer = OcclusionQueryIndirectArgsBuffer;
			PassParameters->VS.View = GetShaderBinding(View->ViewUniformBuffer);
			PassParameters->VS.OcclusionQueryBoxes = OcclusionQueryBoxesSRV;
			PassParameters->PS.Visibility = OcclusionQueryResultsUAV;
			PassParameters->PS.RenderTargets.DepthStencil = FDepthStencilBinding(DepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthRead_StencilNop);

			const FIntVector4 ViewRect = FIntVector4(ViewInfo->ViewRect.Min.X, ViewInfo->ViewRect.Min.Y, ViewInfo->ViewRect.Max.X, ViewInfo->ViewRect.Max.Y);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("WaterQuadTreeOcclusionCulling(View: %i)", ViewIndex),
				PassParameters,
				ERDGPassFlags::Raster,
				[PassParameters, ViewRect, ViewIndex, VertexShader, PixelShader](FRHICommandList& RHICmdList)
				{
					RHICmdList.SetViewport(ViewRect.X, ViewRect.Y, 0.0f, ViewRect.Z, ViewRect.W, 1.0f);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
					GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
					GraphicsPSOInit.PrimitiveType = PT_TriangleList;

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

					PassParameters->IndirectDrawArgsBuffer->MarkResourceAsUsed();

					RHICmdList.SetStreamSource(0, GetUnitCubeVertexBuffer(), 0);
					RHICmdList.DrawIndexedPrimitiveIndirect(GetUnitCubeIndexBuffer(), PassParameters->IndirectDrawArgsBuffer->GetRHI(), ViewIndex * sizeof(FRHIDrawIndexedIndirectParameters));
				});
		}

		// Compute bucket counts
		{
			FWaterQuadTreeBucketCountsCS::FPermutationDomain PermutationDomain;
			PermutationDomain.Set<FWaterQuadTreeBucketCountsCS::FPreciseOcclusionQueries>(bPixelPreciseOQForThisView);
			TShaderMapRef<FWaterQuadTreeBucketCountsCS> ComputeShader(ShaderMap, PermutationDomain);

			FWaterQuadTreeBucketCountsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterQuadTreeBucketCountsCS::FParameters>();
			PassParameters->BucketCounts = BucketCountsUAV;
			PassParameters->QuadTreeTexture = QuadTreeTextureRDG;
			PassParameters->WaterBodyRenderData = WaterBodyRenderDataBufferSRV;
			PassParameters->PackedNodes = PackedNodesSRV;
			PassParameters->OcclusionResults = OcclusionQueryResultsSRV;
			PassParameters->NumDispatchedThreads = FMath::Min(FMath::DivideAndRoundUp(MaxNumDraws, 64u), 65535u) * 64u; // The maximum number of dispatched groups is 64k
			PassParameters->NumDensities = Params.NumDensities;
			PassParameters->NumQuadsLOD0 = Params.NumQuadsLOD0;
			PassParameters->NumQuadsPerDraw = Params.NumQuadsPerTileSide;

			FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("WaterQuadTreeDrawBucketCounts(View: %i)", ViewIndex), ComputeShader, PassParameters, FComputeShaderUtils::GetGroupCount(PassParameters->NumDispatchedThreads, 64));
		}

		// Compute bucket prefix sums
		{
			FWaterQuadTreeBucketPrefixSumCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FWaterQuadTreeBucketPrefixSumCS::FParallelPrefixSum>(GWaterQuadTreeParallelPrefixSum != 0);
			TShaderMapRef<FWaterQuadTreeBucketPrefixSumCS> ComputeShader(ShaderMap, PermutationVector);

			FWaterQuadTreeBucketPrefixSumCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterQuadTreeBucketPrefixSumCS::FParameters>();
			PassParameters->BucketPrefixSums = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(InstanceDataOffsetsBuffer, PF_R32_UINT));
			PassParameters->BucketCounts = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(BucketCounts));
			PassParameters->NumBuckets = NumBucketsPerView;
			PassParameters->OutputOffset = ViewIndex * NumBucketsPerView;
			PassParameters->bWriteTotalSumAtBufferEnd =  Params.Views.IsValidIndex(ViewIndex + 1); // Propagate total sum so the next iteration/view can compute correct global prefix sums/offsets.

			FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("WaterQuadTreeDrawBucketPrefixSums(View: %i)", ViewIndex), ComputeShader, PassParameters, FIntVector3(1, 1, 1));
		}

		// Generate instance data and fill indirect args
		{
			FWaterQuadTreeGenerateInstanceDataCS::FPermutationDomain PermutationDomain;
			PermutationDomain.Set<FWaterQuadTreeGenerateInstanceDataCS::FPreciseOcclusionQueries>(bPixelPreciseOQForThisView);
			TShaderMapRef<FWaterQuadTreeGenerateInstanceDataCS> ComputeShader(ShaderMap, PermutationDomain);

			FWaterQuadTreeGenerateInstanceDataCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterQuadTreeGenerateInstanceDataCS::FParameters>();
			PassParameters->IndirectArgs = IndirectArgsBufferUAV;
			PassParameters->InstanceData0 = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(InstanceData0Buffer, PF_R32_UINT));
			PassParameters->InstanceData1 = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(InstanceData1Buffer, PF_R32_UINT));
			PassParameters->InstanceData2 = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(InstanceData2Buffer, PF_R32_UINT));
			PassParameters->InstanceData3 = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(InstanceData3Buffer, PF_R32_UINT));
			PassParameters->QuadTreeTexture = QuadTreeTextureRDG;
			PassParameters->WaterZBoundsTexture = WaterZBoundsTextureRDG;
			PassParameters->WaterBodyRenderData = WaterBodyRenderDataBufferSRV;
			PassParameters->PackedNodes = PackedNodesSRV;
			PassParameters->InstanceDataOffsets = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(InstanceDataOffsetsBuffer, PF_R32_UINT));
			PassParameters->OcclusionResults = OcclusionQueryResultsSRV;
			PassParameters->QuadTreePosition = QuadTreePositionTranslatedWorldSpace;
			PassParameters->ObserverPosition = ObserverPositionTranslatedWorldSpace;
			PassParameters->QuadTreeResolutionX = QuadTreeResolution.X;
			PassParameters->QuadTreeResolutionY = QuadTreeResolution.Y;
			PassParameters->NumDensities = Params.NumDensities;
			PassParameters->NumMaterials = Params.NumMaterials;
			PassParameters->NumDispatchedThreads = FMath::Min(FMath::DivideAndRoundUp(MaxNumDraws, 64u), 65535u) * 64u; // The maximum number of dispatched groups is 64k
			PassParameters->BucketIndexOffset = ViewIndex * NumBucketsPerView;
			PassParameters->NumLODs = QuadTreeTextureRDG->Desc.NumMips;
			PassParameters->NumQuadsLOD0 = Params.NumQuadsLOD0;
			PassParameters->NumQuadsPerDraw = Params.NumQuadsPerTileSide;
			PassParameters->LeafSize = Params.LeafSize;
			PassParameters->LODScale = Params.LODScale;
			PassParameters->CaptureDepthRange = CaptureDepthRange;
			PassParameters->StereoPassInstanceFactor = View->GetStereoPassInstanceFactor();
			PassParameters->bWithWaterSelectionSupport = Params.bWithWaterSelectionSupport;
			PassParameters->bLODMorphingEnabled = Params.bLODMorphingEnabled;
			PassParameters->bInstancedStereoRendering = View->bIsInstancedStereoEnabled;

			FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("WaterQuadTreeGenerateDraws(View: %i)", ViewIndex), ComputeShader, PassParameters, FComputeShaderUtils::GetGroupCount(PassParameters->NumDispatchedThreads, 64));
		}

		// Debug show tiles
		if (Params.DebugShowTile != 0)
		{
			ShaderPrint::SetEnabled(true);

			const bool bEnableShaderPrint = ViewInfo != nullptr
				&& ShaderPrint::IsEnabled()
				&& ShaderPrint::IsEnabled(ViewInfo->ShaderPrintData)
				&& ShaderPrint::IsValid(ViewInfo->ShaderPrintData)
				&& ShaderPrint::IsSupported(GetFeatureLevelShaderPlatform(GMaxRHIFeatureLevel));

			if (bEnableShaderPrint)
			{
				// We'll be potentially drawing a lot of bounding boxes, each made up of 12 line segments, so reserve some space
				ShaderPrint::RequestSpaceForLines(12 * MaxNumDraws);

				FWaterQuadTreeDebugCS::FPermutationDomain PermutationDomain;
				PermutationDomain.Set<FWaterQuadTreeDebugCS::FPreciseOcclusionQueries>(bPixelPreciseOQForThisView);
				TShaderMapRef<FWaterQuadTreeDebugCS> ComputeShader(ShaderMap, PermutationDomain);

				FWaterQuadTreeDebugCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterQuadTreeDebugCS::FParameters>();
				ShaderPrint::SetParameters(GraphBuilder, ViewInfo->ShaderPrintData, PassParameters->ShaderPrintUniformBuffer);
				PassParameters->QuadTreeTexture = QuadTreeTextureRDG;
				PassParameters->WaterZBoundsTexture = WaterZBoundsTextureRDG;
				PassParameters->WaterBodyRenderData = WaterBodyRenderDataBufferSRV;
				PassParameters->PackedNodes = PackedNodesSRV;
				PassParameters->OcclusionResults = OcclusionQueryResultsSRV;
				PassParameters->QuadTreePosition = QuadTreePositionTranslatedWorldSpace;
				PassParameters->NumDispatchedThreads = FMath::Min(FMath::DivideAndRoundUp(MaxNumDraws, 64u), 65535u) * 64u; // The maximum number of dispatched groups is 64k
				PassParameters->LeafSize = Params.LeafSize;
				PassParameters->CaptureDepthRange = CaptureDepthRange;

				FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("WaterQuadTreeDebug(View: %i)", ViewIndex), ComputeShader, PassParameters, FComputeShaderUtils::GetGroupCount(PassParameters->NumDispatchedThreads, 64));
			}
		}
	}

	GraphBuilder.UseExternalAccessMode(IndirectArgsBuffer, ERHIAccess::IndirectArgs);
	GraphBuilder.UseExternalAccessMode(InstanceDataOffsetsBuffer, ERHIAccess::SRVMask);
	GraphBuilder.UseExternalAccessMode(InstanceData0Buffer, ERHIAccess::SRVMask | ERHIAccess::VertexOrIndexBuffer);
	GraphBuilder.UseExternalAccessMode(InstanceData1Buffer, ERHIAccess::SRVMask | ERHIAccess::VertexOrIndexBuffer);
	GraphBuilder.UseExternalAccessMode(InstanceData2Buffer, ERHIAccess::SRVMask | ERHIAccess::VertexOrIndexBuffer);
	if (Params.bWithWaterSelectionSupport)
	{
		GraphBuilder.UseExternalAccessMode(InstanceData3Buffer, ERHIAccess::SRVMask | ERHIAccess::VertexOrIndexBuffer);
	}
}
