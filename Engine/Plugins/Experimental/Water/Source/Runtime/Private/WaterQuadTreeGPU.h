// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"

class FRHIBuffer;
class FRHITexture;
class FRDGBuilder;
class FRDGTexture;
class FRDGPooledBuffer;
class FSceneView;
struct FShaderPrintData;

class FWaterQuadTreeGPU
{
public:
	struct FDraw
	{
		FMatrix44f Transform = FMatrix44f::Identity;
		FRHIBuffer* VertexBuffer = nullptr;
		FRHIBuffer* TexCoordBuffer = nullptr; // Stores previous and next vertex in the triangle for every vertex when using conservative rasterization
		FRHIBuffer* IndexBuffer = nullptr;
		uint32 FirstIndex = 0;
		uint32 NumPrimitives = 0;
		uint32 BaseVertexIndex = 0;
		uint32 NumVertices = 0;
		uint32 WaterBodyRenderDataIndex = INDEX_NONE;
		int32 Priority = 0;
		float MinZ = 1.0f; // Min Z of the water body bounding box scaled into 0-1 depth buffer space
		float MaxZ = 0.0f; // Max Z of the water body bounding box scaled into 0-1 depth buffer space
		float MaxWaveHeight = 0.0f; // Scaled into 0-1 depth buffer space
		bool bIsRiver = false;
	};

	struct FWaterBodyRenderDataGPU
	{
		uint32_t WaterBodyIndex = INDEX_NONE;
		uint32_t MaterialIndex = INDEX_NONE;
		uint32_t RiverToLakeMaterialIndex = INDEX_NONE;
		uint32_t RiverToOceanMaterialIndex = INDEX_NONE;
		uint32_t WaterBodyType = INDEX_NONE;
		uint32_t HitProxyColorAndIsSelected = 0;
		float SurfaceBaseHeight = 0.0f;
		float MinZ = 1.0f;
		float MaxZ = 0.0f;
		float MaxWaveHeight = 0.0f;
	};

	struct FInitParams
	{
		TArrayView<FWaterBodyRenderDataGPU> WaterBodyRenderData;
		FIntPoint RequestedQuadTreeResolution = FIntPoint::ZeroValue;
		int32 SuperSamplingFactor = 1;
		int32 NumMSAASamples = 1;
		int32 NumJitterSamples = 1;
		float JitterSampleFootprint = 1.0f;
		float CaptureDepthRange = 1.0f;
		bool bUseMSAAJitterPattern = false;
		bool bUseConservativeRasterization = false;
	};

	struct FTraverseParams
	{
		FRDGPooledBuffer* OutIndirectArgsBuffer = nullptr;
		FRDGPooledBuffer* OutInstanceDataOffsetsBuffer = nullptr;
		FRDGPooledBuffer* OutInstanceData0Buffer = nullptr;
		FRDGPooledBuffer* OutInstanceData1Buffer = nullptr;
		FRDGPooledBuffer* OutInstanceData2Buffer = nullptr;
		FRDGPooledBuffer* OutInstanceData3Buffer = nullptr;
		TArray<const FSceneView*> Views;
		FVector QuadTreePosition;
		FBox2D CullingBounds = FBox2D(ForceInit);
		uint32 NumDensities = 0;
		uint32 NumMaterials = 0;
		uint32 NumQuadsLOD0 = 0;
		uint32 NumQuadsPerTileSide = 0;
		int32 ForceCollapseDensityLevel = -1;
		float LeafSize = 1.0f;
		float LODScale = 1.0f;
		int32 DebugShowTile = 0;
		bool bWithWaterSelectionSupport = false;
		bool bLODMorphingEnabled = false;
		bool bDepthBufferIsPopulated = false;
	};

	void Init(FRDGBuilder& GraphBuilder, const FInitParams& Params, TArray<FDraw>& Draws);

	void Traverse(FRDGBuilder& GraphBuilder, const FTraverseParams& Params) const;

	bool IsInitialized() const { return bInitialized; }

private:
	enum class EOcclusionQueryMode
	{
		Disabled = 0,
		HZB = 1,
		PixelPrecise = 2,
		HZBAndPixelPrecise = 3
	};

	/** Mipped texture representing the water quadtree on the GPU */
	TRefCountPtr<FRHITexture> QuadTreeTexture;

	/** Mipped texture representing the Z bounds of each node of the water quadtree on the GPU */
	TRefCountPtr<FRHITexture> WaterZBoundsTexture;

	/** GPU representation of the water quad trees' array of water body render data */
	TRefCountPtr<FRDGPooledBuffer> WaterBodyRenderDataBuffer;

	float CaptureDepthRange = 0.0f;
	bool bInitialized = false;
};