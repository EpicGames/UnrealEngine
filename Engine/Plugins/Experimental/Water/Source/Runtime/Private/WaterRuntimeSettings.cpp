// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterRuntimeSettings.h"
#include "Materials/MaterialInterface.h"
#include "WaterBodyRiverComponent.h"
#include "WaterBodyLakeComponent.h"
#include "WaterBodyOceanComponent.h"
#include "WaterBodyCustomComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterRuntimeSettings)


#if WITH_EDITOR
UWaterRuntimeSettings::FOnUpdateSettings UWaterRuntimeSettings::OnSettingsChange;
#endif //WITH_EDITOR

UWaterRuntimeSettings::UWaterRuntimeSettings()
	: MaterialParameterCollection(FSoftObjectPath(TEXT("/Water/Materials/MPC/MPC_Water.MPC_Water")))
	, DefaultWaterCollisionProfileName(TEXT("WaterBodyCollision"))
	, DefaultWaterInfoMaterial(FSoftObjectPath(TEXT("/Water/Materials/WaterInfo/DrawWaterInfo.DrawWaterInfo")))
{
	
}

FName UWaterRuntimeSettings::GetCategoryName() const
{
	return FName(TEXT("Plugins"));
}

UMaterialInterface* UWaterRuntimeSettings::GetDefaultWaterInfoMaterial() const
{
	return DefaultWaterInfoMaterial.LoadSynchronous();
}

void UWaterRuntimeSettings::PostInitProperties()
{
	Super::PostInitProperties();
}

TSubclassOf<UWaterBodyRiverComponent> UWaterRuntimeSettings::GetWaterBodyRiverComponentClass() const
{
	return WaterBodyRiverComponentClass_DEPRECATED;
}

TSubclassOf<UWaterBodyLakeComponent> UWaterRuntimeSettings::GetWaterBodyLakeComponentClass() const
{
	return WaterBodyLakeComponentClass_DEPRECATED;
}

TSubclassOf<UWaterBodyOceanComponent> UWaterRuntimeSettings::GetWaterBodyOceanComponentClass() const
{
	return WaterBodyOceanComponentClass_DEPRECATED;
}

TSubclassOf<UWaterBodyCustomComponent> UWaterRuntimeSettings::GetWaterBodyCustomComponentClass() const
{
	return WaterBodyCustomComponentClass_DEPRECATED;
}

#if WITH_EDITOR
void UWaterRuntimeSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnSettingsChange.Broadcast(this, PropertyChangedEvent.ChangeType);
}
#endif // WITH_EDITOR

