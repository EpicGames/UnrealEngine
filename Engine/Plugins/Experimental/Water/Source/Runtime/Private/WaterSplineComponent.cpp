// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterSplineComponent.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "WaterBodyActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterSplineComponent)

UWaterSplineComponent::UWaterSplineComponent(const FObjectInitializer& ObjectInitializer)
	: USplineComponent(ObjectInitializer)
{
	SetCollisionEnabled(ECollisionEnabled::NoCollision);

	//@todo_water: Remove once AWaterBody is not Blueprintable
	{
		// Add default spline points
		ClearSplinePoints();

		AddPoint(FSplinePoint(0.f, FVector(0, 0, 0), FVector::ZeroVector, FVector::ZeroVector, FQuat::Identity.Rotator(), FVector(WaterSplineDefaults.DefaultWidth, WaterSplineDefaults.DefaultDepth, 1.0f), ESplinePointType::Curve), false);
		AddPoint(FSplinePoint(1.f, FVector(7000, -3000, 0), FVector::ZeroVector, FVector::ZeroVector, FQuat::Identity.Rotator(), FVector(WaterSplineDefaults.DefaultWidth, WaterSplineDefaults.DefaultDepth, 1.0f), ESplinePointType::Curve), false);
		AddPoint(FSplinePoint(2.f, FVector(6500, 6500, 0), FVector::ZeroVector, FVector::ZeroVector, FQuat::Identity.Rotator(), FVector(WaterSplineDefaults.DefaultWidth, WaterSplineDefaults.DefaultDepth, 1.0f), ESplinePointType::Curve), false);
	}
}

void UWaterSplineComponent::PostLoad()
{
	Super::PostLoad();


#if WITH_EDITOR
	const bool bAnythingChanged = SynchronizeWaterProperties();
	// @todo This can call into script which is illegal during post load
/*
	if (bAnythingChanged)
	{
		WaterSplineDataChangedEvent.Broadcast();
	}*/
#endif
}

void UWaterSplineComponent::PostDuplicate(bool bDuplicateForPie)
{
	Super::PostDuplicate(bDuplicateForPie);

#if WITH_EDITOR
	if (!bDuplicateForPie)
	{
		SynchronizeWaterProperties();

		WaterSplineDataChangedEvent.Broadcast(FOnWaterSplineDataChangedParams());
	}
#endif // WITH_EDITOR
}

USplineMetadata* UWaterSplineComponent::GetSplinePointsMetadata()
{
	if (AWaterBody* OwningBody = GetTypedOuter<AWaterBody>())
	{
		// This function may be called before the water body actor has initialized the component
		return OwningBody->GetWaterBodyComponent() ? OwningBody->GetWaterBodyComponent()->GetWaterSplineMetadata() : nullptr;
	}

	return nullptr;
}

const USplineMetadata* UWaterSplineComponent::GetSplinePointsMetadata() const
{
	if (AWaterBody* OwningBody = GetTypedOuter<AWaterBody>())
	{
		// This function may be called before the water body actor has initialized the component
		return OwningBody->GetWaterBodyComponent() ? OwningBody->GetWaterBodyComponent()->GetWaterSplineMetadata() : nullptr;
	}

	return nullptr;
}

TArray<ESplinePointType::Type> UWaterSplineComponent::GetEnabledSplinePointTypes() const
{
	return
		{
			ESplinePointType::Linear,
			ESplinePointType::Curve,
			ESplinePointType::CurveClamped,
			ESplinePointType::CurveCustomTangent
		};
}

void UWaterSplineComponent::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Super::Serialize(Ar);
}

FBoxSphereBounds UWaterSplineComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	// We should include depth in our calculation
	FBoxSphereBounds SplineBounds = Super::CalcBounds(LocalToWorld);

	const UWaterSplineMetadata* Metadata = Cast<UWaterSplineMetadata>(GetSplinePointsMetadata());
	if(Metadata)
	{
		const int32 NumPoints = Metadata->Depth.Points.Num();

		float MaxDepth = 0.0f;
		for (int32 Index = 0; Index < NumPoints; ++Index)
		{
			MaxDepth = FMath::Max(MaxDepth, Metadata->Depth.Points[Index].OutVal);
		}

		FBox DepthBox(FVector::ZeroVector, FVector(0, 0, -MaxDepth));

		return SplineBounds + FBoxSphereBounds(DepthBox.TransformBy(LocalToWorld));
	}
	else
	{
		return SplineBounds;
	}
}

void UWaterSplineComponent::K2_SynchronizeAndBroadcastDataChange()
{
#if WITH_EDITOR
	SynchronizeWaterProperties();

	FPropertyChangedEvent PropertyChangedEvent(FindFProperty<FProperty>(UWaterSplineComponent::StaticClass(), TEXT("SplineCurves")), 1 << 6);
	FOnWaterSplineDataChangedParams OnWaterSplineDataChangedParams(PropertyChangedEvent);
	OnWaterSplineDataChangedParams.bUserTriggered = true;

	WaterSplineDataChangedEvent.Broadcast(OnWaterSplineDataChangedParams);
#endif
}


#if WITH_EDITOR

bool UWaterSplineComponent::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty && InProperty->GetFName() == TEXT("bClosedLoop"))
	{
		return false;
	}
	return Super::CanEditChange(InProperty);
}

void UWaterSplineComponent::PostEditUndo()
{
	Super::PostEditUndo();

	WaterSplineDataChangedEvent.Broadcast(FOnWaterSplineDataChangedParams());
}

void UWaterSplineComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	SynchronizeWaterProperties();

	FOnWaterSplineDataChangedParams OnWaterSplineDataChangedParams(PropertyChangedEvent);
	OnWaterSplineDataChangedParams.bUserTriggered = true;
	
	WaterSplineDataChangedEvent.Broadcast(OnWaterSplineDataChangedParams);
}

void UWaterSplineComponent::PostEditImport()
{
	Super::PostEditImport();

	SynchronizeWaterProperties();

	WaterSplineDataChangedEvent.Broadcast(FOnWaterSplineDataChangedParams());
}

void UWaterSplineComponent::ResetSpline(const TArray<FVector>& Points)
{
	ClearSplinePoints(false);
	PreviousWaterSplineDefaults = WaterSplineDefaults;
	
	for (const FVector& Point : Points)
	{
		AddSplinePoint(Point, ESplineCoordinateSpace::Local, false);
	}

	UpdateSpline();
	SynchronizeWaterProperties();
	WaterSplineDataChangedEvent.Broadcast(FOnWaterSplineDataChangedParams());
}

bool UWaterSplineComponent::SynchronizeWaterProperties()
{
	const bool bFixOldProperties = GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::FixUpWaterMetadata;

	bool bAnythingChanged = false;
	UWaterSplineMetadata* Metadata = Cast<UWaterSplineMetadata>(GetSplinePointsMetadata());
	if(Metadata)
	{
		Metadata->Fixup(GetNumberOfSplinePoints(), this);
	
		for (int32 Point = 0; Point < GetNumberOfSplinePoints(); ++Point)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (!SplineCurves.Scale.Points.IsValidIndex(Point))
			{
				float Param = GetInputKeyValueAtSplinePoint(Point);

				SplineCurves.Scale.Points.Emplace(Param, FVector(WaterSplineDefaults.DefaultWidth, WaterSplineDefaults.DefaultDepth, 1.0f), FVector::ZeroVector, FVector::ZeroVector, CIM_CurveAuto);
				// Required for the direct write above. Unfortunate we do in loop body.
				SynchronizeSplines();
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			bAnythingChanged |= Metadata->PropagateDefaultValue(Point, PreviousWaterSplineDefaults, WaterSplineDefaults);

			FVector Scale = GetScaleAtSplinePoint(Point);
			float& DepthAtPoint = Metadata->Depth.Points[Point].OutVal;
			float& WidthAtPoint = Metadata->RiverWidth.Points[Point].OutVal;

			if (bFixOldProperties)
			{
				if (FMath::IsNearlyEqual(WidthAtPoint, 0.8f))
				{
					WidthAtPoint = WaterSplineDefaults.DefaultWidth;
				}

				if (FMath::IsNearlyZero(DepthAtPoint))
				{
					DepthAtPoint = WaterSplineDefaults.DefaultDepth;
				}
			}

			if (Scale.X != WidthAtPoint)
			{
				bAnythingChanged = true;
				// Set the splines local scale.x to the width and ensure it has some small positive value. (non-zero scale required for collision to work)
				Scale.X = WidthAtPoint = FMath::Max(WidthAtPoint, KINDA_SMALL_NUMBER);
			}

			if (Scale.Y != DepthAtPoint)
			{
				bAnythingChanged = true;

				// Set the splines local scale.x to the depth and ensure it has some small positive value. (non-zero scale required for collision to work)
				Scale.Y = DepthAtPoint = FMath::Max(DepthAtPoint, KINDA_SMALL_NUMBER);
			}

			SetScaleAtSplinePoint(Point, Scale, false);

			// #hack: temporarily clamp the tangents to a sensible range to prevent multiplicatively scaling until they hit infinity and cause a crash
			auto ClampVec3 = [](const FVector& Vec3, double Min, double Max) -> FVector {
				FVector Result;
				Result.X = FMath::Clamp(Vec3.X, Min, Max);
				Result.Y = FMath::Clamp(Vec3.Y, Min, Max);
				Result.Z = FMath::Clamp(Vec3.Z, Min, Max);
				return Result;
			};

			constexpr double MaxTangentValue = 1.e10L;

			const FVector ArriveTangent = GetArriveTangentAtSplinePoint(Point, ESplineCoordinateSpace::Local);
			const FVector LeaveTangent = GetLeaveTangentAtSplinePoint(Point, ESplineCoordinateSpace::Local);
			
			const FVector ClampedArriveTangent = ClampVec3(ArriveTangent, -MaxTangentValue, MaxTangentValue);
			const FVector ClampedLeaveTangent = ClampVec3(LeaveTangent, -MaxTangentValue, MaxTangentValue);

			// Only change if clamping is actually doing anything so that we don't unnecessarily force the point into custom tangent mode.
			if (!ArriveTangent.Equals(ClampedArriveTangent) || !LeaveTangent.Equals(ClampedLeaveTangent))
			{
				SetTangentsAtSplinePoint(Point, ArriveTangent, LeaveTangent, ESplineCoordinateSpace::Local, false);
			}
		}
	}

	if (bAnythingChanged)
	{
		UpdateSpline();
	}

	PreviousWaterSplineDefaults = WaterSplineDefaults;

	return bAnythingChanged;
}

#endif

