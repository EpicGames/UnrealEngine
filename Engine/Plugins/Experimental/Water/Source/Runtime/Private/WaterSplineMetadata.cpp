// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterSplineMetadata.h"
#include "WaterBodyActor.h"
#include "WaterSplineComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterSplineMetadata)


UWaterSplineMetadata::UWaterSplineMetadata(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, bShouldVisualizeWaterVelocity(false)
	, bShouldVisualizeRiverWidth(false)
	, bShouldVisualizeDepth(false)
#endif
{
}

#if WITH_EDITOR
bool UWaterSplineMetadata::CanEditRiverWidth() const
{
	AWaterBody* Body = GetTypedOuter<AWaterBody>();
	return Body && Body->GetWaterBodyType() == EWaterBodyType::River;
}

bool UWaterSplineMetadata::CanEditDepth() const
{
	AWaterBody* Body = GetTypedOuter<AWaterBody>();
	return Body && Body->GetWaterBodyType() != EWaterBodyType::Lake && Body->GetWaterBodyType() != EWaterBodyType::Ocean;
}

bool UWaterSplineMetadata::CanEditVelocity() const
{
	return true;
}

void UWaterSplineMetadata::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Keep the four parallel curves co-indexed with the owning spline when the edit landed on one of them
	// (or on a bulk/unknown change where the property is null, e.g. multi-property writes from SetEditorProperty(ies) and MCP tools that skip the UI).
	const FName ChangedPropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	const bool bIsParallelCurveProperty =
		ChangedPropertyName == NAME_None ||
		ChangedPropertyName == GET_MEMBER_NAME_CHECKED(UWaterSplineMetadata, Depth) ||
		ChangedPropertyName == GET_MEMBER_NAME_CHECKED(UWaterSplineMetadata, WaterVelocityScalar) ||
		ChangedPropertyName == GET_MEMBER_NAME_CHECKED(UWaterSplineMetadata, RiverWidth) ||
		ChangedPropertyName == GET_MEMBER_NAME_CHECKED(UWaterSplineMetadata, AudioIntensity);

	if (bIsParallelCurveProperty)
	{
		if (AWaterBody* Body = GetTypedOuter<AWaterBody>())
		{
			if (UWaterSplineComponent* OwningSpline = Body->GetWaterSpline())
			{
				Fixup(OwningSpline->GetNumberOfSplinePoints(), OwningSpline);
			}
		}
	}

	FOnWaterSplineMetadataChangedParams WaterSplineMetadataChangedParams(PropertyChangedEvent);
	WaterSplineMetadataChangedParams.WaterSplineMetadata = this;
	WaterSplineMetadataChangedParams.bUserTriggered = true;

	OnChangeMetadata.Broadcast(WaterSplineMetadataChangedParams);
}
#endif // WITH_EDITOR

void UWaterSplineMetadata::InsertPoint(int32 Index, float t, bool bClosedLoop)
{
	check(Index >= 0);

	Modify();

	int32 NumPoints = Depth.Points.Num();
	float InputKey = static_cast<float>(Index);

	if(Index >= NumPoints)
	{ 
		// Just add point to the end instead of trying to insert
		AddPoint(InputKey);
	}
	else
	{
		int32 PrevIndex = (bClosedLoop && Index == 0 ? NumPoints - 1 : Index - 1);
		bool bHasPrevIndex = (PrevIndex >= 0 && PrevIndex < NumPoints);

		float NewDepthVal = Depth.Points[Index].OutVal;
		float NewWidthVal = RiverWidth.Points[Index].OutVal;
		float NewVelocityVal = WaterVelocityScalar.Points[Index].OutVal;
		float NewAudioVal = AudioIntensity.Points[Index].OutVal;

		if (bHasPrevIndex)
		{
			float PrevDepthVal = Depth.Points[PrevIndex].OutVal;
			float PrevWidthVal = RiverWidth.Points[PrevIndex].OutVal;
			float PrevVelocityVal = WaterVelocityScalar.Points[PrevIndex].OutVal;
			float PrevAudioVal = AudioIntensity.Points[PrevIndex].OutVal;
			NewDepthVal = FMath::LerpStable(PrevDepthVal, NewDepthVal, t);
			NewWidthVal = FMath::LerpStable(PrevWidthVal, NewWidthVal, t);
			NewVelocityVal = FMath::LerpStable(PrevVelocityVal, NewVelocityVal, t);
			NewAudioVal = FMath::LerpStable(PrevAudioVal, NewAudioVal, t);
		}

		FInterpCurvePoint<float> NewDepth(InputKey, NewDepthVal);
		Depth.Points.Insert(NewDepth, Index);

		FInterpCurvePoint<float> NewLevel(InputKey, NewWidthVal);
		RiverWidth.Points.Insert(NewLevel, Index);

		FInterpCurvePoint<float> NewVelocity(InputKey, NewVelocityVal);
		WaterVelocityScalar.Points.Insert(NewVelocity, Index);

		FInterpCurvePoint<float> NewAudioIntensity(InputKey, NewAudioVal);
		AudioIntensity.Points.Insert(NewAudioIntensity, Index);

		for (int32 i = Index + 1; i < Depth.Points.Num(); ++i)
		{
			Depth.Points[i].InVal += 1.0f;
			RiverWidth.Points[i].InVal += 1.0f;
			WaterVelocityScalar.Points[i].InVal += 1.0f;
			AudioIntensity.Points[i].InVal += 1.0f;
		}
	}
}

void UWaterSplineMetadata::UpdatePoint(int32 Index, float t, bool bClosedLoop)
{
	const int32 NumPoints = Depth.Points.Num();
	check(Index >= 0 && Index < NumPoints);

	int32 PrevIndex = (bClosedLoop && Index == 0 ? NumPoints - 1 : Index - 1);
	int32 NextIndex = (bClosedLoop && Index + 1 > NumPoints ? 0 : Index + 1);
	
	bool bHasPrevIndex = (PrevIndex >= 0 && PrevIndex < NumPoints);
	bool bHasNextIndex = (NextIndex >= 0 && NextIndex < NumPoints);

	Modify();

	if (bHasPrevIndex && bHasNextIndex)
	{
		float PrevDepthVal = Depth.Points[PrevIndex].OutVal;
		float PrevWidthVal = RiverWidth.Points[PrevIndex].OutVal;
		float PrevVelocityVal = WaterVelocityScalar.Points[PrevIndex].OutVal;
		float PrevAudioIntensityVal = AudioIntensity.Points[PrevIndex].OutVal;

		float NextDepthVal = Depth.Points[NextIndex].OutVal;
		float NextWidthVal = RiverWidth.Points[NextIndex].OutVal;
		float NextVelocityVal = WaterVelocityScalar.Points[NextIndex].OutVal;
		float NextAudioIntensityVal = AudioIntensity.Points[NextIndex].OutVal;

		Depth.Points[Index].OutVal = FMath::LerpStable(PrevDepthVal, NextDepthVal, t);
		RiverWidth.Points[Index].OutVal = FMath::LerpStable(PrevWidthVal, NextWidthVal, t);
		WaterVelocityScalar.Points[Index].OutVal = FMath::LerpStable(PrevVelocityVal, NextVelocityVal, t);
		AudioIntensity.Points[Index].OutVal = FMath::LerpStable(PrevAudioIntensityVal, NextAudioIntensityVal, t);
	}
}

void UWaterSplineMetadata::AddPoint(float InputKey)
{
	Modify();
	
	float NewDepthVal = 0;
	float NewRiverWidthVal = 0;
	float NewWaterVelocityVal = 0;
	float NewAudioVal = 0;

	if (AWaterBody* Body = GetTypedOuter<AWaterBody>())
	{
		const FWaterSplineCurveDefaults& Defaults = Body->GetWaterSpline()->WaterSplineDefaults;
		NewDepthVal = Defaults.DefaultDepth;
		NewRiverWidthVal = Defaults.DefaultWidth;
		NewWaterVelocityVal = Defaults.DefaultVelocity;
		NewAudioVal = Defaults.DefaultAudioIntensity;
	}


	int Index = Depth.Points.Num() - 1;

	if (Index >= 0)
	{
		NewDepthVal = Depth.Points[Index].OutVal;
		NewRiverWidthVal = RiverWidth.Points[Index].OutVal;
		NewWaterVelocityVal = WaterVelocityScalar.Points[Index].OutVal;
		NewAudioVal = AudioIntensity.Points[Index].OutVal;
	}

	float NewInputKey = static_cast<float>(++Index);
	Depth.Points.Emplace(NewInputKey, NewDepthVal);
	RiverWidth.Points.Emplace(NewInputKey, NewRiverWidthVal);
	WaterVelocityScalar.Points.Emplace(NewInputKey, NewWaterVelocityVal);
	AudioIntensity.Points.Emplace(NewInputKey, NewAudioVal);
}

void UWaterSplineMetadata::RemovePoint(int32 Index)
{
	check(Index < Depth.Points.Num());

	Modify();
	Depth.Points.RemoveAt(Index);
	RiverWidth.Points.RemoveAt(Index);
	WaterVelocityScalar.Points.RemoveAt(Index);
	AudioIntensity.Points.RemoveAt(Index);

	for (int32 i = Index; i < Depth.Points.Num(); ++i)
	{
		Depth.Points[i].InVal -= 1.0f;
		RiverWidth.Points[i].InVal -= 1.0f;
		WaterVelocityScalar.Points[i].InVal -= 1.0f;
		AudioIntensity.Points[i].InVal -= 1.0f;
	}
}

void UWaterSplineMetadata::DuplicatePoint(int32 Index)
{
	check(Index < Depth.Points.Num());

	Modify();
	Depth.Points.Insert(FInterpCurvePoint<float>(Depth.Points[Index]), Index);
	RiverWidth.Points.Insert(FInterpCurvePoint<float>(RiverWidth.Points[Index]), Index);
	WaterVelocityScalar.Points.Insert(FInterpCurvePoint<float>(WaterVelocityScalar.Points[Index]), Index);
	AudioIntensity.Points.Insert(FInterpCurvePoint<float>(AudioIntensity.Points[Index]), Index);

	for (int32 i = Index + 1; i < Depth.Points.Num(); ++i)
	{
		Depth.Points[i].InVal += 1.0f;
		RiverWidth.Points[i].InVal += 1.0f;
		WaterVelocityScalar.Points[i].InVal += 1.0f;
		AudioIntensity.Points[i].InVal += 1.0f;
	}
}

void UWaterSplineMetadata::CopyPoint(const USplineMetadata* FromSplineMetadata, int32 FromIndex, int32 ToIndex)
{
	check(FromSplineMetadata != nullptr);

	if (const UWaterSplineMetadata* FromMetadata = Cast<UWaterSplineMetadata>(FromSplineMetadata))
	{
		check(ToIndex < Depth.Points.Num());
		check(FromIndex < FromMetadata->Depth.Points.Num());

		Modify();
		Depth.Points[ToIndex].OutVal = FromMetadata->Depth.Points[FromIndex].OutVal;
		RiverWidth.Points[ToIndex].OutVal = FromMetadata->RiverWidth.Points[FromIndex].OutVal;
		WaterVelocityScalar.Points[ToIndex].OutVal = FromMetadata->WaterVelocityScalar.Points[FromIndex].OutVal;
		AudioIntensity.Points[ToIndex].OutVal = FromMetadata->AudioIntensity.Points[FromIndex].OutVal;
	}
}

void UWaterSplineMetadata::Reset(int32 NumPoints)
{
	Modify();
	Depth.Points.Reset(NumPoints);
	RiverWidth.Points.Reset(NumPoints);
	WaterVelocityScalar.Points.Reset(NumPoints);
	AudioIntensity.Points.Reset(NumPoints);
}

#if WITH_EDITORONLY_DATA
template <class T>
void FixupCurve(FInterpCurve<T>& Curve, const T& DefaultValue, int32 NumPoints)
{
	// Fixup bad InVal values from when the add operation below used the wrong value
	for (int32 PointIndex = 0; PointIndex < Curve.Points.Num(); PointIndex++)
	{
		float InVal = PointIndex;
		Curve.Points[PointIndex].InVal = InVal;
	}

	while (Curve.Points.Num() < NumPoints)
	{
		// InVal is the point index which is ascending so use previous point plus one.
		float InVal = Curve.Points.Num() > 0 ? Curve.Points[Curve.Points.Num() - 1].InVal + 1.0f : 0.0f;
		Curve.Points.Add(FInterpCurvePoint<T>(InVal, DefaultValue));
	}
	
	if (Curve.Points.Num() > NumPoints)
	{
		Curve.Points.RemoveAt(NumPoints, Curve.Points.Num()-NumPoints);
	}
}

template <class T>
bool PropagateDefaultValueIfChanged(FInterpCurve<T>& Curve, const T& PreviousDefaultValue, const T& NewDefaultValue, int32 PointIndex)
{
	if (FMath::IsNearlyEqual(Curve.Points[PointIndex].OutVal, PreviousDefaultValue) && !FMath::IsNearlyEqual(PreviousDefaultValue, NewDefaultValue))
	{
		Curve.Points[PointIndex].OutVal = NewDefaultValue;
		return true;
	}

	return false;
}

#endif

void UWaterSplineMetadata::Fixup(int32 NumPoints, USplineComponent* SplineComp)
{
	const FWaterSplineCurveDefaults& Defaults = CastChecked<UWaterSplineComponent>(SplineComp)->WaterSplineDefaults;
#if WITH_EDITORONLY_DATA
	FixupCurve(Depth, Defaults.DefaultDepth, NumPoints);
	FixupCurve(RiverWidth, Defaults.DefaultWidth, NumPoints);
	FixupCurve(WaterVelocityScalar, Defaults.DefaultVelocity, NumPoints);
	FixupCurve(AudioIntensity, Defaults.DefaultAudioIntensity, NumPoints);
#endif
}

bool UWaterSplineMetadata::PropagateDefaultValue(int32 PointIndex, const FWaterSplineCurveDefaults& PreviousDefaults, const FWaterSplineCurveDefaults& NewDefaults)
{
	bool bAnythingChanged = false;
#if WITH_EDITORONLY_DATA
	bAnythingChanged |= PropagateDefaultValueIfChanged(Depth, PreviousDefaults.DefaultDepth, NewDefaults.DefaultDepth, PointIndex);
	bAnythingChanged |= PropagateDefaultValueIfChanged(RiverWidth, PreviousDefaults.DefaultWidth, NewDefaults.DefaultWidth, PointIndex);
	bAnythingChanged |= PropagateDefaultValueIfChanged(WaterVelocityScalar, PreviousDefaults.DefaultVelocity, NewDefaults.DefaultVelocity, PointIndex);
	bAnythingChanged |= PropagateDefaultValueIfChanged(AudioIntensity, PreviousDefaults.DefaultAudioIntensity, NewDefaults.DefaultAudioIntensity, PointIndex);
#endif

	return bAnythingChanged;
}


