// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterSubsystem.h"
#include "BuoyancyManager.h"
#include "Engine/StaticMesh.h"
#include "EngineUtils.h"
#include "LandscapeSubsystem.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "SceneView.h"
#include "UObject/ConstructorHelpers.h"
#include "WaterBodyActor.h"
#include "WaterBodyExclusionVolume.h"
#include "WaterBodyIslandActor.h"
#include "WaterModule.h"
#include "WaterRuntimeSettings.h"
#include "WaterTerrainComponent.h"
#include "WaterUtils.h"
#include "WaterViewExtension.h"
#include "Algo/MaxElement.h"
#include "Algo/RemoveIf.h"
#include "Algo/AnyOf.h"
#include "VisualLogger/VisualLogger.h"

#if WITH_EDITOR
#include "WaterZoneActorDesc.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionHelpers.h"
extern UNREALED_API UEditorEngine* GEditor;
#else
#include "BuoyancyTypes.h"
#endif // WITH_EDITOR

#include "LandscapeComponent.h"
#include "LandscapeProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterSubsystem)

// ----------------------------------------------------------------------------------

#define LOCTEXT_NAMESPACE "WaterSubsystem"

// ----------------------------------------------------------------------------------

DECLARE_CYCLE_STAT(TEXT("IsUnderwater Test"), STAT_WaterIsUnderwater, STATGROUP_Water);

// ----------------------------------------------------------------------------------

// General purpose CVars:
TAutoConsoleVariable<int32> CVarWaterEnabled(
	TEXT("r.Water.Enabled"),
	1,
	TEXT("If all water rendering is enabled or disabled"),
	ECVF_RenderThreadSafe
);

static int32 FreezeWaves = 0;
static FAutoConsoleVariableRef CVarFreezeWaves(
	TEXT("r.Water.FreezeWaves"),
	FreezeWaves,
	TEXT("Freeze time for waves if non-zero"),
	ECVF_Cheat
);

static float OverrideWavesTime = -1.f;
static FAutoConsoleVariableRef CVarOverrideWavesTime(
	TEXT("r.Water.OverrideWavesTime"),
	OverrideWavesTime,
	TEXT("Forces the time used for waves if >= 0.0"),
	ECVF_Cheat
);

// Underwater post process CVars : 
static int32 EnableUnderwaterPostProcess = 1;
static FAutoConsoleVariableRef CVarEnableUnderwaterPostProcess(
	TEXT("r.Water.EnableUnderwaterPostProcess"),
	EnableUnderwaterPostProcess,
	TEXT("Controls whether the underwater post process is enabled"),
	ECVF_Scalability
);

static int32 VisualizeActiveUnderwaterPostProcess = 0;
static FAutoConsoleVariableRef CVarVisualizeUnderwaterPostProcess(
	TEXT("r.Water.VisualizeActiveUnderwaterPostProcess"),
	VisualizeActiveUnderwaterPostProcess,
	TEXT("Shows which water body is currently being picked up for underwater post process"),
	ECVF_Default
);

static int32 EnableUnderwaterPostProcessVisualLogger = 0;
static FAutoConsoleVariableRef CVarEnableUnderwaterPostProcessVisualLogger(
	TEXT("r.Water.EnableUnderwaterPostProcessVisualLogger"),
	EnableUnderwaterPostProcessVisualLogger,
	TEXT("Enables underwater post process collision detection Visual Logger code"),
	ECVF_Default
);

static float UnderwaterCollisionTraceDistance = 100.f;
static FAutoConsoleVariableRef CVarUnderwaterCollisionTraceDistance(
	TEXT("r.Water.UnderwaterCollisionTraceDistance"),
	UnderwaterCollisionTraceDistance,
	TEXT("Underwater post processing collision trace distance. Default is inflated to account for waves which will not be hit by trace"),
	ECVF_Scalability
);

static float UnderwaterCollisionPreciseTraceDistance = 10.f;
static FAutoConsoleVariableRef CVarUnderwaterCollisionPreciseTraceDistance(
	TEXT("r.Water.UnderwaterCollisionPreciseTraceDistance"),
	UnderwaterCollisionPreciseTraceDistance,
	TEXT("Underwater precise collision trace distance. Verifies precise overlap between camera and volume when camera is underneath the water collision impact location"),
	ECVF_Scalability
);

// Shallow water CVars : 
static int32 ShallowWaterSim = 1;
static FAutoConsoleVariableRef CVarShallowWaterSim(
	TEXT("r.Water.EnableShallowWaterSimulation"),
	ShallowWaterSim,
	TEXT("Controls whether the shallow water fluid sim is enabled"),
	ECVF_Scalability
);

static int32 ShallowWaterSimulationMaxDynamicForces = 6;
static FAutoConsoleVariableRef CVarShallowWaterSimulationMaxDynamicForces(
	TEXT("r.Water.ShallowWaterMaxDynamicForces"),
	ShallowWaterSimulationMaxDynamicForces,
	TEXT("Max number of dynamic forces that will be registered with sim at a time."),
	ECVF_Scalability
);

static int32 ShallowWaterSimulationMaxImpulseForces = 3;
static FAutoConsoleVariableRef CVarShallowWaterSimulationMaxImpulseForces(
	TEXT("r.Water.ShallowWaterMaxImpulseForces"),
	ShallowWaterSimulationMaxImpulseForces,
	TEXT("Max number of impulse forces that will be registered with sim at a time."),
	ECVF_Scalability
);

static int32 ShallowWaterSimulationRenderTargetSize = 1024;
static FAutoConsoleVariableRef CVarShallowWaterSimulationRenderTargetSize(
	TEXT("r.Water.ShallowWaterRenderTargetSize"),
	ShallowWaterSimulationRenderTargetSize,
	TEXT("Size for square shallow water fluid sim render target. Effective dimensions are SizexSize"),
	ECVF_Scalability
);

extern TAutoConsoleVariable<int32> CVarWaterMeshEnabled;
extern TAutoConsoleVariable<int32> CVarWaterMeshEnableRendering;


// ----------------------------------------------------------------------------------

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
/** Debug-only struct for displaying some information about which post process material is being used : */
struct FUnderwaterPostProcessDebugInfo
{
	TArray<TWeakObjectPtr<UWaterBodyComponent>> OverlappedWaterBodyComponents;
	TWeakObjectPtr<UWaterBodyComponent> ActiveWaterBodyComponent;
	FWaterBodyQueryResult ActiveWaterBodyQueryResult;
	bool bIsPostProcessingEnabled = false;
	bool bIsUnderwaterPostProcessEnabled = false;
	bool bIsPathTracingEnabled = false;
};
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

// ----------------------------------------------------------------------------------

#if WITH_EDITOR

bool UWaterSubsystem::bAllowWaterSubsystemOnPreviewWorld = false;

#endif // WITH_EDITOR

// ----------------------------------------------------------------------------------

UWaterSubsystem::UWaterSubsystem()
{
	SmoothedWorldTimeSeconds = 0.f;
	NonSmoothedWorldTimeSeconds = 0.f;
	PrevWorldTimeSeconds = 0.f;
	bUnderWaterForAudio = false;
	bPauseWaveTime = false;

	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinderOptional<UStaticMesh> LakeMesh;
		ConstructorHelpers::FObjectFinderOptional<UStaticMesh> RiverMesh;

		FConstructorStatics()
			: LakeMesh(TEXT("/Water/Meshes/LakeMesh.LakeMesh"))
			, RiverMesh(TEXT("/Water/Meshes/RiverMesh.RiverMesh"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	DefaultLakeMesh = ConstructorStatics.LakeMesh.Get();
	DefaultRiverMesh = ConstructorStatics.RiverMesh.Get();
}

UWaterSubsystem* UWaterSubsystem::GetWaterSubsystem(const UWorld* InWorld)
{
	if (InWorld)
	{
		return InWorld->GetSubsystem<UWaterSubsystem>();
	}

	return nullptr;
}

FWaterBodyManager* UWaterSubsystem::GetWaterBodyManager(const UWorld* InWorld)
{
	if (UWaterSubsystem* Subsystem = GetWaterSubsystem(InWorld))
	{
		return &Subsystem->WaterBodyManager;
	}

	return nullptr;
}

FWaterViewExtension* UWaterSubsystem::GetWaterViewExtension(const UWorld* InWorld)
{
	if (FWaterBodyManager* Manager = GetWaterBodyManager(InWorld))
	{
		return Manager->GetWaterViewExtension();
	}
	return {};
}

TWeakPtr<FWaterViewExtension, ESPMode::ThreadSafe> UWaterSubsystem::GetWaterViewExtensionWeakPtr(const UWorld* InWorld)
{
	if (FWaterBodyManager* Manager = GetWaterBodyManager(InWorld))
	{
		return Manager->GetWaterViewExtensionWeakPtr();
	}
	return {};
}

void UWaterSubsystem::Tick(float DeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWaterSubsystem::Tick);

	Super::Tick(DeltaTime);

	check(GetWorld() != nullptr);
	if (FreezeWaves == 0 && bPauseWaveTime == false)
	{
		NonSmoothedWorldTimeSeconds += DeltaTime;
	}

	float MPCTime = GetWaterTimeSeconds();
	SetMPCTime(MPCTime, PrevWorldTimeSeconds);
	PrevWorldTimeSeconds = MPCTime;

	WaterBodyManager.ForEachWaterZone([](AWaterZone* WaterZoneActor)
	{
		WaterZoneActor->Update();
		return true;
	});

	if (!bUnderWaterForAudio && CachedDepthUnderwater > 0.0f)
	{
		bUnderWaterForAudio = true;
		OnCameraUnderwaterStateChanged.Broadcast(bUnderWaterForAudio, CachedDepthUnderwater);
	}
	else if (bUnderWaterForAudio && CachedDepthUnderwater <= 0.0f)
	{
		bUnderWaterForAudio = false;
		OnCameraUnderwaterStateChanged.Broadcast(bUnderWaterForAudio, CachedDepthUnderwater);
	}
}

TStatId UWaterSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UWaterSubsystem, STATGROUP_Tickables);
}

bool UWaterSubsystem::DoesSupportWorldType(EWorldType::Type WorldType) const
{
#if WITH_EDITOR
	// In editor, don't let preview worlds instantiate a water subsystem (except if explicitly allowed by a tool that requested it by setting bAllowWaterSubsystemOnPreviewWorld)
	if (WorldType == EWorldType::EditorPreview)
	{
		return bAllowWaterSubsystemOnPreviewWorld;
	}
#endif // WITH_EDITOR

	return WorldType == EWorldType::Game || WorldType == EWorldType::Editor || WorldType == EWorldType::PIE;
}

void UWaterSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	UWorld* World = GetWorld();
	check(World != nullptr);

	WaterBodyManager.Initialize(World);

	bUsingSmoothedTime = false;
	FConsoleVariableDelegate NotifyWaterScalabilityChanged = FConsoleVariableDelegate::CreateUObject(this, &UWaterSubsystem::NotifyWaterScalabilityChangedInternal);
	CVarShallowWaterSim->SetOnChangedCallback(NotifyWaterScalabilityChanged);
	CVarShallowWaterSimulationRenderTargetSize->SetOnChangedCallback(NotifyWaterScalabilityChanged);

	FConsoleVariableDelegate NotifyWaterVisibilityChanged = FConsoleVariableDelegate::CreateUObject(this, &UWaterSubsystem::NotifyWaterVisibilityChangedInternal);
	CVarWaterEnabled->SetOnChangedCallback(NotifyWaterVisibilityChanged);
	CVarWaterMeshEnabled->SetOnChangedCallback(NotifyWaterVisibilityChanged);
	CVarWaterMeshEnableRendering->SetOnChangedCallback(NotifyWaterVisibilityChanged);

#if WITH_EDITOR
	GetDefault<UWaterRuntimeSettings>()->OnSettingsChange.AddUObject(this, &UWaterSubsystem::ApplyRuntimeSettings);
#endif //WITH_EDITOR
	ApplyRuntimeSettings(GetDefault<UWaterRuntimeSettings>(), EPropertyChangeType::ValueSet);

	UnderwaterPostProcessVolume = NewObject<UUnderwaterPostProcessVolume>(this);

	World->OnBeginPostProcessSettings.AddUObject(this, &UWaterSubsystem::ComputeUnderwaterPostProcess);
	World->AddPostProcessVolume(UnderwaterPostProcessVolume);
	{
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		SpawnInfo.ObjectFlags = RF_Transient;

#if WITH_EDITOR
		// The buoyancy manager should be a subsytem really, but for now, just hide it from the outliner : 
		SpawnInfo.bHideFromSceneOutliner = true;
#endif //WITH_EDITOR

		// Store the buoyancy manager we create for future use.
		BuoyancyManager = World->SpawnActor<ABuoyancyManager>(SpawnInfo);
	}
}

void UWaterSubsystem::PostInitialize()
{
	Super::PostInitialize();

	UWorld* World = GetWorld();
	check(World);

#if WITH_EDITOR
	if (GEngine)
	{
		GEngine->OnActorMoved().AddUObject(this, &UWaterSubsystem::OnActorMoved);
	}
#endif // WITH_EDITOR
}

void UWaterSubsystem::Deinitialize()
{
	UWorld* World = GetWorld();
	check(World != nullptr);

#if WITH_EDITOR
	if (GEngine)
	{
		GEngine->OnActorMoved().RemoveAll(this);
	}
#endif // WITH_EDITOR

	if (OnMarkRenderStateDirtyHandle.IsValid())
	{
		UActorComponent::MarkRenderStateDirtyEvent.Remove(OnMarkRenderStateDirtyHandle);
	}

	FConsoleVariableDelegate NullCallback;
	CVarShallowWaterSimulationRenderTargetSize->SetOnChangedCallback(NullCallback);
	CVarShallowWaterSim->SetOnChangedCallback(NullCallback);
	CVarWaterEnabled->SetOnChangedCallback(NullCallback);
	CVarWaterMeshEnabled->SetOnChangedCallback(NullCallback);
	CVarWaterMeshEnableRendering->SetOnChangedCallback(NullCallback);

	World->OnBeginPostProcessSettings.RemoveAll(this);
	World->RemovePostProcessVolume(UnderwaterPostProcessVolume);

	WaterBodyManager.Deinitialize();

#if WITH_EDITOR
	GetDefault<UWaterRuntimeSettings>()->OnSettingsChange.RemoveAll(this);
#endif //WITH_EDITOR

	Super::Deinitialize();
}

void UWaterSubsystem::ApplyRuntimeSettings(const UWaterRuntimeSettings* Settings, EPropertyChangeType::Type ChangeType)
{
	UWorld* World = GetWorld();
	check(World != nullptr);
	UnderwaterTraceChannel = Settings->CollisionChannelForWaterTraces;
	MaterialParameterCollection = Settings->MaterialParameterCollection.LoadSynchronous();

#if WITH_EDITOR
	// Update sprites since we may have changed the sprite Z offset setting.

	WaterBodyManager.ForEachWaterBodyComponent([](UWaterBodyComponent* Component)
	{
		Component->UpdateWaterSpriteComponent();
		return true;
	});

	for (TActorIterator<AWaterBodyIsland> ActorItr(World); ActorItr; ++ActorItr)
	{
		(*ActorItr)->UpdateActorIcon();
	}

	for (TActorIterator<AWaterBodyExclusionVolume> ActorItr(World); ActorItr; ++ActorItr)
	{
		(*ActorItr)->UpdateActorIcon();
	}
#endif // WITH_EDITOR
}


void UWaterSubsystem::OnMarkRenderStateDirty(UActorComponent& Component)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWaterSubsystem::OnMarkRenderStateDirty);

	const AActor* ComponentOwner = Component.GetOwner();

	if (WaterTerrainActors.Find(ComponentOwner) != nullptr)
	{
		OnWaterTerrainActorChanged(ComponentOwner);
	}
}

void UWaterSubsystem::OnWaterTerrainActorChanged(const AActor* TerrainActor)
{
	TArray<TWeakObjectPtr<UWaterTerrainComponent>, TInlineAllocator<4>> WaterTerrainComponentPtrs;
	WaterTerrainActors.MultiFind(TerrainActor, WaterTerrainComponentPtrs);

	check(TerrainActor != nullptr && TerrainActor->GetWorld() == GetWorld());

	for (TWeakObjectPtr<UWaterTerrainComponent> WaterTerrainComponentPtr : WaterTerrainComponentPtrs)
	{
		 if (!WaterTerrainComponentPtr.IsValid())
		 {
			 continue;
		 }

		 if (const UWaterTerrainComponent* WaterTerrainComponent = WaterTerrainComponentPtr.Get())
		 {
			 const FBox2D TerrainBounds = WaterTerrainComponent->GetTerrainBounds();
			 for (AWaterZone* WaterZone : TActorRange<AWaterZone>(GetWorld()))
			 {
				 if (WaterTerrainComponent->AffectsWaterZone(WaterZone))
				 {
					  WaterZone->MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture, TerrainBounds, TerrainActor);
				 }
			 }
		 }
		 
	}
}

#if WITH_EDITOR
void UWaterSubsystem::OnActorMoved(AActor* MovedActor)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWaterSubsystem::OnActorMoved);

	if (WaterTerrainActors.Find(MovedActor) != nullptr)
	{
		OnWaterTerrainActorChanged(MovedActor);
	}
}
#endif // WITH_EDITOR

bool UWaterSubsystem::IsShallowWaterSimulationEnabled() const
{
	return ShallowWaterSim != 0;
}

bool UWaterSubsystem::IsUnderwaterPostProcessEnabled() const
{
	return EnableUnderwaterPostProcess != 0;
}

float UWaterSubsystem::GetUnderwaterCollisionTraceDistance()
{
	return UnderwaterCollisionTraceDistance;
}

float UWaterSubsystem::GetUnderwaterPreciseTraceDistance()
{
	return UnderwaterCollisionPreciseTraceDistance;
}

int32 UWaterSubsystem::GetShallowWaterMaxDynamicForces()
{
	return ShallowWaterSimulationMaxDynamicForces;
}

int32 UWaterSubsystem::GetShallowWaterMaxImpulseForces()
{
	return ShallowWaterSimulationMaxImpulseForces;
}

int32 UWaterSubsystem::GetShallowWaterSimulationRenderTargetSize()
{
	return ShallowWaterSimulationRenderTargetSize;
}

bool UWaterSubsystem::IsWaterRenderingEnabled() const
{
	return FWaterUtils::IsWaterEnabled(/*bIsRenderThread = */ false);
}

float UWaterSubsystem::GetWaterTimeSeconds() const
{
	if (OverrideWavesTime >= 0.0f)
	{
		return OverrideWavesTime;
	}

	if (UWorld* World = GetWorld())
	{
		if (World->IsGameWorld() && bUsingSmoothedTime)
		{
			return GetSmoothedWorldTimeSeconds();
		}
	}
	return NonSmoothedWorldTimeSeconds;
}

float UWaterSubsystem::GetSmoothedWorldTimeSeconds() const
{
	return bUsingOverrideWorldTimeSeconds ? OverrideWorldTimeSeconds : SmoothedWorldTimeSeconds;
}

void UWaterSubsystem::PrintToWaterLog(const FString& Message, bool bWarning)
{
	if (bWarning)
	{
		UE_LOGF(LogWater, Warning, "%ls", *Message);
	}
	else
	{
		UE_LOGF(LogWater, Log, "%ls", *Message);
	}
}

void UWaterSubsystem::SetSmoothedWorldTimeSeconds(float InTime)
{
	bUsingSmoothedTime = true;
	if (FreezeWaves == 0)
	{
		SmoothedWorldTimeSeconds = InTime;
	}
}


void UWaterSubsystem::SetOverrideSmoothedWorldTimeSeconds(float InTime)
{
	OverrideWorldTimeSeconds = InTime;
}

void UWaterSubsystem::SetShouldOverrideSmoothedWorldTimeSeconds(bool bOverride)
{
	bUsingOverrideWorldTimeSeconds = bOverride;
}

void UWaterSubsystem::SetShouldPauseWaveTime(bool bInPauseWaveTime)
{
	bPauseWaveTime = bInPauseWaveTime;
}

void UWaterSubsystem::SetOceanFloodHeight(float InFloodHeight)
{
	if (UWorld* World = GetWorld())
	{
		const float ClampedFloodHeight = FMath::Max(0.0f, InFloodHeight);

		if (FloodHeight != ClampedFloodHeight)
		{
			FloodHeight = ClampedFloodHeight;
			MarkAllWaterZonesForRebuild(EWaterZoneRebuildFlags::All, /* DebugRequestingObject = */ this);

			// the ocean body is dynamic and needs to be readjusted when the flood height changes : 
			if (OceanBodyComponent.IsValid())
			{
				OceanBodyComponent->SetHeightOffset(InFloodHeight);
			}

			WaterBodyManager.ForEachWaterBodyComponent([this](UWaterBodyComponent* WaterBodyComponent)
			{
				WaterBodyComponent->UpdateMaterialInstances();
				return true;
			});
		}
	}
}

float UWaterSubsystem::GetOceanBaseHeight() const
{
	if (OceanBodyComponent.IsValid())
	{
		return OceanBodyComponent->GetComponentLocation().Z;
	}

	return TNumericLimits<float>::Lowest();
}

void UWaterSubsystem::MarkAllWaterZonesForRebuild(EWaterZoneRebuildFlags RebuildFlags, const UObject* DebugRequestingObject)
{
	if (UWorld* World = GetWorld())
	{
		for (AWaterZone* WaterZone : TActorRange<AWaterZone>(World))
		{
			WaterZone->MarkForRebuild(RebuildFlags, DebugRequestingObject);
		}
	}
}

void UWaterSubsystem::MarkWaterZonesInRegionForRebuild(const FBox2D& InUpdateRegion, EWaterZoneRebuildFlags InRebuildFlags, const UObject* DebugRequestingObject)
{
	if (UWorld* World = GetWorld())
	{
		for (AWaterZone* WaterZone : TActorRange<AWaterZone>(World))
		{
			const FBox2D WaterZoneBounds = WaterZone->GetZoneBounds2D();

			if (WaterZoneBounds.Intersect(InUpdateRegion))
			{
				WaterZone->MarkForRebuild(InRebuildFlags, InUpdateRegion, DebugRequestingObject);
			}
		}
	}
}

TSoftObjectPtr<AWaterZone> UWaterSubsystem::FindWaterZone(const UWorld* World, const FBox2D& Bounds, const TSoftObjectPtr<const ULevel> PreferredLevel)
{
	if (!World)
	{
		return {};
	}

	// Score each overlapping water zone and then pick the best.
	TMap<TSoftObjectPtr<AWaterZone>, int32> ViableZones;

#if WITH_EDITOR
	// Within the editor, we also want to check unloaded actors to ensure that the water body has serialized the best possible water zone, rather than just looking through what might be loaded now.
	if (GEditor && !World->IsGameWorld())
	{
		if (UWorldPartition* WorldPartition = World->GetWorldPartition(); WorldPartition && WorldPartition->IsInitialized())
		{
			const FBox Bounds3D(FVector(Bounds.Min.X, Bounds.Min.Y, -HALF_WORLD_MAX), FVector(Bounds.Max.X, Bounds.Max.Y, HALF_WORLD_MAX));
			FWorldPartitionHelpers::ForEachIntersectingActorDescInstance<AWaterZone>(WorldPartition, Bounds3D, [&Bounds, &ViableZones](const FWorldPartitionActorDescInstance* ActorDescInstance)
			{
				FWaterZoneActorDesc* WaterZoneActorDesc = (FWaterZoneActorDesc*)ActorDescInstance->GetActorDesc();
				ViableZones.Emplace(ActorDescInstance->GetActorSoftPath(), WaterZoneActorDesc->GetOverlapPriority());
				return true;
			});
		}
	}
#endif // WITH_EDITOR

	for (AWaterZone* WaterZone : TActorRange<AWaterZone>(World, AWaterZone::StaticClass(), EActorIteratorFlags::SkipPendingKill))
	{
		const FBox2D WaterZoneBounds = WaterZone->GetZoneBounds2D();

		if (Bounds.Intersect(WaterZoneBounds))
		{
			ViableZones.Emplace(WaterZone, WaterZone->GetOverlapPriority());
		}
	}

	if (ViableZones.Num() == 0)
	{
		return {};
	}

	// Return best match in PreferredLevel if there is a match
	if (!PreferredLevel.IsNull() && ViableZones.Num() > 1)
	{
		TSoftObjectPtr<AWaterZone> PreferredZone;
		int32 PreferredZoneMax = INT32_MIN;

		FNameBuilder ParentPath;
		PreferredLevel.ToSoftObjectPath().ToString(ParentPath);
		FStringView ParentPathView(ParentPath);

		for (const TPair<TSoftObjectPtr<AWaterZone>, int32>& It : ViableZones)
		{
			if (PreferredZone.IsNull() || (It.Value > PreferredZoneMax))
			{
				const TSoftObjectPtr<AWaterZone>& WaterZoneSoftPath = It.Key;
				FNameBuilder ActorPath;
				WaterZoneSoftPath.ToSoftObjectPath().ToString(ActorPath);
				if (ActorPath.ToView().StartsWith(ParentPathView))
				{
					PreferredZone = WaterZoneSoftPath;
					PreferredZoneMax = It.Value;
				}
			}
		}

		if (!PreferredZone.IsNull())
		{
			return PreferredZone;
		}
	}

	return Algo::MaxElementBy(ViableZones, [](const TPair<TSoftObjectPtr<AWaterZone>, int32>& A) { return A.Value; })->Key;
}

TSoftObjectPtr<AWaterZone> UWaterSubsystem::FindWaterZone(const FBox2D& Bounds, const TSoftObjectPtr<const ULevel> PreferredLevel) const
{
	return FindWaterZone(GetWorld(), Bounds, PreferredLevel);
}

void UWaterSubsystem::RegisterWaterTerrainComponent(UWaterTerrainComponent* InWaterTerrainComponent)
{
	check(InWaterTerrainComponent);
	if (const AActor* TerrainActor = InWaterTerrainComponent->GetOwner())
	{
		 WaterTerrainActors.Add(TerrainActor,  InWaterTerrainComponent);
	}

	if (!WaterTerrainActors.IsEmpty() && !OnMarkRenderStateDirtyHandle.IsValid())
	{
		OnMarkRenderStateDirtyHandle = UActorComponent::MarkRenderStateDirtyEvent.AddUObject(this, &UWaterSubsystem::OnMarkRenderStateDirty);
	}
}

void UWaterSubsystem::UnregisterWaterTerrainComponent(UWaterTerrainComponent* InWaterTerrainComponent)
{
	check(InWaterTerrainComponent);
	if (const AActor* TerrainActor = InWaterTerrainComponent->GetOwner())
	{
		WaterTerrainActors.RemoveSingle(TerrainActor, InWaterTerrainComponent);
	}

	if (WaterTerrainActors.IsEmpty() && ensure(OnMarkRenderStateDirtyHandle.IsValid()))
	{
		UActorComponent::MarkRenderStateDirtyEvent.Remove(OnMarkRenderStateDirtyHandle);
		OnMarkRenderStateDirtyHandle.Reset();
	}
}

void UWaterSubsystem::GetWaterTerrainComponents(TArray<UWaterTerrainComponent*>& OutWaterTerrainComponents) const
{
	OutWaterTerrainComponents.Empty(WaterTerrainActors.Num());
	for (const TTuple<const AActor*, TWeakObjectPtr<UWaterTerrainComponent>>& Pair : WaterTerrainActors)
	{
		if (UWaterTerrainComponent* WaterTerrainComponent = Pair.Value.Get())
		{
			OutWaterTerrainComponents.Add(WaterTerrainComponent);
		}
	}
}

void UWaterSubsystem::NotifyWaterScalabilityChangedInternal(IConsoleVariable* CVar)
{
	OnWaterScalabilityChanged.Broadcast();
}

void UWaterSubsystem::NotifyWaterVisibilityChangedInternal(IConsoleVariable* CVar)
{
	// Water body visibility depends on various CVars. All need to update the visibility in water body components : 
	WaterBodyManager.ForEachWaterBodyComponent([](UWaterBodyComponent* WaterBodyComponent)
	{
		WaterBodyComponent->UpdateVisibility();
		return true;
	});
}

struct FWaterBodyPostProcessQuery
{
	FWaterBodyPostProcessQuery(UWaterBodyComponent& InWaterBodyComponent, const FVector& InWorldLocation, const FWaterBodyQueryResult& InQueryResult)
		: WaterBodyComponent(InWaterBodyComponent)
		, WorldLocation(InWorldLocation)
		, QueryResult(InQueryResult)
	{}

	UWaterBodyComponent& WaterBodyComponent;
	FVector WorldLocation;
	FWaterBodyQueryResult QueryResult;
};

static bool GetWaterBodyDepthUnderwater(const FWaterBodyPostProcessQuery& InQuery, float& OutDepthUnderwater)
{
	// Account for max possible wave height
	const FWaveInfo& WaveInfo = InQuery.QueryResult.GetWaveInfo();
	const float ZFudgeFactor = FMath::Max(WaveInfo.MaxHeight, WaveInfo.AttenuationFactor * 10.0f);
	const FBox BoxToCheckAgainst = FBox::BuildAABB(InQuery.WorldLocation, FVector(10, 10, ZFudgeFactor));

	float ImmersionDepth = InQuery.QueryResult.GetImmersionDepth();
	check(!InQuery.QueryResult.IsInExclusionVolume());
	if ((ImmersionDepth >= 0.0f) || BoxToCheckAgainst.IsInsideOrOn(InQuery.QueryResult.GetWaterSurfaceLocation()))
	{
		OutDepthUnderwater = ImmersionDepth;
		return true;
	}

	OutDepthUnderwater = 0.0f;
	return false;
}

void UWaterSubsystem::ComputeUnderwaterPostProcess(FVector ViewLocation, FSceneView* SceneView)
{
	SCOPE_CYCLE_COUNTER(STAT_WaterIsUnderwater);

	UWorld* World = GetWorld();
	const bool bIsPostProcessingEnabled = (SceneView->Family->EngineShowFlags.PostProcessing != 0);
	const bool bIsUnderwaterPostProcessEnabled = IsUnderwaterPostProcessEnabled();
	const bool bIsPathTracingEnabled = (SceneView->Family->EngineShowFlags.PathTracing != 0);

	const float PrevDepthUnderwater = CachedDepthUnderwater;
	CachedDepthUnderwater = -1;

	// Set all that needs to be set before an eventual early-out
	UnderwaterPostProcessVolume->PostProcessProperties.bIsEnabled = false;
	UnderwaterPostProcessVolume->PostProcessProperties.Settings = nullptr;
	SceneView->UnderwaterDepth = CachedDepthUnderwater;
	SceneView->WaterIntersection = EViewWaterIntersection::OutsideWater;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FUnderwaterPostProcessDebugInfo UnderwaterPostProcessDebugInfo;
	UnderwaterPostProcessDebugInfo.bIsPostProcessingEnabled = bIsPostProcessingEnabled;
	UnderwaterPostProcessDebugInfo.bIsUnderwaterPostProcessEnabled = bIsUnderwaterPostProcessEnabled;
	UnderwaterPostProcessDebugInfo.bIsPathTracingEnabled = bIsPathTracingEnabled;
	ON_SCOPE_EXIT { ShowOnScreenDebugInfo(ViewLocation, UnderwaterPostProcessDebugInfo); };
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	if ((World == nullptr) || !bIsPostProcessingEnabled || !bIsUnderwaterPostProcessEnabled || bIsPathTracingEnabled)
	{
		return;
	}

	// Compute distance from view origin to the corner of the near plane. This distance needs to be taken into account when computing whether the view intersects the water surface.
	const FVector4f NearPlaneCornerViewSpace = FVector4f(SceneView->ViewMatrices.GetClipToView().TransformFVector4(FVector4(1.0f, 1.0f, 1.0f, 1.0f)));
	const float ViewToNearPlaneCornerDistance = FVector2f(NearPlaneCornerViewSpace / NearPlaneCornerViewSpace.W).Length();
	bool bAnyDefinitelyUnderwater = false;
	bool bAnyPossiblyUnderwater = false;
	bool bUnderwaterForPostProcess = false;

	const bool bIsVisualLoggerEnabled = EnableUnderwaterPostProcessVisualLogger != 0;
	const float TraceDistance = GetUnderwaterCollisionTraceDistance();
	const float PreciseTraceDistance = GetUnderwaterPreciseTraceDistance();

	// Always force simple collision traces
	static FCollisionQueryParams TraceSimple(SCENE_QUERY_STAT(DefaultQueryParam), false);

	TArray<FHitResult> Hits;
	TArray<FHitResult> PreciseViewHits;
	TArray<FWaterBodyPostProcessQuery, TInlineAllocator<4>> WaterBodyQueriesToProcess;
	const bool bWorldHasWater  = WaterBodyManager.HasAnyWaterBodies();
	if (bWorldHasWater && World->SweepMultiByChannel(Hits, ViewLocation, ViewLocation + FVector(0, 0, TraceDistance), FQuat::Identity, UnderwaterTraceChannel, FCollisionShape::MakeSphere(TraceDistance), TraceSimple))
	{
		if (Hits.Num() > 1)
		{
			// Prepass to remove non-waterbody elements
			Hits.SetNum(Algo::RemoveIf(Hits, [](const FHitResult& A)
			{
				return A.HitObjectHandle.FetchActor<AWaterBody>() == nullptr;
			}));
			
			// Sort hits based on their water priority for rendering since we should prioritize evaluating waves in the order those waves will be considered for rendering. 
			Hits.Sort([](const FHitResult& A, const FHitResult& B)
			{
				const AWaterBody* ABody = A.HitObjectHandle.FetchActor<AWaterBody>();
				const AWaterBody* BBody = B.HitObjectHandle.FetchActor<AWaterBody>();

				// If both water bodies either have waves or both don't have waves, use the overlap priority to determine which to use, since in this case we need to respect the surface waves
				if (ABody->GetWaterBodyComponent()->HasWaves() == BBody->GetWaterBodyComponent()->HasWaves())
				{
					const int32 APriority = ABody->GetWaterBodyComponent()->GetOverlapMaterialPriority();
					const int32 BPriority = BBody->GetWaterBodyComponent()->GetOverlapMaterialPriority();
					return APriority > BPriority;
				}

				// Otherwise, prefer the water body with waves to ensure the PP calculates the waves correctly.
				return ABody->GetWaterBodyComponent()->HasWaves() && !BBody->GetWaterBodyComponent()->HasWaves();
			});
		}

		UE_IFVLOG(
			if (bIsVisualLoggerEnabled)
			{
				// Visualize camera marker 
				UE_VLOG_LOCATION(this, LogWater, Log, ViewLocation, /*Radius=*/10.f, FColor::Green, TEXT("Camera"));

				// Visulize primary sweep capsule
				const FVector SweepStart = ViewLocation;
				const FVector SweepEnd = ViewLocation + FVector(0, 0, TraceDistance);
				UE_VLOG_CAPSULE(this, LogWater, Log, SweepStart, /*HalfHeight=*/ 0.0f, /*Radius=*/TraceDistance, FQuat::Identity, FColor::Yellow, TEXT("Primary sweep sphere@start"));

				// Visualize raw hit results
				for (const FHitResult& Hit : Hits)
				{
					UE_VLOG_LOCATION(this, LogWater, Log, Hit.ImpactPoint, 8.f, FColor::Red, TEXT("Primary Hit"));

					const float NormalLen = 50.f;
					UE_VLOG_ARROW(this, LogWater, Log, Hit.ImpactPoint, Hit.ImpactPoint + Hit.ImpactNormal * NormalLen, FColor::Red, TEXT("Primary Normal"));
				}
			}
		);

		// Determine if view is under water (camera is below impact point)
		const double& ViewLocationZ = ViewLocation.Z;
		const float ViewTolerance = 1.0f;
		const bool bIsViewLocationUnderImpactPoint = Algo::AnyOf(Hits, [&ViewLocationZ, &ViewTolerance](const FHitResult& Hit)
		{
			// Add a smaller tolerance to avoid flickering when the view location hovers around the impact point
			return (Hit.ImpactPoint.Z + ViewTolerance) > ViewLocationZ;
		});

		if (bIsViewLocationUnderImpactPoint)
		{
			World->SweepMultiByChannel(PreciseViewHits, ViewLocation, ViewLocation, FQuat::Identity, UnderwaterTraceChannel, FCollisionShape::MakeSphere(PreciseTraceDistance), TraceSimple);

			UE_IFVLOG(
				if (bIsVisualLoggerEnabled)
				{
					// Visualize precise sphere capsule
					UE_VLOG_CAPSULE(this, LogWater, Log, ViewLocation, /*HalfHeight=*/ 0.0f, /*Radius=*/PreciseTraceDistance, FQuat::Identity, FColor::Blue, TEXT("Precise sphere@start"));

					// Visualize precise hit results
					for (const FHitResult& PreciseHit : PreciseViewHits)
					{
						UE_VLOG_LOCATION(this, LogWater, Log, PreciseHit.ImpactPoint, 10.f, FColor::Blue, TEXT("Precise Hit"));

						const float NormalLen = 30.f;
						UE_VLOG_ARROW(this, LogWater, Log, PreciseHit.ImpactPoint, PreciseHit.ImpactPoint + PreciseHit.ImpactNormal * NormalLen, FColor::Blue, TEXT("Precise Normal"));
					}
				}
			);
		}

		float MaxWaterLevel = TNumericLimits<float>::Lowest();
		// When the camera is above water use the extended sphere collision to ensure post processing gets applied in waves
		TArray<FHitResult>& RefinedHits = bIsViewLocationUnderImpactPoint ? PreciseViewHits : Hits;
		for (const FHitResult& Result : RefinedHits)
		{
			if (AWaterBody* WaterBodyActor = Result.HitObjectHandle.FetchActor<AWaterBody>())
			{
				UWaterBodyComponent* WaterBodyComponent = WaterBodyActor->GetWaterBodyComponent();
				check(WaterBodyComponent);
				
				// Don't consider water bodies with no post process material : 
				if (WaterBodyComponent->ShouldRender() && (WaterBodyComponent->UnderwaterPostProcessMaterial != nullptr))
				{
					// Base water body info needed : 
					EWaterBodyQueryFlags QueryFlags = EWaterBodyQueryFlags::ComputeImmersionDepth
						| EWaterBodyQueryFlags::ComputeLocation
						| EWaterBodyQueryFlags::IncludeWaves;
					AdjustUnderwaterWaterInfoQueryFlags(QueryFlags);

					TValueOrError<FWaterBodyQueryResult, EWaterBodyQueryError> QueryResult = WaterBodyComponent->TryQueryWaterInfoClosestToWorldLocation(ViewLocation, QueryFlags);
					if (QueryResult.HasValue() && !QueryResult.GetValue().IsInExclusionVolume())
					{
						// Calculate the surface max Z at the view XY location
						float WaterSurfaceZ = QueryResult.GetValue().GetWaterPlaneLocation().Z + QueryResult.GetValue().GetWaveInfo().MaxHeight;

						// Only add the waterbody for processing if it has a higher surface than the previous waterbody (the Hits array is sorted by priority already)
						// This also removed any duplicate waterbodies possibly returned by the sweep query
						if (WaterSurfaceZ > MaxWaterLevel)
						{
							MaxWaterLevel = WaterSurfaceZ;
							WaterBodyQueriesToProcess.Add(FWaterBodyPostProcessQuery(*WaterBodyComponent, ViewLocation, QueryResult.GetValue()));
						}
					}
				}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				UnderwaterPostProcessDebugInfo.OverlappedWaterBodyComponents.AddUnique(WaterBodyComponent);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			}
		}

		for (const FWaterBodyPostProcessQuery& Query : WaterBodyQueriesToProcess)
		{
			const float MaxWaveHeight = Query.QueryResult.GetWaveInfo().MaxHeight;
			const double ImmersionWithoutWaves = Query.QueryResult.GetWaterPlaneLocation().Z - ViewLocation.Z; // Positive is under water, negative above water
			bAnyDefinitelyUnderwater |= (ImmersionWithoutWaves - ViewToNearPlaneCornerDistance) > MaxWaveHeight;
			bAnyPossiblyUnderwater |= (FMath::Abs(ImmersionWithoutWaves) - ViewToNearPlaneCornerDistance) <= MaxWaveHeight;

			float LocalDepthUnderwater = 0.0f;

			// Underwater is fudged a bit for post process so its possible to get a true return here but depth underwater is < 0
			// Post process should appear under any part of the water that clips the camera but underwater audio sounds should only play if the camera is actualy under water (i.e LocalDepthUnderwater > 0)
			bUnderwaterForPostProcess = GetWaterBodyDepthUnderwater(Query, LocalDepthUnderwater);
			if (bUnderwaterForPostProcess)
			{
				CachedDepthUnderwater = FMath::Max(LocalDepthUnderwater, CachedDepthUnderwater);
				UnderwaterPostProcessVolume->PostProcessProperties = Query.WaterBodyComponent.GetPostProcessProperties();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				UnderwaterPostProcessDebugInfo.ActiveWaterBodyComponent = &Query.WaterBodyComponent;
				UnderwaterPostProcessDebugInfo.ActiveWaterBodyQueryResult = Query.QueryResult;
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)					
				break;
			}
		}
	}

	SceneView->UnderwaterDepth = CachedDepthUnderwater;
	if (bAnyPossiblyUnderwater)
	{
		SceneView->WaterIntersection = EViewWaterIntersection::PossiblyIntersectingWater;
	}
	else if (bAnyDefinitelyUnderwater)
	{
		SceneView->WaterIntersection = EViewWaterIntersection::InsideWater;
	}
	else
	{
		SceneView->WaterIntersection = EViewWaterIntersection::OutsideWater;
	}
}

void UWaterSubsystem::SetMPCTime(float Time, float PrevTime)
{
	if (UWorld* World = GetWorld())
	{
		if (MaterialParameterCollection)
		{
			UMaterialParameterCollectionInstance* MaterialParameterCollectionInstance = World->GetParameterCollectionInstance(MaterialParameterCollection);
			const static FName TimeParam(TEXT("Time"));
			const static FName PrevTimeParam(TEXT("PrevTime"));
			MaterialParameterCollectionInstance->SetScalarParameterValue(TimeParam, Time);
			MaterialParameterCollectionInstance->SetScalarParameterValue(PrevTimeParam, PrevTime);
		}
	}
}


void UWaterSubsystem::AdjustUnderwaterWaterInfoQueryFlags(EWaterBodyQueryFlags& InOutFlags)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// We might need some extra info when showing debug info for the post process : 
	if (VisualizeActiveUnderwaterPostProcess > 1)
	{
		InOutFlags |= (EWaterBodyQueryFlags::ComputeDepth | EWaterBodyQueryFlags::ComputeLocation | EWaterBodyQueryFlags::IncludeWaves);
	}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void UWaterSubsystem::ShowOnScreenDebugInfo(const FVector& InViewLocation, const FUnderwaterPostProcessDebugInfo& InDebugInfo)
{
	// Visualize the active post process if any
	if (VisualizeActiveUnderwaterPostProcess == 0)
	{
		return;
	}

	TArray<FText, TInlineAllocator<8>> OutputStrings;

	OutputStrings.Add(FText::Format(LOCTEXT("VisualizeActiveUnderwaterPostProcess_ViewLocationDetails", "Underwater post process debug : view location : {0}"), FText::FromString(InViewLocation.ToCompactString())));

	if (InDebugInfo.ActiveWaterBodyComponent.IsValid())
	{
		FString MaterialDescription(TEXT("No material"));
		if (UMaterialInstanceDynamic* MID = InDebugInfo.ActiveWaterBodyComponent->GetUnderwaterPostProcessMaterialInstance())
		{
			check(MID->Parent != nullptr);
			MaterialDescription = FString::Format(TEXT("{0} (parent: {1})"), { MID->Parent->GetName(), MID->GetMaterial()->GetName() });
		}
		OutputStrings.Add(FText::Format(LOCTEXT("VisualizeActiveUnderwaterPostProcess_ActivePostprocess", "Active underwater post process water body {0} (material: {1})"),
			FText::FromString(InDebugInfo.ActiveWaterBodyComponent->GetOwner()->GetActorNameOrLabel()),
			FText::FromString(MaterialDescription)));
	}
	else
	{
		OutputStrings.Add(LOCTEXT("VisualizeActiveUnderwaterPostProcess_InactivePostprocess", "Inactive underwater post process"));
	}

	// Add more details : 
	if (VisualizeActiveUnderwaterPostProcess > 1)
	{
		// Display details about the water query that resulted in this underwater post process to picked :
		if (InDebugInfo.ActiveWaterBodyComponent.IsValid())
		{
			FText WaveDetails(LOCTEXT("VisualizeActiveUnderwaterPostProcess_WavelessDetails", "No waves"));
			if (InDebugInfo.ActiveWaterBodyComponent->HasWaves())
			{
				WaveDetails = FText::Format(LOCTEXT("VisualizeActiveUnderwaterPostProcess_WaveDetails", "- Wave Height : {0} (Max : {1}, Max here: {2}, Attenuation Factor : {3})"),
					InDebugInfo.ActiveWaterBodyQueryResult.GetWaveInfo().Height,
					InDebugInfo.ActiveWaterBodyComponent->GetMaxWaveHeight(),
					InDebugInfo.ActiveWaterBodyQueryResult.GetWaveInfo().MaxHeight,
					InDebugInfo.ActiveWaterBodyQueryResult.GetWaveInfo().AttenuationFactor);
			}

			OutputStrings.Add(FText::Format(LOCTEXT("VisualizeActiveUnderwaterPostProcess_QueryDetails", "- Water Surface Z : {0}\n- Water Depth : {1}\n{2}"),
				InDebugInfo.ActiveWaterBodyQueryResult.GetWaterSurfaceLocation().Z,
				InDebugInfo.ActiveWaterBodyQueryResult.GetWaterSurfaceDepth(),
				WaveDetails));
		}

		// Display each water body returned by the overlap query : 
		if (InDebugInfo.OverlappedWaterBodyComponents.Num() > 0)
		{
			OutputStrings.Add(FText::Format(LOCTEXT("VisualizeActiveUnderwaterPostProcess_OverlappedWaterBodyDetailsHeader", "{0} overlapping water bodies :"),
				InDebugInfo.OverlappedWaterBodyComponents.Num()));
			for (TWeakObjectPtr<UWaterBodyComponent> WaterBody : InDebugInfo.OverlappedWaterBodyComponents)
			{
				if (WaterBody.IsValid() && WaterBody->GetOwner())
				{
					OutputStrings.Add(FText::Format(LOCTEXT("VisualizeActiveUnderwaterPostProcess_OverlappedWaterBodyDetails", "- {0} (overlap material priority: {1})"),
						FText::FromString(WaterBody->GetOwner()->GetActorNameOrLabel()),
						FText::AsNumber(WaterBody->GetOverlapMaterialPriority())));
				}
			}
		}

		if (!InDebugInfo.bIsPostProcessingEnabled)
		{
			OutputStrings.Add(LOCTEXT("VisualizeActiveUnderwaterPostProcess_PostProcessingDisabled", "Post processing is disabled"));
		}

		if (!InDebugInfo.bIsUnderwaterPostProcessEnabled)
		{
			OutputStrings.Add(LOCTEXT("VisualizeActiveUnderwaterPostProcess_UnderwaterPostProcessDisabled", "Underwater post process is disabled"));
		}

		if (InDebugInfo.bIsPathTracingEnabled)
		{
			OutputStrings.Add(LOCTEXT("VisualizeActiveUnderwaterPostProcess_PathTracingEnabled", "Path tracing is enabled"));
		}
	}

	// Output a single message because multi-line texts end up overlapping over messages
	FString OutputMessage;
	for (const FText& Message : OutputStrings)
	{
		OutputMessage += Message.ToString() + "\n";
	}
	static const FName DebugMessageKeyName(TEXT("ActiveUnderwaterPostProcessMessage"));
	if (GEngine != nullptr)
	{
		GEngine->AddOnScreenDebugMessage((int32)DebugMessageKeyName.GetNumber(), 0.f, FColor::White, OutputMessage);
	}
}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

// ----------------------------------------------------------------------------------

#if WITH_EDITOR

UWaterSubsystem::FScopedAllowWaterSubsystemOnPreviewWorld::FScopedAllowWaterSubsystemOnPreviewWorld(bool bNewValue)
{
	bPreviousValue = UWaterSubsystem::GetAllowWaterSubsystemOnPreviewWorld();
	UWaterSubsystem::SetAllowWaterSubsystemOnPreviewWorld(bNewValue);
}

UWaterSubsystem::FScopedAllowWaterSubsystemOnPreviewWorld::~FScopedAllowWaterSubsystemOnPreviewWorld()
{
	UWaterSubsystem::SetAllowWaterSubsystemOnPreviewWorld(bPreviousValue);
}

#endif // WITH_EDITOR

// ----------------------------------------------------------------------------------

#undef LOCTEXT_NAMESPACE
