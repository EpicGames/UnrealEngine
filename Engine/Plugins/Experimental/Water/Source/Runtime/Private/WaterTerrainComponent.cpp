// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterTerrainComponent.h"

#include "WaterSubsystem.h"
#include "GameFramework/Actor.h"
#include "Components/PrimitiveComponent.h"
#include "EngineUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterTerrainComponent)

TArray<UPrimitiveComponent*> UWaterTerrainComponent::GetTerrainPrimitives() const
{
	TArray<UPrimitiveComponent*> Result;
	if (const AActor* Owner = GetOwner())
	{
		Owner->GetComponents(Result, /* bNonColliding */ true);
	}
	return Result;
}

FBox2D UWaterTerrainComponent::GetTerrainBounds() const
{
	FBox2D Result(ForceInitToZero);
	if (const AActor* Owner = GetOwner())
	{
		const FBox OwnerBounds = Owner->GetComponentsBoundingBox(/* bNonColliding */ true);
		Result = FBox2D(FVector2D(OwnerBounds.Min), FVector2D(OwnerBounds.Max));
	}
	return Result;
}

bool UWaterTerrainComponent::AffectsWaterZone(AWaterZone* WaterZone) const
{
	// if we have a water zone override set, and the passed water zone is not it, we do not affect it
	if (!WaterZoneOverride.IsNull() && WaterZoneOverride.Get() != WaterZone)
	{
		return false;
	}

	const FBox2D WaterZoneBounds = WaterZone->GetZoneBounds2D();

	return GetTerrainBounds().Intersect(WaterZoneBounds);
}

void UWaterTerrainComponent::OnRegister()
{
	Super::OnRegister();

	if (GetOwner() == nullptr)
	{
		return;
	}
	
	if (UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(GetWorld()))
	{
		WaterSubsystem->RegisterWaterTerrainComponent(this);
	}

	for (AWaterZone* WaterZone : TActorRange<AWaterZone>(GetWorld()))
	{
		if (AffectsWaterZone(WaterZone))
		{
			WaterZone->MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture, GetTerrainBounds(), GetOwner());
		}
	}
}

void UWaterTerrainComponent::OnUnregister()
{
	if (GetOwner() != nullptr)
	{
		for (AWaterZone* WaterZone : TActorRange<AWaterZone>(GetWorld()))
		{
			if (AffectsWaterZone(WaterZone))
			{
				 WaterZone->MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture, GetTerrainBounds(), GetOwner());
			}
		}
	}

	if (UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(GetWorld()))
	{
		WaterSubsystem->UnregisterWaterTerrainComponent(this);
	}
	
	Super::OnUnregister();
}

#if WITH_EDITOR
void UWaterTerrainComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaterTerrainComponent, WaterZoneOverride))
	{
		// If the override pointer is set, mark all water zones to re-render since we can't know which ones we used to affect:
		if (UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(GetWorld()))
		{
			WaterSubsystem->MarkWaterZonesInRegionForRebuild(GetTerrainBounds(), EWaterZoneRebuildFlags::UpdateWaterInfoTexture, GetOwner());
		}
	}
}
#endif // WITH_EDITOR

