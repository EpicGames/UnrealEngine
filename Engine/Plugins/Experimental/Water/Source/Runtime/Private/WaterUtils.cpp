// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterUtils.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "UObject/Package.h"
#include "RenderingThread.h"

extern TAutoConsoleVariable<int32> CVarWaterEnabled;
extern TAutoConsoleVariable<int32> CVarWaterMeshEnabled;
extern TAutoConsoleVariable<int32> CVarWaterMeshEnableRendering;

static TAutoConsoleVariable<float> CVarWaterMaxFlowVelocity(
	TEXT("r.Water.MaxFlowVelocity"),
	1024.0f,
	TEXT("The maximum magnitude for the velocity of a river to encode in the WaterInfo texture"),
	ECVF_Default);

UMaterialInstanceDynamic* FWaterUtils::GetOrCreateTransientMID(UMaterialInstanceDynamic* InMID, FName InMIDName, UMaterialInterface* InMaterialInterface, EObjectFlags InAdditionalObjectFlags)
{
	if (!IsValid(InMaterialInterface))
	{
		return nullptr;
	}

	UMaterialInstanceDynamic* ResultMID = InMID;

	// If there's no MID yet or if the MID's parent material interface (could be a Material or a MIC) doesn't match the requested material interface (could be a Material, MIC or MID), create a new one : 
	if (!IsValid(InMID) || (InMID->Parent != InMaterialInterface))
	{
		// If the requested material is already a UMaterialInstanceDynamic, we can use it as is :
		ResultMID = Cast<UMaterialInstanceDynamic>(InMaterialInterface);

		if (ResultMID != nullptr)
		{
			ensure(EnumHasAnyFlags(InMaterialInterface->GetFlags(), EObjectFlags::RF_Transient)); // The name of the function implies we're dealing with transient MIDs
		}
		else
		{
			// If it's not a UMaterialInstanceDynamic, it's a UMaterialInstanceConstant or a UMaterial, both of which can be used to create a MID : 
			ResultMID = UMaterialInstanceDynamic::Create(InMaterialInterface, nullptr, MakeUniqueObjectName(GetTransientPackage(), UMaterialInstanceDynamic::StaticClass(), InMIDName));
			ResultMID->SetFlags(InAdditionalObjectFlags);
		}
	}

	check(ResultMID != nullptr);
	return ResultMID;
}

UTextureRenderTarget2D* FWaterUtils::GetOrCreateTransientRenderTarget2D(UTextureRenderTarget2D* InRenderTarget, FName InRenderTargetName, const FIntPoint& InSize, ETextureRenderTargetFormat InFormat, 
	const FLinearColor& InClearColor, bool bInAutoGenerateMipMaps)
{
	EPixelFormat PixelFormat = GetPixelFormatFromRenderTargetFormat(InFormat);
	if ((InSize.X <= 0) 
		|| (InSize.Y <= 0) 
		|| (PixelFormat == EPixelFormat::PF_Unknown))
	{
		return nullptr;
	}

	if (IsValid(InRenderTarget))
	{
		if ((InRenderTarget->SizeX == InSize.X) 
			&& (InRenderTarget->SizeY == InSize.Y) 
			&& (InRenderTarget->GetFormat() == PixelFormat) // Watch out : GetFormat() returns a EPixelFormat (non-class enum), so we can't compare with a ETextureRenderTargetFormat
			&& (InRenderTarget->ClearColor == InClearColor)
			&& (InRenderTarget->bAutoGenerateMips == bInAutoGenerateMipMaps))
		{
			return InRenderTarget;
		}
	}

	UTextureRenderTarget2D* NewRenderTarget2D = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), MakeUniqueObjectName(GetTransientPackage(), UTextureRenderTarget2D::StaticClass(), InRenderTargetName));
	check(NewRenderTarget2D);
	NewRenderTarget2D->RenderTargetFormat = InFormat;
	NewRenderTarget2D->ClearColor = InClearColor;
	NewRenderTarget2D->bAutoGenerateMips = bInAutoGenerateMipMaps;
	NewRenderTarget2D->InitAutoFormat(InSize.X, InSize.Y);
	NewRenderTarget2D->UpdateResourceImmediate(true);

	// Flush RHI thread after creating texture render target to make sure that RHIUpdateTextureReference is executed before doing any rendering with it
	// This makes sure that Value->TextureReference.TextureReferenceRHI->GetReferencedTexture() is valid so that FUniformExpressionSet::FillUniformBuffer properly uses the texture for rendering, instead of using a fallback texture
	ENQUEUE_RENDER_COMMAND(FlushRHIThreadToUpdateTextureRenderTargetReference)(
	[](FRHICommandListImmediate& RHICmdList)
	{
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
	});

	return NewRenderTarget2D;
}

WATER_API UTextureRenderTarget2DArray* FWaterUtils::GetOrCreateTransientRenderTarget2DArray(UTextureRenderTarget2DArray* InRenderTarget, FName InRenderTargetName, const FIntPoint& InSize, int32 InSlices, ETextureRenderTargetFormat InFormat, const FLinearColor& InClearColor, bool bInAutoGenerateMipMaps)
{
	EPixelFormat PixelFormat = GetPixelFormatFromRenderTargetFormat(InFormat);
	if ((InSize.X <= 0)
		|| (InSize.Y <= 0)
		|| (InSlices <= 0)
		|| (PixelFormat == EPixelFormat::PF_Unknown))
	{
		return nullptr;
	}

	if (IsValid(InRenderTarget))
	{
		if ((InRenderTarget->SizeX == InSize.X)
			&& (InRenderTarget->SizeY == InSize.Y)
			&& (InRenderTarget->Slices == InSlices)
			&& (InRenderTarget->GetFormat() == PixelFormat) // Watch out : GetFormat() returns a EPixelFormat (non-class enum), so we can't compare with a ETextureRenderTargetFormat
			&& (InRenderTarget->ClearColor == InClearColor)
			&& (InRenderTarget->bTargetArraySlicesIndependently))
		{
			return InRenderTarget;
		}
	}

	UTextureRenderTarget2DArray* NewRenderTarget2DArray = NewObject<UTextureRenderTarget2DArray>(GetTransientPackage(), MakeUniqueObjectName(GetTransientPackage(), UTextureRenderTarget2DArray::StaticClass(), InRenderTargetName));
	check(NewRenderTarget2DArray);
	NewRenderTarget2DArray->ClearColor = InClearColor;
	NewRenderTarget2DArray->bTargetArraySlicesIndependently = true; // When r.water.waterinfo.rendermethod is 1, we directly render into each individual slice
	NewRenderTarget2DArray->Init(InSize.X, InSize.Y, InSlices, PixelFormat);
	NewRenderTarget2DArray->UpdateResourceImmediate(true);

	// Flush RHI thread after creating texture render target to make sure that RHIUpdateTextureReference is executed before doing any rendering with it
	// This makes sure that Value->TextureReference.TextureReferenceRHI->GetReferencedTexture() is valid so that FUniformExpressionSet::FillUniformBuffer properly uses the texture for rendering, instead of using a fallback texture
	ENQUEUE_RENDER_COMMAND(FlushRHIThreadToUpdateTextureRenderTargetReference)(
	[](FRHICommandListImmediate& RHICmdList)
	{
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
	});

	return NewRenderTarget2DArray;
}

FGuid FWaterUtils::StringToGuid(const FString& InStr)
{
	// Compute a 128-bit hash based on the string and use that as a GUID :
	FTCHARToUTF8 Converted(*InStr);
	FMD5 MD5Gen;
	MD5Gen.Update((const uint8*)Converted.Get(), Converted.Length());
	uint32 Digest[4];
	MD5Gen.Final((uint8*)Digest);

	// FGuid::NewGuid() creates a version 4 UUID (at least on Windows), which will have the top 4 bits of the
	// second field set to 0100. We'll set the top bit to 1 in the GUID we create, to ensure that we can never
	// have a collision with other implicitly-generated GUIDs.
	Digest[1] |= 0x80000000;
	return FGuid(Digest[0], Digest[1], Digest[2], Digest[3]);
}

bool FWaterUtils::IsWaterEnabled(bool bIsRenderThread)
{
	return !!(bIsRenderThread ? CVarWaterEnabled.GetValueOnRenderThread() : CVarWaterEnabled.GetValueOnGameThread());
}

bool FWaterUtils::IsWaterMeshEnabled(bool bIsRenderThread)
{
	return IsWaterEnabled(bIsRenderThread) && !!(bIsRenderThread ? CVarWaterMeshEnabled.GetValueOnRenderThread() : CVarWaterMeshEnabled.GetValueOnGameThread());
}

bool FWaterUtils::IsWaterMeshRenderingEnabled(bool bIsRenderThread)
{
	return IsWaterMeshEnabled(bIsRenderThread) && !!(bIsRenderThread ? CVarWaterMeshEnableRendering.GetValueOnRenderThread() : CVarWaterMeshEnableRendering.GetValueOnGameThread());
}

float FWaterUtils::GetWaterMaxFlowVelocity(bool bIsRenderThread)
{
	return (bIsRenderThread ? CVarWaterMaxFlowVelocity.GetValueOnRenderThread() : CVarWaterMaxFlowVelocity.GetValueOnGameThread());
}

FVector4f FWaterUtils::PackFlowData(float VelocityMagnitude, float DirectionAngle)
{
	check((DirectionAngle >= 0.f) && (DirectionAngle <= TWO_PI));
	checkf(VelocityMagnitude >= 0., TEXT("We expect a positive magnitude"));

	const float MaxVelocity = GetWaterMaxFlowVelocity(false);

	FVector4f Result;
	Result.X = FMath::Clamp(VelocityMagnitude / MaxVelocity, 0.f, 1.f);
	Result.Y = FMath::Clamp(DirectionAngle / TWO_PI, 0.f, 1.f);

	return Result;
}

