// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterVersion.h"
#include "Serialization/CustomVersion.h"

#define LOCTEXT_NAMESPACE "WaterVersion"

const FGuid FWaterCustomVersion::GUID(0x40D2FBA7, 0x4B484CE5, 0xB0385A75, 0x884E499E);

// Register the custom version with core
FCustomVersionRegistration GRegisterWaterCustomVersion(FWaterCustomVersion::GUID, FWaterCustomVersion::LatestVersion, TEXT("Water"));

#undef LOCTEXT_NAMESPACE 