// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterVertexFactory.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "MeshDrawShaderBindings.h"
#include "MeshBatch.h"
#include "MeshMaterialShader.h"
#include "RenderUtils.h"

#include "SceneView.h"

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FWaterVertexFactoryParameters, "WaterVF");
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FWaterVertexFactoryRaytracingParameters, "WaterRaytracingVF");

/**
 * Shader parameters for water vertex factory.
 */
template <bool bWithWaterSelectionSupport, EWaterVertexFactoryDrawMode DrawMode>
class TWaterVertexFactoryShaderParameters : public FVertexFactoryShaderParameters
{
	using WaterVertexFactoryShaderParametersType = TWaterVertexFactoryShaderParameters<bWithWaterSelectionSupport, DrawMode>;
	DECLARE_TYPE_LAYOUT(WaterVertexFactoryShaderParametersType, NonVirtual);

public:
	using WaterVertexFactoryType = TWaterVertexFactory<bWithWaterSelectionSupport, DrawMode>;
	using WaterMeshUserDataType = TWaterMeshUserData<bWithWaterSelectionSupport>;
	using WaterInstanceDataBuffersType = TWaterInstanceDataBuffers<bWithWaterSelectionSupport>;

	void Bind(const FShaderParameterMap& ParameterMap)
	{
		if (DrawMode == EWaterVertexFactoryDrawMode::Indirect || DrawMode == EWaterVertexFactoryDrawMode::IndirectInstancedStereo)
		{
			QuadTreePositionParameter.Bind(ParameterMap, TEXT("QuadTreePosition"));
			CaptureDepthRangeParameter.Bind(ParameterMap, TEXT("CaptureDepthRange"));
			LODMorphingEnabledParameter.Bind(ParameterMap, TEXT("bLODMorphingEnabled"));

			if (DrawMode == EWaterVertexFactoryDrawMode::IndirectInstancedStereo)
			{
				InstanceDataOffsetsBufferParameter.Bind(ParameterMap, TEXT("InstanceDataOffsetsBuffer"));
				InstanceData0BufferParameter.Bind(ParameterMap, TEXT("InstanceData0Buffer"));
				InstanceData1BufferParameter.Bind(ParameterMap, TEXT("InstanceData1Buffer"));
				InstanceData2BufferParameter.Bind(ParameterMap, TEXT("InstanceData2Buffer"));
				if (bWithWaterSelectionSupport)
				{
					InstanceData3BufferParameter.Bind(ParameterMap, TEXT("InstanceData3Buffer"));
				}

				DrawBucketIndexParameter.Bind(ParameterMap, TEXT("DrawBucketIndex"));
				StereoPassInstanceFactorParameter.Bind(ParameterMap, TEXT("StereoPassInstanceFactor"));
			}
		}
	}

	void GetElementShaderBindings(
		const class FSceneInterface* Scene,
		const class FSceneView* View,
		const class FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const class FVertexFactory* InVertexFactory,
		const struct FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		WaterVertexFactoryType* VertexFactory = (WaterVertexFactoryType*)InVertexFactory;

		const WaterMeshUserDataType* WaterMeshUserData = (const WaterMeshUserDataType*)BatchElement.UserData;

		const WaterInstanceDataBuffersType* InstanceDataBuffers = WaterMeshUserData->InstanceDataBuffers;

		ShaderBindings.Add(Shader->GetUniformBufferParameter<FWaterVertexFactoryParameters>(), VertexFactory->GetWaterVertexFactoryUniformBuffer(WaterMeshUserData->RenderGroupType));

#if RHI_RAYTRACING
		if (IsRayTracingEnabled())
		{
			ShaderBindings.Add(Shader->GetUniformBufferParameter<FWaterVertexFactoryRaytracingParameters>(), WaterMeshUserData->WaterVertexFactoryRaytracingVFUniformBuffer);
		}
#endif

		if (DrawMode == EWaterVertexFactoryDrawMode::Indirect || DrawMode == EWaterVertexFactoryDrawMode::IndirectInstancedStereo)
		{
			check((DrawMode == EWaterVertexFactoryDrawMode::IndirectInstancedStereo) == View->bIsInstancedStereoEnabled);

			if (View->bIsInstancedStereoEnabled)
			{
				const uint32 DrawBucketIndex = BatchElement.UserIndex;
				const uint32 StereoPassInstanceFactor = View->GetStereoPassInstanceFactor();
				ShaderBindings.Add(InstanceDataOffsetsBufferParameter, WaterMeshUserData->IndirectInstanceDataOffsetsSRV);
				ShaderBindings.Add(InstanceData0BufferParameter, WaterMeshUserData->IndirectInstanceData0SRV);
				ShaderBindings.Add(InstanceData1BufferParameter, WaterMeshUserData->IndirectInstanceData1SRV);
				ShaderBindings.Add(InstanceData2BufferParameter, WaterMeshUserData->IndirectInstanceData2SRV);
				if (bWithWaterSelectionSupport)
				{
					ShaderBindings.Add(InstanceData3BufferParameter, WaterMeshUserData->IndirectInstanceData3SRV);
				}
				ShaderBindings.Add(DrawBucketIndexParameter, DrawBucketIndex);
				ShaderBindings.Add(StereoPassInstanceFactorParameter, StereoPassInstanceFactor);
			}
			else if (VertexStreams.Num() > 0)
			{
				const int32 NumBuffers = bWithWaterSelectionSupport ? 4 : 3;
				FRHIBuffer* InstanceVertexBuffers[] = 
				{ 
					WaterMeshUserData->IndirectInstanceData0, 
					WaterMeshUserData->IndirectInstanceData1,
					WaterMeshUserData->IndirectInstanceData2,
					WaterMeshUserData->IndirectInstanceData3,
				};
				for (int32 i = 0; i < NumBuffers; ++i)
				{
					FVertexInputStream* InstanceInputStream = VertexStreams.FindByPredicate([i](const FVertexInputStream& InStream) { return InStream.StreamIndex == 1 + i; });
					check(InstanceInputStream);

					// Bind vertex buffer
					check(InstanceVertexBuffers[i]);
					InstanceInputStream->VertexBuffer = InstanceVertexBuffers[i];
				}
			}

			const FVector PreViewTranslation = View->ViewMatrices.GetPreViewTranslation();
			ShaderBindings.Add(QuadTreePositionParameter, FVector3f(PreViewTranslation + WaterMeshUserData->QuadTreePosition));
			ShaderBindings.Add(CaptureDepthRangeParameter, WaterMeshUserData->CaptureDepthRange);

			static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Water.WaterMesh.LODMorphEnabled"));
			const bool bLODMorphingEnabled = CVar && CVar->GetValueOnRenderThread() != 0;
			ShaderBindings.Add(LODMorphingEnabledParameter, bLODMorphingEnabled ? 1 : 0);
		}
		else if (VertexStreams.Num() > 0)
		{
			for (int32 i = 0; i < WaterInstanceDataBuffersType::NumBuffers; ++i)
			{
				FVertexInputStream* InstanceInputStream = VertexStreams.FindByPredicate([i](const FVertexInputStream& InStream) { return InStream.StreamIndex == i + 1; });
				check(InstanceInputStream);

				// Bind vertex buffer
				check(InstanceDataBuffers->GetBuffer(i));
				InstanceInputStream->VertexBuffer = InstanceDataBuffers->GetBuffer(i);
			}

			const int32 InstanceOffsetValue = BatchElement.UserIndex;
			if (InstanceOffsetValue > 0)
			{
				VertexFactory->OffsetInstanceStreams(InstanceOffsetValue, InputStreamType, VertexStreams);
			}
		}
	}

private:
	LAYOUT_FIELD(FShaderParameter, QuadTreePositionParameter);
	LAYOUT_FIELD(FShaderParameter, CaptureDepthRangeParameter);
	LAYOUT_FIELD(FShaderParameter, LODMorphingEnabledParameter);
	LAYOUT_FIELD(FShaderResourceParameter, InstanceDataOffsetsBufferParameter);
	LAYOUT_FIELD(FShaderResourceParameter, InstanceData0BufferParameter);
	LAYOUT_FIELD(FShaderResourceParameter, InstanceData1BufferParameter);
	LAYOUT_FIELD(FShaderResourceParameter, InstanceData2BufferParameter);
	LAYOUT_FIELD(FShaderResourceParameter, InstanceData3BufferParameter);
	LAYOUT_FIELD(FShaderParameter, DrawBucketIndexParameter);
	LAYOUT_FIELD(FShaderParameter, StereoPassInstanceFactorParameter);
};

// ----------------------------------------------------------------------------------

#define IMPLEMENT_WATER_VERTEX_FACTORY_RAYTRACING(VertexFactoryType, ParametersType, bImplementRaytracing) IMPLEMENT_WATER_VERTEX_FACTORY_RAYTRACING_##bImplementRaytracing(VertexFactoryType, ParametersType)
#define IMPLEMENT_WATER_VERTEX_FACTORY_RAYTRACING_false(VertexFactoryType, ParametersType)
#define IMPLEMENT_WATER_VERTEX_FACTORY_RAYTRACING_true(VertexFactoryType, ParametersType)														\
	IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(VertexFactoryType, SF_Compute, ParametersType);														\
	IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(VertexFactoryType, SF_RayHitGroup, ParametersType);

#define IMPLEMENT_WATER_VERTEX_FACTORY(bWidthWaterSelectionSupport, DrawMode, bImplementRaytracing, Suffix)										\
	using FWaterVertexFactoryParameters##Suffix = TWaterVertexFactoryShaderParameters<bWidthWaterSelectionSupport, DrawMode>;					\
	using FWaterVertexFactory##Suffix = TWaterVertexFactory<bWidthWaterSelectionSupport, DrawMode>;												\
	IMPLEMENT_TEMPLATE_TYPE_LAYOUT(template<>, FWaterVertexFactoryParameters##Suffix);															\
	IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FWaterVertexFactory##Suffix, SF_Vertex, FWaterVertexFactoryParameters##Suffix);						\
	IMPLEMENT_WATER_VERTEX_FACTORY_RAYTRACING(FWaterVertexFactory##Suffix, FWaterVertexFactoryParameters##Suffix, bImplementRaytracing)			\
	IMPLEMENT_TEMPLATE_VERTEX_FACTORY_TYPE(template<>, FWaterVertexFactory##Suffix, "/Plugin/Water/Private/WaterMeshVertexFactory.ush",			\
		EVertexFactoryFlags::UsedWithMaterials																									\
		| EVertexFactoryFlags::SupportsDynamicLighting																							\
		| EVertexFactoryFlags::SupportsPrecisePrevWorldPos																						\
		| EVertexFactoryFlags::SupportsPrimitiveIdStream																						\
		| EVertexFactoryFlags::SupportsRayTracing																								\
		| EVertexFactoryFlags::SupportsRayTracingDynamicGeometry																				\
		| EVertexFactoryFlags::SupportsPSOPrecaching																							\
	);

// Always implement the basic vertex factory so that it's there for both editor and non-editor builds :
#if RHI_RAYTRACING
IMPLEMENT_WATER_VERTEX_FACTORY(false /*bWithWaterSelectionSupport*/, EWaterVertexFactoryDrawMode::NonIndirect, true /*bImplementRaytracing*/, NoSelectionNonIndirect);
#else
IMPLEMENT_WATER_VERTEX_FACTORY(false /*bWithWaterSelectionSupport*/, EWaterVertexFactoryDrawMode::NonIndirect, false /*bImplementRaytracing*/, NoSelectionNonIndirect);
#endif
IMPLEMENT_WATER_VERTEX_FACTORY(false /*bWithWaterSelectionSupport*/, EWaterVertexFactoryDrawMode::Indirect, false /*bImplementRaytracing*/, NoSelectionWithIndirect);
IMPLEMENT_WATER_VERTEX_FACTORY(false /*bWithWaterSelectionSupport*/, EWaterVertexFactoryDrawMode::IndirectInstancedStereo, false /*bImplementRaytracing*/, NoSelectionWithIndirectISR);

// In editor builds, also implement the vertex factory that supports water selection:
#if WITH_WATER_SELECTION_SUPPORT

#if RHI_RAYTRACING
IMPLEMENT_WATER_VERTEX_FACTORY(true /*bWithWaterSelectionSupport*/, EWaterVertexFactoryDrawMode::NonIndirect, true /*bImplementRaytracing*/, WithSelectionNonIndirect);
#else
IMPLEMENT_WATER_VERTEX_FACTORY(true /*bWithWaterSelectionSupport*/, EWaterVertexFactoryDrawMode::NonIndirect, false /*bImplementRaytracing*/, WithSelectionNonIndirect);
#endif

IMPLEMENT_WATER_VERTEX_FACTORY(true /*bWithWaterSelectionSupport*/, EWaterVertexFactoryDrawMode::Indirect, false /*bImplementRaytracing*/, WithSelectionWithIndirect);
IMPLEMENT_WATER_VERTEX_FACTORY(true /*bWithWaterSelectionSupport*/, EWaterVertexFactoryDrawMode::IndirectInstancedStereo, false /*bImplementRaytracing*/, WithSelectionWithIndirectISR);
#endif // WITH_WATER_SELECTION_SUPPORT

const FVertexFactoryType* GetWaterVertexFactoryType(bool bWithWaterSelectionSupport, EWaterVertexFactoryDrawMode DrawMode)
{
#if WITH_WATER_SELECTION_SUPPORT
	if (bWithWaterSelectionSupport)
	{
		switch (DrawMode)
		{
		case EWaterVertexFactoryDrawMode::NonIndirect: return &TWaterVertexFactory</*bWithWaterSelectionSupport = */ true, EWaterVertexFactoryDrawMode::NonIndirect>::StaticType;
		case EWaterVertexFactoryDrawMode::Indirect: return &TWaterVertexFactory</*bWithWaterSelectionSupport = */ true, EWaterVertexFactoryDrawMode::Indirect>::StaticType;
		case EWaterVertexFactoryDrawMode::IndirectInstancedStereo: return &TWaterVertexFactory</*bWithWaterSelectionSupport = */ true, EWaterVertexFactoryDrawMode::IndirectInstancedStereo>::StaticType;
		default: checkNoEntry(); return nullptr;
		}
	}
	else
#endif // WITH_WATER_SELECTION_SUPPORT
	{
		switch (DrawMode)
		{
		case EWaterVertexFactoryDrawMode::NonIndirect: return &TWaterVertexFactory</*bWithWaterSelectionSupport = */ false, EWaterVertexFactoryDrawMode::NonIndirect>::StaticType;
		case EWaterVertexFactoryDrawMode::Indirect: return &TWaterVertexFactory</*bWithWaterSelectionSupport = */ false, EWaterVertexFactoryDrawMode::Indirect>::StaticType;
		case EWaterVertexFactoryDrawMode::IndirectInstancedStereo: return &TWaterVertexFactory</*bWithWaterSelectionSupport = */ false, EWaterVertexFactoryDrawMode::IndirectInstancedStereo>::StaticType;
		default: checkNoEntry(); return nullptr;
		}
	}
}
