// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterViewExtension.h"
#include "WaterBodyComponent.h"
#include "WaterZoneActor.h"
#include "EngineUtils.h"
#include "SceneView.h"
#include "WaterMeshComponent.h"
#include "GerstnerWaterWaveSubsystem.h"
#include "WaterBodyManager.h"
#include "WaterSubsystem.h"
#include "RHIResourceUtils.h"
#include "WaterMeshSceneProxy.h"
#include "Engine/GameInstance.h"
#include "WaterModule.h"

static TAutoConsoleVariable<bool> CVarLocalTessellationFreeze(
	TEXT("r.Water.WaterMesh.LocalTessellation.Freeze"),
	false,
	TEXT("Pauses the local tessellation updates to allow the view to move forward without moving the sliding window.\n")
	TEXT("Can be used to view things outside the sliding window more closely."),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarLocalTessellationUpdateMargin(
	TEXT("r.Water.WaterMesh.LocalTessellation.UpdateMargin"),
	15000.,
	TEXT("Controls the minimum distance between the view and the edge of the dynamic water mesh when local tessellation is enabled.\n")
	TEXT("If the view is less than UpdateMargin units away from the edge, it moves the sliding window forward."),
	ECVF_Default);

extern void OnCVarWaterInfoSceneProxiesValueChanged(IConsoleVariable*);
static TAutoConsoleVariable<int32> CVarWaterInfoRenderMethod(
	TEXT("r.Water.WaterInfo.RenderMethod"),
	2,
	TEXT("1: Custom, 2: CustomRenderPasses"),
	FConsoleVariableDelegate::CreateStatic(OnCVarWaterInfoSceneProxiesValueChanged),
	ECVF_Default | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarDrawPerViewDebugInfo(
	TEXT("r.Water.WaterInfo.DrawPerViewDebugInfo"),
	false,
	TEXT("Enable this to draw WaterZoneInfo debug information per view."),
	ECVF_Default);

// ----------------------------------------------------------------------------------

FWaterMeshGPUWork GWaterMeshGPUWork;

FWaterViewExtension::FWaterViewExtension(const FAutoRegister& AutoReg, UWorld* InWorld)
	: FWorldSceneViewExtension(AutoReg, InWorld)
	, WaterGPUData(MakeShared<FWaterGPUResources, ESPMode::ThreadSafe>())
{
}

FWaterViewExtension::~FWaterViewExtension()
{
}

// this delegates helps the editor view's WaterViewExtension detect scenarios when PIE ends, to make sure we schedule a forced bounds update
void FWaterViewExtension::OnWorldDestroyed(UWorld* InWorld)
{
	// this is needed because when switching back from PIE to Editor, there isn't a proper initialization of
	// the editor WaterViewExtension, since during PIE it is kept alive and not updated. Because of that when
	// PIE ends and we start updating the editor view extension, it can contain out of date values which can cause
	// water rendering artifacts until stepping out of bounds forces the first update.
	if (InWorld->IsPlayInEditor())
	{
		bRequestForcedBoundsUpdate = true;
	}
}

void FWaterViewExtension::Initialize()
{
	// Register the view extension to the Gerstner Wave subsystem so we can rebuild the water gpu data when waves change.
	if (UGerstnerWaterWaveSubsystem* GerstnerWaterWaveSubsystem = GEngine->GetEngineSubsystem<UGerstnerWaterWaveSubsystem>())
	{
		GerstnerWaterWaveSubsystem->Register(this);
	}

	CurrentNumViews = 0;

    FWorldDelegates::OnPreWorldFinishDestroy.AddRaw(this, &FWaterViewExtension::OnWorldDestroyed);

	QuadTreeKeyLocationMap.Empty();
}

void FWaterViewExtension::Deinitialize()
{
	ENQUEUE_RENDER_COMMAND(DeallocateWaterInstanceDataBuffer)
	(
		// Copy the shared ptr into a local copy for this lambda, this will increase the ref count and keep it alive on the renderthread until this lambda is executed
		[WaterGPUData=WaterGPUData](FRHICommandListImmediate& RHICmdList){}
	);

	if (UGerstnerWaterWaveSubsystem* GerstnerWaterWaveSubsystem = GEngine->GetEngineSubsystem<UGerstnerWaterWaveSubsystem>())
	{
		GerstnerWaterWaveSubsystem->Unregister(this);
	}

	CurrentNumViews = 0;

    FWorldDelegates::OnPreWorldFinishDestroy.RemoveAll(this);

	QuadTreeKeyLocationMap.Empty();
}

void FWaterViewExtension::UpdateGPUBuffers()
{
	if (bRebuildGPUData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Water::RebuildWaterGPUData);

		const UWorld* WorldPtr = GetWorld();
		check(WorldPtr != nullptr);
		FWaterBodyManager* WaterBodyManager = UWaterSubsystem::GetWaterBodyManager(WorldPtr);
		check(WaterBodyManager);

		// Shrink the water manager storage to avoid over-preallocating in the WaterBodyDataBuffer.
		WaterBodyManager->Shrink();


		struct FWaterBodyData
		{
			float WaterZoneIndex;
			float WaveDataIndex;
			float NumWaves;
			float TargetWaveMaskDepth;
			float FixedVelocityXY; // Packed as two 16 bit floats. X is in the lower 16 bits.
			float FixedVelocityZ;
			float FixedZHeight;
			float FixedWaterDepth;
		};
		static_assert(sizeof(FWaterBodyData) == 2 * sizeof(FVector4f));

		struct FWaterZoneData
		{
			FVector2f Extent;
			FVector2f HeightExtent;
			float GroundZMin;
			float bIsLocalOnlyTessellation;

			float _Padding[2]; // Unused

			FWaterZoneData(const FVector2f& InExtent, const FVector2f& InHeightExtent, float InGroundZMin, float bInIsLocalOnlyTessellation)
				: Extent(InExtent), HeightExtent(InHeightExtent), GroundZMin(InGroundZMin), bIsLocalOnlyTessellation(bInIsLocalOnlyTessellation) {}
		};
		static_assert(sizeof(FWaterZoneData) == 2 * sizeof(FVector4f));

		struct FGerstnerWaveData
		{
			FVector2f Direction;
			float WaveLength;
			float Amplitude;
			float Steepness;

			float _Padding[3]; // Unused

			FGerstnerWaveData(const FGerstnerWave& Wave)
				: Direction(FVector2D(Wave.Direction)), WaveLength(Wave.WaveLength), Amplitude(Wave.Amplitude), Steepness(Wave.Steepness) {}
		};
		static_assert(sizeof(FGerstnerWaveData) == 2 * sizeof(FVector4f));

		struct FWaterZoneViewData
		{
			FVector2f Location;

			FVector2f _Padding; // Unused

			FWaterZoneViewData(const FVector2f& InLocation)
				: Location(InLocation) {}
		};
		static_assert(sizeof(FWaterZoneViewData) == sizeof(FVector4f));

		// Water Body Data Buffer layout:
		// -------------------------------------------------------------------------------
		// || WaterZoneIndex | WaveDataIndex | NumWaves | (Other members) ||   ...   ||
		// -------------------------------------------------------------------------------
		//
		// Water Aux Data Buffer layout:
		// -----------------------------------------------------------------------------
		// ||| WaterZone Data | ... || WaterZoneView Data | ... || GerstnerWaveData | ... |||
		// -----------------------------------------------------------------------------
		//

		TArray<FWaterZoneData> WaterZoneData;

		{
			WaterBodyManager->ForEachWaterZone([&WaterZoneData, this](AWaterZone* WaterZone)
			{
				const FVector2f ZoneExtent = FVector2f(FVector2D(WaterZone->GetDynamicWaterInfoExtent()));
				const FVector2f WaterHeightExtents = WaterZone->GetWaterHeightExtents();
				const float GroundZMin = WaterZone->GetGroundZMin();
				const float bIsLocalOnlyTessellation = WaterZone->IsLocalOnlyTessellationEnabled() ? 1.0f : -1.0f;

				WaterZoneData.Emplace(ZoneExtent, WaterHeightExtents, GroundZMin, bIsLocalOnlyTessellation);

				return true;
			});
		}

		// We store views packed per zone:
		// ||| Zone0View0 | Zone0View1 | ... | Zone0ViewN ||...|| ZoneNView0 | ZoneNView1 | ... | ZoneNViewN |||
		TArray<FWaterZoneViewData> WaterZoneViewData;
		{
			WaterBodyManager->ForEachWaterZone([&WaterZoneViewData, this](AWaterZone* WaterZone)
			{
				FWaterZoneInfo* WaterZoneInfo = WaterZoneInfos.Find(WaterZone);

				check(WaterZoneInfo != nullptr);

				if (WaterZoneInfo != nullptr)
				{
					for (const FWaterZoneInfo::FWaterZoneViewInfo& ViewInfos : WaterZoneInfo->ViewInfos)
					{
						// #todo_water: LWC
						FVector2f ZoneViewLocation = FVector2f(FVector2D(ViewInfos.Center));
						WaterZoneViewData.Emplace(ZoneViewLocation);
					}
				}
				
				return true;
			});
		}

		TArray<FWaterBodyData> WaterBodyData;
		TArray<FGerstnerWaveData> WaveData;
		{
			const int32 NumWaterBodies =  WaterBodyManager->NumWaterBodies();
			// Pre-set up to the max water body index. Some entries may be empty and NumWaterBodies != MaxIndex
			WaterBodyData.SetNumZeroed(WaterBodyManager->MaxWaterBodyIndex());

			TMap<const UGerstnerWaterWaves*, int32> GerstnerWavesIndices;

			WaterBodyManager->ForEachWaterBodyComponent([&WaterBodyData, &WaveData, &GerstnerWavesIndices](UWaterBodyComponent* WaterBodyComponent)
			{
				const int32 WaterZoneIndex = WaterBodyComponent->GetWaterZone() ? WaterBodyComponent->GetWaterZone()->GetWaterZoneIndex() : -1;

				const FVector FixedVelocity = WaterBodyComponent->GetConstantVelocity();

				check(WaterBodyComponent->GetWaterBodyIndex() < WaterBodyData.Num());
				FWaterBodyData& WaterBodyDataEntry = WaterBodyData[WaterBodyComponent->GetWaterBodyIndex()];
				WaterBodyDataEntry.WaterZoneIndex = WaterZoneIndex;
				WaterBodyDataEntry.TargetWaveMaskDepth = WaterBodyComponent->TargetWaveMaskDepth;
				WaterBodyDataEntry.FixedVelocityXY = FMath::AsFloat(static_cast<uint32>(FFloat16(FixedVelocity.X).Encoded) | static_cast<uint32>(FFloat16(FixedVelocity.Y).Encoded) << 16u);
				WaterBodyDataEntry.FixedVelocityZ = static_cast<float>(FixedVelocity.Z);
				WaterBodyDataEntry.FixedZHeight = WaterBodyComponent->GetConstantSurfaceZ();
				WaterBodyDataEntry.FixedWaterDepth = WaterBodyComponent->GetConstantDepth();

				if (WaterBodyComponent->HasWaves())
				{
					const UWaterWavesBase* WaterWavesBase = WaterBodyComponent->GetWaterWaves();
					check(WaterWavesBase != nullptr);
					if (const UGerstnerWaterWaves* GerstnerWaves = Cast<const UGerstnerWaterWaves>(WaterWavesBase->GetWaterWaves()))
					{
						int32* WaveDataIndex = GerstnerWavesIndices.Find(GerstnerWaves);

						if (WaveDataIndex == nullptr)
						{
							// Where the data for this set of waves starts
							const int32 WaveDataBase = WaveData.Num();

							WaveDataIndex = &GerstnerWavesIndices.Add(GerstnerWaves, WaveDataBase);

							// Some max value
							constexpr int32 MaxWavesPerGerstnerWaves = 4096;

							const TArray<FGerstnerWave>& Waves = GerstnerWaves->GetGerstnerWaves();
							
							// Allocate for the waves in this water body
							const int32 NumWaves = FMath::Min(Waves.Num(), MaxWavesPerGerstnerWaves);
							WaveData.AddZeroed(NumWaves);

							for (int32 WaveIndex = 0; WaveIndex < NumWaves; WaveIndex++)
							{
								const uint32 WavesDataIndex = WaveDataBase + WaveIndex;
								WaveData[WavesDataIndex] = FGerstnerWaveData(Waves[WaveIndex]);
							}
						}

						const TArray<FGerstnerWave>& Waves = GerstnerWaves->GetGerstnerWaves();

						check(WaveDataIndex);

						WaterBodyDataEntry.WaveDataIndex = *WaveDataIndex;
						WaterBodyDataEntry.NumWaves = Waves.Num();
					}
				}
				return true;
			});
		}

		TArray<FVector4f> WaterBodyDataBuffer;
		TArray<FVector4f> WaterAuxDataBuffer;

		// The first element of the WaterDataBuffer contains the offsets to each of the sub-buffers and the number of view data.
		// X = WaterZoneDataOffset
		// Y = WaterWaveDataOffset
		// Z = WaterViewDataOffset
		// W = NumWaterViewData
		WaterAuxDataBuffer.AddZeroed();

		// Transform the individual arrays into the single buffer:
		{
			/** Copy a buffer of arbitrary PoD into a float4 resource array. Returns the starting offset of the source buffer in the dest buffer. */
			auto AppendDataToFloat4Buffer = []<typename T>(TArray<FVector4f>& Dest, const TArray<T>& Source)
			{
				constexpr int32 NumFloat4PerElement = (sizeof(T) / sizeof(FVector4f));
				const int32 StartOffset = Dest.Num();
				Dest.AddUninitialized(Source.Num() * NumFloat4PerElement);
				FMemory::Memcpy(
					Dest.GetData() + StartOffset,
					Source.GetData(),
					Source.Num() * sizeof(T));
				return StartOffset;
			};

			AppendDataToFloat4Buffer(WaterBodyDataBuffer, WaterBodyData);

			const int32 ZoneDataOffset = AppendDataToFloat4Buffer(WaterAuxDataBuffer, WaterZoneData);
			const int32 ZoneViewDataOffset = AppendDataToFloat4Buffer(WaterAuxDataBuffer, WaterZoneViewData);
			const int32 WaveDataOffset = AppendDataToFloat4Buffer(WaterAuxDataBuffer, WaveData);

			// Store the offsets to each sub-buffer in the first entry.
			// If this layout ever changes, corresponding decode functions must be updated in GerstnerWaveFunctions.ush!
			FVector4f& OffsetData = WaterAuxDataBuffer[0];
			OffsetData.X = ZoneDataOffset;
			OffsetData.Y = WaveDataOffset;
			OffsetData.Z = ZoneViewDataOffset;
			OffsetData.W = CurrentNumViews;
		}

		if (WaterBodyDataBuffer.Num() == 0)
		{
			WaterBodyDataBuffer.AddZeroed();
		}

		ENQUEUE_RENDER_COMMAND(AllocateWaterInstanceDataBuffer)
		(
			[WaterGPUData=WaterGPUData, WaterAuxDataBuffer, WaterBodyDataBuffer](FRHICommandListImmediate& RHICmdList) mutable
			{
				WaterGPUData->AuxDataBuffer = UE::RHIResourceUtils::CreateBufferFromArray(
					RHICmdList,
					TEXT("WaterAuxDataBuffer"),
					EBufferUsageFlags::VertexBuffer | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Static,
					ERHIAccess::SRVMask,
					MakeConstArrayView(WaterAuxDataBuffer)
				);
				WaterGPUData->AuxDataSRV = RHICmdList.CreateShaderResourceView(
					WaterGPUData->AuxDataBuffer, 
					FRHIViewDesc::CreateBufferSRV()
						.SetType(FRHIViewDesc::EBufferType::Typed)
						.SetFormat(PF_A32B32G32R32F));
				
				WaterGPUData->WaterBodyDataBuffer = UE::RHIResourceUtils::CreateBufferFromArray(
					RHICmdList,
					TEXT("WaterBodyDataBuffer"),
					EBufferUsageFlags::VertexBuffer | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Static,
					ERHIAccess::SRVMask,
					MakeConstArrayView(WaterBodyDataBuffer)
				);
				WaterGPUData->WaterBodyDataSRV = RHICmdList.CreateShaderResourceView(
					WaterGPUData->WaterBodyDataBuffer, 
					FRHIViewDesc::CreateBufferSRV()
						.SetType(FRHIViewDesc::EBufferType::Typed)
						.SetFormat(PF_A32B32G32R32F));
			}
		);

		bRebuildGPUData = false;
	}
}

int32 FWaterViewExtension::GetOrAddViewindex(const FSceneView& InView)
{
	const int32 ViewPlayerIndex = (InView.PlayerIndex != INDEX_NONE) ? InView.PlayerIndex : 0;

	return ViewPlayerIndices.AddUnique(ViewPlayerIndex);
}

int32 FWaterViewExtension::GetViewIndex(int32 PlayerIndex) const
{
	int32 OutIndex = INDEX_NONE;

	if (ViewPlayerIndices.Find(PlayerIndex, OutIndex))
	{
		return OutIndex;
	}

	return OutIndex;
}

int32 FWaterViewExtension::GetViewIndex(const FSceneView& InView) const
{
	const int32 PlayerIndex = (InView.PlayerIndex != INDEX_NONE) ? InView.PlayerIndex : 0;
	return GetViewIndex(PlayerIndex);
}

void FWaterViewExtension::SetupViewFamily(FSceneViewFamily& InViewFamily)
{

}

void FWaterViewExtension::UpdateViewInfo(AWaterZone* WaterZone, const FSceneView& InView)
{
	const int32 ViewPlayerIndex = GetViewIndex(InView);
	check(ViewPlayerIndex != INDEX_NONE);

	check(WaterZone != nullptr);

	const FVector ViewLocation = InView.ViewLocation;

	FWaterZoneInfo* WaterZoneInfo = WaterZoneInfos.Find(WaterZone);
	if (ensureMsgf(WaterZoneInfo != nullptr, TEXT("We are trying to render a water info texture for a water zone that is not registered!")))
	{
		FWaterZoneInfo::FWaterZoneViewInfo& WaterZoneViewInfo = WaterZoneInfo->ViewInfos[ViewPlayerIndex];

		if (WaterZone->IsLocalOnlyTessellationEnabled())
		{
			UWaterMeshComponent* WaterMesh = WaterZone->GetWaterMeshComponent();
			check(WaterMesh);

			const double TileSize = WaterMesh->GetTileSize();

			// UWaterMeshComponent::GetGlobalWaterMeshCenter()  already does some snapping, also taking into account
			// r.Water.WaterMesh.LODCountBias logic for the current sliding window. Should we apply this here too? 
			// TODO: Look into the above.
			FVector SlidingWindowCenter = ViewLocation.GridSnap(TileSize);

			const FVector2D SlidingWindowHalfExtent(WaterZone->GetDynamicWaterInfoExtent() / 2.0);

			WaterZoneViewInfo.Center = SlidingWindowCenter;

			// Trigger the next update when the camera is <UpdateMargin> units away from the border of the current window.
			const FVector2D UpdateMargin(CVarLocalTessellationUpdateMargin.GetValueOnGameThread());

			// Keep a minimum of <1., 1.> bounds to avoid updating every frame if the update margin is larger than the zone.
			const FVector2D UpdateExtents = FVector2D::Max(FVector2D(1., 1.), SlidingWindowHalfExtent - UpdateMargin);
			WaterZoneViewInfo.UpdateBounds.Emplace(FVector2D(SlidingWindowCenter) - UpdateExtents, FVector2D(SlidingWindowCenter) + UpdateExtents);

			// Mark GPU data dirty since we have a new WaterArea parameter and need to push this to water bodies.
			MarkGPUDataDirty();
		}
		else
		{
			WaterZoneViewInfo.UpdateBounds.Reset();
			WaterZoneViewInfo.Center = WaterZone->GetActorLocation();
			MarkGPUDataDirty();
		}
	}
}

void FWaterViewExtension::RenderWaterInfoTexture(FSceneViewFamily& InViewFamily, FSceneView& InView, const FWaterZoneInfo* WaterZoneInfo, FSceneInterface* Scene, const FVector& ZoneCenter)
{
	const int32 WaterInfoRenderMethod = CVarWaterInfoRenderMethod.GetValueOnGameThread();

	int32 ViewPlayerIndex = GetViewIndex(InView);
	check(ViewPlayerIndex != INDEX_NONE);

	const UE::WaterInfo::FRenderingContext& Context(WaterZoneInfo->RenderContext);
	
	// Render the water info texture using custom render pass method
	if (WaterInfoRenderMethod == 2)
	{
		UE::WaterInfo::UpdateWaterInfoRendering_CustomRenderPass(Scene, InViewFamily, Context, ViewPlayerIndex, ZoneCenter);
	}
	// Rendering is done in a separate pass when rendering the main view
	else if (WaterInfoRenderMethod == 1)
	{
		UE::WaterInfo::UpdateWaterInfoRendering2(InView, Context, ViewPlayerIndex, ZoneCenter);
	}
	else if (WaterInfoRenderMethod == 0)
	{
		UE_LOGF(LogWater, Error, "Water Info Render Method 0 is deprecated and no longer functions! Please set r.Water.WaterInfo.RenderMethod to either 1 or 2");
	}
}

void FWaterViewExtension::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	if (CVarLocalTessellationFreeze.GetValueOnGameThread())
	{
		return;
	}

	const FVector ViewLocation = InView.ViewLocation;

	const TWeakObjectPtr<UWorld> WorldPtr = GetWorld();

	if (!ensureMsgf(WorldPtr.IsValid(), TEXT("FWaterViewExtension::SetupView was called while its owning world is not valid! Lifetime of the WaterViewExtension is tied to the world, this should be impossible!")))
	{
		return;
	}

	int32 NumViews = 1;

	if (WorldPtr->GetGameInstance())
	{
		NumViews = WorldPtr->GetGameInstance()->GetLocalPlayers().Num();
	}

	// Prevent re-entrancy. 
	// Since the water info render will update the view extensions we could end up with a re-entrant case.
	static bool bUpdatingWaterInfo = false;
	if (bUpdatingWaterInfo)
	{
		return;
	}
	bUpdatingWaterInfo = true;
	ON_SCOPE_EXIT { bUpdatingWaterInfo = false; };

	int32 ViewPlayerIndex = GetOrAddViewindex(InView);

	FSceneInterface* Scene = WorldPtr.Get()->Scene;
	check(Scene != nullptr);

	// if a texture rebuild is pending, we need to wait for that to be done by the WaterZone, so skip any view update while that is completed.
	if (bWaterInfoTextureRebuildPending)
	{
		ViewPlayerIndices.Empty();

		TWeakPtr<FWaterViewExtension, ESPMode::ThreadSafe> WaterViewExtension = UWaterSubsystem::GetWaterViewExtensionWeakPtr(WorldPtr.Get());

		ENQUEUE_RENDER_COMMAND(WaterViewExtensionNonDataViewsQuadtreeKeysReset)(
			[WaterViewExtension](FRHICommandList& THICmdList)
			{
				if (TSharedPtr<FWaterViewExtension, ESPMode::ThreadSafe> WaterViewExtensionPtr = WaterViewExtension.Pin(); WaterViewExtensionPtr.IsValid())
				{
					WaterViewExtensionPtr->NonDataViewsQuadtreeKeys.Empty();
				}
			});
	}
	else
	{
		if (ShouldHaveWaterZoneViewData(InView))
		{
			if (CurrentNumViews != NumViews)
			{
				for (AWaterZone* WaterZone : TActorRange<AWaterZone>(WorldPtr.Get()))
				{
					if (WaterZone->HasActorRegisteredAllComponents())
					{
						FWaterZoneInfo& WaterZoneInfo = WaterZoneInfos.FindChecked(WaterZone);

						// make sure that if !IsLocalOnlyTessellationEnabled we only create a single WaterInfoTexture slice.
						WaterZone->WaterInfoTextureArrayNumSlices = WaterZone->IsLocalOnlyTessellationEnabled() ? NumViews : 1;
						// Mark for rebuild the WaterZone if the size changes
						WaterZone->MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture);

						// init per view info

						WaterZoneInfo.ViewInfos.Empty();

						for (int i = 0; i < NumViews; ++i)
						{
							WaterZoneInfo.ViewInfos.Emplace(FWaterZoneInfo::FWaterZoneViewInfo());
						}
					}

					UE_LOGF(LogWater, Verbose, "Number of views changed. Water Zone (%ls) ViewInfos was reset.", *GetNameSafe(WaterZone));
				}

				CurrentNumViews = NumViews;
				bWaterInfoTextureRebuildPending = true;

				return;
			}

			// Check if the view location is no longer within the current update bounds of a water zone and if so, queue an update for it.
			for (AWaterZone* WaterZone : TActorRange<AWaterZone>(WorldPtr.Get()))
			{
				if (WaterZone->HasActorRegisteredAllComponents())
				{
					FWaterZoneInfo& WaterZoneInfo = WaterZoneInfos.FindChecked(WaterZone);

					if (WaterZone->WaterInfoTextureArray.Get() == nullptr)
					{
						continue;
					}

					if (CVarDrawPerViewDebugInfo.GetValueOnGameThread())
					{
						DrawDebugInfo(InView, WaterZone);
					}

					FWaterZoneInfo::FWaterZoneViewInfo& WaterZoneViewInfo = WaterZoneInfo.ViewInfos[ViewPlayerIndex];

					bool bBoundsUpdateNeeded = WaterZone->IsLocalOnlyTessellationEnabled() && (!WaterZoneViewInfo.UpdateBounds.IsSet() || !WaterZoneViewInfo.UpdateBounds->IsInside(FVector2D(ViewLocation)));
					bBoundsUpdateNeeded |= WaterZoneViewInfo.bIsDirty;
					bBoundsUpdateNeeded |= bForceBoundsUpdate;

					if (bBoundsUpdateNeeded)
					{
						UpdateViewInfo(WaterZone, InView);

						UE_LOGF(LogWater, Verbose, "Water Zone (%ls) ViewInfo for view %d updated.", *GetNameSafe(WaterZone), ViewPlayerIndex);

						// make sure that if !IsLocalOnlyTessellationEnabled, we only update the WaterInfoTexture for a single view
						if (WaterZone->IsLocalOnlyTessellationEnabled() || ViewPlayerIndex == 0)
						{
							RenderWaterInfoTexture(InViewFamily, InView, &WaterZoneInfo, Scene, WaterZoneViewInfo.Center);
						}

						WaterZoneViewInfo.bShouldUpdateQuadtree = true;
						bAnyQuadTreeUpdateRequired = true;
					}

					{
						UWaterMeshComponent* WaterMeshComponent = WaterZone->GetWaterMeshComponent();
						check(WaterMeshComponent != nullptr);

						FWaterMeshSceneProxy* SceneProxy = static_cast<FWaterMeshSceneProxy*>(WaterMeshComponent->GetSceneProxy());

						if (SceneProxy != nullptr)
						{
							bAnyQuadTreeUpdateRequired |= (SceneProxy != WaterZoneViewInfo.OldSceneProxy);
						}
					}

					WaterZoneViewInfo.bIsDirty = false;
				}
			}
		}

		if (bRequestForcedBoundsUpdate)
		{
			bForceBoundsUpdate = true;
			bRequestForcedBoundsUpdate = false;

			UE_LOGF(LogWater, Verbose, "Forced Bounds Update requested for view %d.", ViewPlayerIndex);
		}
		else
		{
			bForceBoundsUpdate = false;
		}
	}
	

	// The logic in UpdateGPUBuffers() used to be done in SetupViewFamily(). However, SetupView() (which is responsible for water info rendering) potentially modifies the WaterZone but is called after SetupViewFamily().
	// This can lead to visual artifacts due to outdated data in the GPU buffers.
	
	UpdateGPUBuffers();
}

void FWaterViewExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	if (bWaterInfoTextureRebuildPending)
	{
		return;
	}

	const TWeakObjectPtr<UWorld> WorldPtr = GetWorld();

	if (!ensureMsgf(WorldPtr.IsValid(), TEXT("FWaterViewExtension::BeginRenderViewFamily was called while it's owning world is not valid! Lifetime of the WaterViewExtension is tied to the world, this should be impossible!")))
	{
		return;
	}

	bool bHadWaterZoneViewData = false;

	for (const FSceneView* View : InViewFamily.Views)
	{
		// Warning: Do not capture View in ENQUEUE_RENDER_COMMAND lambdas, since the RT's view hasn't been created yet,
		// so we would pass the GT view, which then would end up being a dangling pointer when accessed by the RT.

		if (ShouldHaveWaterZoneViewData(*View))
		{
			bHadWaterZoneViewData = true;

			if (!bAnyQuadTreeUpdateRequired)
			{
				continue;
			}

			for (AWaterZone* WaterZone : TActorRange<AWaterZone>(WorldPtr.Get()))
			{
				int32 ViewPlayerIndex = GetViewIndex(*View);
				check(ViewPlayerIndex != INDEX_NONE);

				if (WaterZone->HasActorRegisteredAllComponents())
				{
					FWaterZoneInfo& WaterZoneInfo = WaterZoneInfos.FindChecked(WaterZone);

					if (WaterZone->WaterInfoTextureArray.Get() == nullptr)
					{
						continue;
					}

					FWaterZoneInfo::FWaterZoneViewInfo& WaterZoneViewInfo = WaterZoneInfo.ViewInfos[ViewPlayerIndex];

					UWaterMeshComponent* WaterMeshComponent = WaterZone->GetWaterMeshComponent();
					check(WaterMeshComponent != nullptr);

					FWaterMeshSceneProxy* SceneProxy = static_cast<FWaterMeshSceneProxy*>(WaterMeshComponent->GetSceneProxy());

					if (SceneProxy != nullptr)
					{
						// push a quadtree update
						if (WaterZone->IsLocalOnlyTessellationEnabled() && (WaterZoneViewInfo.bShouldUpdateQuadtree || SceneProxy != WaterZoneViewInfo.OldSceneProxy))
						{
							int32 QuadtreeKey = View->PlayerIndex;
							FVector2D QuadtreeLocation = WaterZoneViewInfo.UpdateBounds->GetCenter();

							ENQUEUE_RENDER_COMMAND(WaterViewExtensionQuadtreeUpdate)(
								[SceneProxy, QuadtreeKey, QuadtreeLocation](FRHICommandList& THICmdList)
								{
									// TODO: We could add a CreateOrUpdateViewWaterQuadTree to WaterSceneProxy
									// if the creation fails it means a quadtree for the current view already exists, so we just update it
									if (!SceneProxy->CreateViewWaterQuadTree(QuadtreeKey, QuadtreeLocation))
									{
										bool bResult = SceneProxy->UpdateViewWaterQuadTree(QuadtreeKey, QuadtreeLocation);

										check(bResult);
									}
								});

							UE_LOGF(LogWater, Verbose, "Water Zone (%ls) queued a quadtree update for view %d updated.", *GetNameSafe(WaterZone), ViewPlayerIndex);
							
							WaterZoneViewInfo.bShouldUpdateQuadtree = false;
						}
					}

					WaterZoneViewInfo.OldSceneProxy = SceneProxy;
				}
			}
		}
		else
		{
			// if !bShouldHaveWaterZoneViewData, we want to store the closest quadtree ID for this view (since it is not
			// going to have its own quadtree associated with it), so we can assign the WaterInfoTexture index of the view assigned to that quadtree
			for (AWaterZone* WaterZone : TActorRange<AWaterZone>(WorldPtr.Get()))
			{
				if (WaterZone->WaterInfoTextureArray.Get() == nullptr)
				{
					continue;
				}

				if (WaterZone->HasActorRegisteredAllComponents())
				{
					if (WaterZone->IsLocalOnlyTessellationEnabled())
					{
						UWaterMeshComponent* WaterMeshComponent = WaterZone->GetWaterMeshComponent();
						check(WaterMeshComponent != nullptr);

						FWaterMeshSceneProxy* SceneProxy = static_cast<FWaterMeshSceneProxy*>(WaterMeshComponent->GetSceneProxy());
						if (SceneProxy != nullptr)
						{
							const FVector2D ViewPosition2D = FVector2D(View->ViewLocation);
							FSceneViewStateInterface* ViewState = View->State;

							TWeakPtr<FWaterViewExtension, ESPMode::ThreadSafe> WaterViewExtension = UWaterSubsystem::GetWaterViewExtensionWeakPtr(WorldPtr.Get());

							ENQUEUE_RENDER_COMMAND(WaterViewExtensionNonDataViewsQuadtreeKeysUpdate)(
								[WaterViewExtension, SceneProxy, ViewPosition2D, ViewState](FRHICommandList& THICmdList)
								{
									if (TSharedPtr<FWaterViewExtension, ESPMode::ThreadSafe> WaterViewExtensionPtr = WaterViewExtension.Pin(); WaterViewExtensionPtr.IsValid())
									{
										WaterViewExtensionPtr->NonDataViewsQuadtreeKeys.Add(ViewState, SceneProxy->FindBestQuadTreeForViewLocation(ViewPosition2D));
									}
								});

							UE_LOGF(LogWater, Verbose, "Water Zone (%ls) queued a search for the closest quadtree for a View (0x%p) which has no WaterInfo.", *GetNameSafe(WaterZone), View);
						}
					}
				}
			}
		}

	}

	// Only reset the flag if this view family contained views that could consume it.
	// Otherwise, a scene capture's view family (which has no water zone view data) would
	// clear the flag before the main camera's view family gets a chance to process it.
	if (bHadWaterZoneViewData)
	{
		bAnyQuadTreeUpdateRequired = false;
	}
}

bool FWaterViewExtension::ShouldHaveWaterZoneViewData(const FSceneView& InView) const
{
	// Don't dirty the water info texture when we're rendering from a scene capture (unless we're in a commandlet in order to support world partition minimap captures).
	// Due to the frame delay after marking the texture as dirty, scene captures wouldn't have the right texture anyways.
	// #todo_water [roey]: Once we have no frame-delay for updating the texture and lesser performance impact, we can re-enable updates within scene captures.
	return (!InView.bIsSceneCapture || IsRunningCommandlet()) && !InView.bIsSceneCaptureCube && !InView.bIsReflectionCapture && !InView.bIsPlanarReflection && !InView.bIsVirtualTexture 
		// Also don't update water info texture when rendering hit proxies as it bypasses custom render passes 
		&& !InView.Family->EngineShowFlags.HitProxies;
}

void FWaterViewExtension::DrawDebugInfo(const FSceneView& InView, AWaterZone* WaterZone)
{
	if (GEngine)
	{
		FWaterZoneInfo& WaterZoneInfo = WaterZoneInfos.FindChecked(WaterZone);

		const FVector ViewLocation = InView.ViewLocation;
		int32 ViewPlayerIndex = GetViewIndex(InView);
		check(ViewPlayerIndex != INDEX_NONE);

		FWaterZoneInfo::FWaterZoneViewInfo& WaterZoneViewInfo = WaterZoneInfo.ViewInfos[ViewPlayerIndex];

		FVector2D Center = FVector2D::Zero();
		FVector2D Extents = FVector2D::Zero();
		FVector SnappedCenter = WaterZoneViewInfo.Center;

		if (WaterZoneViewInfo.UpdateBounds.IsSet())
		{
			WaterZoneViewInfo.UpdateBounds.GetValue().GetCenterAndExtents(Center, Extents);
		}

		FColor Colors[4] = { FColor::Yellow, FColor::Green, FColor::Red, FColor::Blue };

		if (ViewPlayerIndex < 4)
		{
			int32 StartingOffset = ViewPlayerIndex * 3;
			GEngine->AddOnScreenDebugMessage(StartingOffset, 0.01f, Colors[ViewPlayerIndex], FString::Printf(TEXT("View %d Location: %f, %f, %f"), ViewPlayerIndex, ViewLocation.X, ViewLocation.Y, ViewLocation.Z));
			GEngine->AddOnScreenDebugMessage(StartingOffset + 1, 0.01f, Colors[ViewPlayerIndex], FString::Printf(TEXT("Center: %f, %f; Extents: %f, %f"), Center.X, Center.Y, Extents.X, Extents.Y));
			GEngine->AddOnScreenDebugMessage(StartingOffset + 2, 0.01f, Colors[ViewPlayerIndex], FString::Printf(TEXT("Snapped Center: %f, %f"), SnappedCenter.X, SnappedCenter.Y));
		}
	}
}

void FWaterViewExtension::PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily)
{
	
}

void FWaterViewExtension::PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView)
{
	if (WaterGPUData->WaterBodyDataSRV && WaterGPUData->AuxDataSRV)
	{
		// TODO: Rename members on FSceneView in a separate CL. This will invalidate almost all shaders.
		InView.WaterDataBuffer = WaterGPUData->AuxDataSRV;
		InView.WaterIndirectionBuffer = WaterGPUData->WaterBodyDataSRV;
	}

	{
		int32 WaterInfoTextureIndex = GetViewIndex(InView);

		// check if this view is one of the queues without its own WaterZoneViewData, if it is assign the 
		// WaterInfoTextureIndex corresponding to the closest quadtree index we stored in NonDataViewsQuadtreeKeys
		const int32* QuadtreeKey = NonDataViewsQuadtreeKeys.Find(InView.State);
		if (QuadtreeKey != nullptr)
		{
			// swap the index with the one corresponding to the closest quadtree
			WaterInfoTextureIndex = *QuadtreeKey;
		}

		WaterInfoTextureIndex = (WaterInfoTextureIndex != INDEX_NONE) ? WaterInfoTextureIndex : 0;

		InView.WaterInfoTextureViewIndex = WaterInfoTextureIndex;
	}
}

void FWaterViewExtension::PreRenderBasePass_RenderThread(FRDGBuilder& GraphBuilder, bool bDepthBufferIsPopulated)
{
	for (FWaterMeshGPUWork::FCallback& Callback : GWaterMeshGPUWork.Callbacks)
	{
		Callback.Function(GraphBuilder, bDepthBufferIsPopulated);
	}
}

void FWaterViewExtension::MarkWaterInfoTextureForRebuild(const UE::WaterInfo::FRenderingContext& RenderContext)
{
	MarkGPUDataDirty();

	bWaterInfoTextureRebuildPending = false;

	// this should mark dirty the RenderContext.ZoneToRender waterzone info

	FWaterZoneInfo* WaterZoneInfo = WaterZoneInfos.Find(RenderContext.ZoneToRender);

	if (WaterZoneInfo != nullptr)
	{
		WaterZoneInfo->RenderContext = RenderContext;

		for (int i = 0; i < WaterZoneInfo->ViewInfos.Num(); ++i)
		{
			WaterZoneInfo->ViewInfos[i].bIsDirty = true;
		}
	}
}

void FWaterViewExtension::MarkGPUDataDirty()
{
	bRebuildGPUData = true;
}

void FWaterViewExtension::AddWaterZone(AWaterZone* InWaterZone)
{
	check(!WaterZoneInfos.Contains(InWaterZone));
	FWaterZoneInfo& WaterZoneInfo = WaterZoneInfos.Emplace(InWaterZone);

	// init per view info

	CurrentNumViews = 0;
	WaterZoneInfo.ViewInfos.Emplace(FWaterZoneInfo::FWaterZoneViewInfo());

	UE_LOGF(LogWater, Verbose, "Water Zone (%ls): AddWaterZone was called.", *GetNameSafe(InWaterZone));
}

void FWaterViewExtension::RemoveWaterZone(AWaterZone* InWaterZone)
{
	WaterZoneInfos.FindAndRemoveChecked(InWaterZone);

	UE_LOGF(LogWater, Verbose, "Water Zone (%ls): RemoveWaterZone was called.", *GetNameSafe(InWaterZone));
}

bool FWaterViewExtension::GetZoneLocation(const AWaterZone* InWaterZone, int32 PlayerIndex, FVector& OutLocation) const
{
	if (ensure(InWaterZone->HasActorRegisteredAllComponents()))
	{
		const FWaterZoneInfo* WaterZoneInfo = WaterZoneInfos.Find(InWaterZone);
		if (ensure(WaterZoneInfo))
		{
			int32 ViewPlayerIndex = GetViewIndex(PlayerIndex);

			if (ViewPlayerIndex != INDEX_NONE && ViewPlayerIndex < WaterZoneInfo->ViewInfos.Num())
			{
				OutLocation = WaterZoneInfo->ViewInfos[ViewPlayerIndex].Center;
				return true;
			}
		}
	}

	UE_LOGF(LogWater, Verbose, "Water Zone (%ls) called FWaterViewExtension::GetZoneLocation() but didn't get a valid location because of missing/uninitialized WaterZoneInfo->ViewInfos.", *GetNameSafe(InWaterZone));

	return false;
}

void FWaterViewExtension::CreateSceneProxyQuadtrees(FWaterMeshSceneProxy* SceneProxy)
{
	for (auto& Pair : QuadTreeKeyLocationMap)
	{
		SceneProxy->CreateViewWaterQuadTree(Pair.Key, Pair.Value);
	}
}
