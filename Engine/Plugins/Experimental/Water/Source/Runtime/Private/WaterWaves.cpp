// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterWaves.h"
#include "GerstnerWaterWaves.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterWaves)

#if WITH_EDITOR
void UWaterWavesBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnUpdateWavesData.Broadcast(this, PropertyChangedEvent.ChangeType);
}
#endif // WITH_EDITOR


// ----------------------------------------------------------------------------------

UWaterWavesAsset::UWaterWavesAsset()
{
	// Default wave source
	WaterWaves = CreateDefaultSubobject<UGerstnerWaterWaves>(TEXT("WaterWaves"), /* bTransient = */false);
}

void UWaterWavesAsset::SetWaterWaves(UWaterWaves* InWaterWaves)
{
	if (InWaterWaves != WaterWaves)
	{
#if WITH_EDITOR
		RegisterOnUpdateWavesData(WaterWaves, /*bRegister = */false);
#endif // WITH_EDITOR

		WaterWaves = InWaterWaves;

#if WITH_EDITOR
		RegisterOnUpdateWavesData(WaterWaves, /*bRegister = */true);
		OnUpdateWavesAssetData.Broadcast(this, EPropertyChangeType::ValueSet);
#endif // WITH_EDITOR		
	}
}

void UWaterWavesAsset::BeginDestroy()
{
	Super::BeginDestroy();

#if WITH_EDITOR
	RegisterOnUpdateWavesData(WaterWaves, /*bRegister = */false);
#endif // WITH_EDITOR		
}

#if WITH_EDITOR
void UWaterWavesAsset::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	RegisterOnUpdateWavesData(WaterWaves, /*bRegister = */true);
}

void UWaterWavesAsset::PostLoad()
{
	Super::PostLoad();

	RegisterOnUpdateWavesData(WaterWaves, /*bRegister = */true);
}

void UWaterWavesAsset::PreEditUndo()
{
	Super::PreEditUndo();

	// On undo, when PreEditChange is called, PropertyAboutToChange is nullptr so we need to unregister from the previous object here :
	RegisterOnUpdateWavesData(WaterWaves, /*bRegister = */false);
}

void UWaterWavesAsset::PostEditUndo()
{
	Super::PostEditUndo();

	// On undo, when PostEditChangeProperty is called, PropertyChangedEvent is fake so we need to register to the new object here :
	RegisterOnUpdateWavesData(WaterWaves, /*bRegister = */true);
}

void UWaterWavesAsset::PreEditChange(FProperty* PropertyAboutToChange) 
{
	Super::PreEditChange(PropertyAboutToChange);

	const FName PropertyName = PropertyAboutToChange ? PropertyAboutToChange->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaterWavesAsset, WaterWaves))
	{
		RegisterOnUpdateWavesData(WaterWaves, /*bRegister = */false);
	}
}


void UWaterWavesAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UWaterWavesAsset, WaterWaves))
	{
		RegisterOnUpdateWavesData(WaterWaves, /*bRegister = */true);
	}

	OnUpdateWavesAssetData.Broadcast(this, PropertyChangedEvent.ChangeType);
}

void UWaterWavesAsset::OnWavesDataUpdated(UWaterWavesBase* InWaterWaves, EPropertyChangeType::Type InChangeType)
{
	// There was a data change on our internal data, just forward the event : 
	OnUpdateWavesAssetData.Broadcast(this, InChangeType);
}

void UWaterWavesAsset::RegisterOnUpdateWavesData(UWaterWaves* InWaterWaves, bool bRegister)
{
	if (InWaterWaves != nullptr)
	{
		if (bRegister)
		{
			InWaterWaves->OnUpdateWavesData.AddUObject(this, &UWaterWavesAsset::OnWavesDataUpdated);
		}
		else
		{
			InWaterWaves->OnUpdateWavesData.RemoveAll(this);
		}
	}
}

#endif // WITH_EDITOR


// ----------------------------------------------------------------------------------

void UWaterWavesAssetReference::SetWaterWavesAsset(UWaterWavesAsset* InWaterWavesAsset)
{
	if (InWaterWavesAsset != WaterWavesAsset)
	{
#if WITH_EDITOR
		RegisterOnUpdateWavesAssetData(WaterWavesAsset, /*bRegister = */false);
#endif // WITH_EDITOR

		WaterWavesAsset = InWaterWavesAsset;

#if WITH_EDITOR
		RegisterOnUpdateWavesAssetData(WaterWavesAsset, /*bRegister = */true);
		OnUpdateWavesData.Broadcast(this, EPropertyChangeType::ValueSet);
#endif // WITH_EDITOR	
	}
}

void UWaterWavesAssetReference::BeginDestroy()
{
	Super::BeginDestroy();

#if WITH_EDITOR
	RegisterOnUpdateWavesAssetData(WaterWavesAsset, /*bRegister = */false);
#endif // WITH_EDITOR		
}

float UWaterWavesAssetReference::GetMaxWaveHeight() const
{
	if (const UWaterWaves* Waves = GetWaterWaves())
	{
		return Waves->GetMaxWaveHeight();
	}

	return 0.0f;
}

float UWaterWavesAssetReference::GetWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime, FVector& OutNormal) const
{
	if (const UWaterWaves* Waves = GetWaterWaves())
	{
		return Waves->GetWaveHeightAtPosition(InPosition, InWaterDepth, InTime, OutNormal);
	}

	return 0.0f;
}

float UWaterWavesAssetReference::GetSimpleWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime) const
{
	if (const UWaterWaves* Waves = GetWaterWaves())
	{
		return Waves->GetSimpleWaveHeightAtPosition(InPosition, InWaterDepth, InTime);
	}

	return 0.0f;
}

float UWaterWavesAssetReference::GetWaveAttenuationFactor(const FVector& InPosition, float InWaterDepth, float InTargetWaveMaskDepth) const
{
	if (const UWaterWaves* Waves = GetWaterWaves())
	{
		return Waves->GetWaveAttenuationFactor(InPosition, InWaterDepth, InTargetWaveMaskDepth);
	}

	return 1.0f;
}

#if WITH_EDITOR
void UWaterWavesAssetReference::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	RegisterOnUpdateWavesAssetData(WaterWavesAsset, /*bRegister = */true);
}

void UWaterWavesAssetReference::PostLoad()
{
	Super::PostLoad();

	RegisterOnUpdateWavesAssetData(WaterWavesAsset, /*bRegister = */true);
}

void UWaterWavesAssetReference::PreEditUndo()
{
	Super::PreEditUndo();

	// On undo, when PreEditChange is called, PropertyAboutToChange is nullptr so we need to unregister from the previous object here :
	RegisterOnUpdateWavesAssetData(WaterWavesAsset, /*bRegister = */false);
}

void UWaterWavesAssetReference::PostEditUndo()
{
	Super::PostEditUndo();

	// On undo, when PostEditChangeProperty is called, PropertyChangedEvent is fake so we need to register to the new object here :
	RegisterOnUpdateWavesAssetData(WaterWavesAsset, /*bRegister = */true);
}

void UWaterWavesAssetReference::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	const FName PropertyName = PropertyAboutToChange ? PropertyAboutToChange->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaterWavesAssetReference, WaterWavesAsset))
	{
		RegisterOnUpdateWavesAssetData(WaterWavesAsset, /*bRegister = */false);
	}
}

void UWaterWavesAssetReference::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UWaterWavesAssetReference, WaterWavesAsset))
	{
		RegisterOnUpdateWavesAssetData(WaterWavesAsset, /*bRegister = */true);
	}

	OnUpdateWavesData.Broadcast(this, PropertyChangedEvent.ChangeType);
}

void UWaterWavesAssetReference::OnWavesAssetDataUpdated(UWaterWavesAsset* InWaterWavesAsset, EPropertyChangeType::Type InChangeType)
{
	// There was a data change on our referenced data, just forward the event : 
	OnUpdateWavesData.Broadcast(this, InChangeType);
}

void UWaterWavesAssetReference::RegisterOnUpdateWavesAssetData(UWaterWavesAsset* InWaterWavesAsset, bool bRegister)
{
	if (InWaterWavesAsset != nullptr)
	{
		if (bRegister)
		{
			InWaterWavesAsset->OnUpdateWavesAssetData.AddUObject(this, &UWaterWavesAssetReference::OnWavesAssetDataUpdated);
		}
		else
		{
			InWaterWavesAsset->OnUpdateWavesAssetData.RemoveAll(this);
		}
	}
}
#endif // WITH_EDITOR
