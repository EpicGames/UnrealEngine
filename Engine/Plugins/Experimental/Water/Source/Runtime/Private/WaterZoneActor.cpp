// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterZoneActor.h"
#include "Engine/TextureRenderTarget2D.h"
#include "WaterModule.h"
#include "WaterSubsystem.h"
#include "WaterMeshComponent.h"
#include "WaterBodyActor.h"
#include "EngineUtils.h"
#include "LandscapeComponent.h"
#include "LandscapeProxy.h"
#include "LandscapeSubsystem.h"
#include "WaterUtils.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "WaterViewExtension.h"
#include "Algo/AnyOf.h"
#include "Engine/GameViewportClient.h"
#include "WaterBodyInfoMeshComponent.h"
#include "WaterTerrainComponent.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterZoneActor)

#if WITH_EDITOR
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "WaterIconHelper.h"
#include "Components/BoxComponent.h"
#include "WaterZoneActorDesc.h"
#endif // WITH_EDITOR

static int32 ForceUpdateWaterInfoNextFrames = 0;
static FAutoConsoleVariableRef CVarForceUpdateWaterInfoNextFrames(
	TEXT("r.Water.WaterInfo.ForceUpdateWaterInfoNextFrames"),
	ForceUpdateWaterInfoNextFrames,
	TEXT("Force the water info texture to regenerate on the next N frames. A negative value will force update every frame."));

TAutoConsoleVariable<float> CVarWaterFallbackDepth(
	TEXT("r.Water.FallbackDepth"),
	3000.0f,
	TEXT("Depth to use for all water when there are no ground actors defined."),
	ECVF_Default);

void OnWaterInfoRenderTargetResolutionMaxChanged_Callback(IConsoleVariable*);

TAutoConsoleVariable<int32> CVarWaterInfoRenderTargetResolutionMax(
	TEXT("r.Water.WaterInfo.RenderTargetResolutionMax"),
	0,
	TEXT("The maximum resolution of the water zone actor's water info render target. If zero, resolution is uncapped"),
	FConsoleVariableDelegate::CreateStatic(OnWaterInfoRenderTargetResolutionMaxChanged_Callback),
	ECVF_Default);

void OnWaterInfoRenderTargetResolutionMaxChanged_Callback(IConsoleVariable*)
{
	for (AWaterZone* WaterZoneActor : TObjectRange<AWaterZone>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
	{
		if (WaterZoneActor)
		{
			WaterZoneActor->MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture);
		}
	}
}

void OnSkipWaterInfoTextureRenderWhenWorldRenderingDisabled_Callback(IConsoleVariable*);

// HACK [jonathan.bard] : (details underneath)
TAutoConsoleVariable<int32> CVarSkipWaterInfoTextureRenderWhenWorldRenderingDisabled(
	TEXT("r.Water.SkipWaterInfoTextureRenderWhenWorldRenderingDisabled"),
	1,
	TEXT("Use this to prevent the water info from rendering when world rendering is disabled."),
	FConsoleVariableDelegate::CreateStatic(OnSkipWaterInfoTextureRenderWhenWorldRenderingDisabled_Callback),
	ECVF_Default);


// HACK [roey.borsteinas] : This hack piggy-backs off the above hack for disabled world rendering. See details for that first.
// MRQ creates it's own View/ViewFamily when rendering which breaks compatibility with the WaterInfo RenderMethod 1.
// RenderMethod 1 tries to enqueue the water info update directly on the View but since the ULocalPlayer view is discarded before rendering, the water info texture never renders.
void OnSkipWaterInfoTextureRenderWhenWorldRenderingDisabled_Callback(IConsoleVariable*)
{
	static int PreviousWaterInfoRenderMethodValue = 1;
	IConsoleVariable* WaterInfoRenderMethodCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Water.WaterInfo.RenderMethod"));
	if (CVarSkipWaterInfoTextureRenderWhenWorldRenderingDisabled.GetValueOnAnyThread() == 0)
	{
		PreviousWaterInfoRenderMethodValue = WaterInfoRenderMethodCVar->GetInt();
		WaterInfoRenderMethodCVar->SetWithCurrentPriority(2);
	}
	else
	{
		WaterInfoRenderMethodCVar->SetWithCurrentPriority(PreviousWaterInfoRenderMethodValue);
	}

}

AWaterZone::AWaterZone(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, RenderTargetResolution(512, 512)
{
	WaterMesh = CreateDefaultSubobject<UWaterMeshComponent>(TEXT("WaterMesh"));
	SetRootComponent(WaterMesh);
	ZoneExtent = FVector2D(51200., 51200.);
	LocalTessellationExtent = FVector(51200., 51200., 10000.);
	
#if	WITH_EDITOR
	// Setup bounds component
	{
		BoundsComponent = CreateDefaultSubobject<UBoxComponent>(TEXT("BoundsComponent"));
		BoundsComponent->SetCollisionObjectType(ECC_WorldStatic);
		BoundsComponent->SetCollisionResponseToAllChannels(ECR_Ignore);
		BoundsComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		BoundsComponent->SetGenerateOverlapEvents(false);
		BoundsComponent->SetupAttachment(WaterMesh);
		// Bounds component extent is half-extent, ZoneExtent is full extent.
		BoundsComponent->SetBoxExtent(FVector(ZoneExtent / 2., 8192.));
		BoundsComponent->bIsEditorOnly = true;

		FEngineShowFlags BoundsComponentShowFlags = BoundsComponent->GetShowFlags();
		BoundsComponentShowFlags.SetVolumes(true);
		BoundsComponent->SetShowFlags(BoundsComponentShowFlags);
	}

	if (GIsEditor && !IsTemplate())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.OnActorSelectionChanged().AddUObject(this, &AWaterZone::OnActorSelectionChanged);
	}

	ActorIcon = FWaterIconHelper::EnsureSpriteComponentCreated(this, TEXT("/Water/Icons/WaterZoneActorSprite"));
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	TessellatedWaterMeshExtent_DEPRECATED = FVector(35000., 35000., 10000.);

	bIsSpatiallyLoaded = false;
#endif
}

void AWaterZone::SetZoneExtent(FVector2D NewExtent)
{
	ZoneExtent = NewExtent;
	OnExtentChanged();
}

FBox2D AWaterZone::GetZoneBounds2D() const
{
	// GetZoneExtents returns the full extent of the zone but BoxSphereBounds expects a half-extent.
	const FVector2D WaterZoneLocation = FVector2D(GetActorLocation());
	const FVector2D WaterZoneHalfExtent = GetZoneExtent() / 2.0;
	const FBox2D WaterZoneBounds(WaterZoneLocation - WaterZoneHalfExtent, WaterZoneLocation + WaterZoneHalfExtent);

	return WaterZoneBounds;
}

void AWaterZone::GetAllDynamicWaterInfoBounds(TArray<FBox>& OutBounds) const
{
	if (const FWaterViewExtension* WaterViewExtension = UWaterSubsystem::GetWaterViewExtension(GetWorld()))
	{
		const FWaterViewExtension::FWaterZoneInfo* WaterZoneInfo = WaterViewExtension->GetWaterZoneInfos().Find(this);

		if (WaterZoneInfo != nullptr)
		{
			for (const FWaterViewExtension::FWaterZoneInfo::FWaterZoneViewInfo& ViewInfo : WaterZoneInfo->ViewInfos)
			{
				FVector Center(ViewInfo.Center);

				if (!IsLocalOnlyTessellationEnabled())
				{
					Center = GetActorLocation();
				}

				const FVector HalfExtent(GetDynamicWaterInfoExtent() / 2);

				OutBounds.Emplace(FBox(Center - HalfExtent, Center + HalfExtent));
			}
		}
		else
		{
			UE_LOGF(LogWater, Verbose, "AWaterZone (%ls) GetAllDynamicWaterInfoBounds did not find any WaterZoneInfo associated to this WaterZone in FWaterViewExtension", *GetNameSafe(this));
		}
	}
}

void AWaterZone::GetAllDynamicWaterInfoCenters(TArray<FVector>& OutCenters) const
{
	if (const FWaterViewExtension* WaterViewExtension = UWaterSubsystem::GetWaterViewExtension(GetWorld()))
	{
		const FWaterViewExtension::FWaterZoneInfo* WaterZoneInfo = WaterViewExtension->GetWaterZoneInfos().Find(this);

		if (WaterZoneInfo != nullptr)
		{
			for (const FWaterViewExtension::FWaterZoneInfo::FWaterZoneViewInfo& ViewInfo : WaterZoneInfo->ViewInfos)
			{
				FVector Center(ViewInfo.Center);

				if (!IsLocalOnlyTessellationEnabled())
				{
					Center = GetActorLocation();
				}

				OutCenters.Emplace(Center);
			}
		}
		else
		{
			UE_LOGF(LogWater, Verbose, "AWaterZone (%ls) GetAllDynamicWaterInfoCenters did not find any WaterZoneInfo associated to this WaterZone in FWaterViewExtension", *GetNameSafe(this));
		}
	}
}

FBox AWaterZone::GetZoneBounds() const
{
	const FBox2D ZoneBounds2D = GetZoneBounds2D();
	// #todo_water [roey]: Water zone doesn't have an explicit z bounds yet. For now just use the x or y.
	const double StreamingBoundsZ = FMath::Max(ZoneExtent.X, ZoneExtent.Y);
	const FVector ActorLocation = GetActorLocation();

	return FBox(FVector3d(ZoneBounds2D.Min, ActorLocation.Z - StreamingBoundsZ / 2.), FVector3d(ZoneBounds2D.Max, ActorLocation.Z + StreamingBoundsZ / 2.0));}

FIntPoint AWaterZone::GetRenderTargetResolution() const
{
	// Clamp the render resolution if the CVar is set
	const int32 RenderTargetResolutionMax = CVarWaterInfoRenderTargetResolutionMax.GetValueOnGameThread();
	if (RenderTargetResolutionMax > 0)
	{
		return FIntPoint(FMath::Clamp(RenderTargetResolution.X, 0, RenderTargetResolutionMax), FMath::Clamp(RenderTargetResolution.Y, 0, RenderTargetResolutionMax));
	}

	return RenderTargetResolution;
}

void AWaterZone::SetRenderTargetResolution(FIntPoint NewResolution)
{
	RenderTargetResolution = NewResolution;
	MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture, this);
}

void AWaterZone::BeginPlay()
{
	Super::BeginPlay();

	MarkForRebuild(EWaterZoneRebuildFlags::All, this);

	FWorldDelegates::LevelAddedToWorld.AddUObject(this, &AWaterZone::OnLevelAddedToWorld);
	FWorldDelegates::LevelRemovedFromWorld.AddUObject(this, &AWaterZone::OnLevelRemovedFromWorld);
}

void AWaterZone::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);

	Super::EndPlay(EndPlayReason);
}

void AWaterZone::PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph)
{
	// Water mesh component was made new root component. Make sure it doesn't have a parent
	WaterMesh->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);

	Super::PostLoadSubobjects(OuterInstanceGraph);
}

void AWaterZone::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WaterBodyStaticMeshComponents)
	{
		LocalTessellationExtent = TessellatedWaterMeshExtent_DEPRECATED;
		bEnableLocalOnlyTessellation = bEnableNonTesselatedLODMesh_DEPRECATED;
	}
#endif // WITH_EDITORONLY_DATA

	OnExtentChanged();
}

void AWaterZone::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	// PostRegisterAllComponents is called many times in a row during BP reinstancing, property changes, etc.
	// Only register to the water body manager if we haven't already (don't have an index yet).
	FWaterBodyManager* Manager = UWaterSubsystem::GetWaterBodyManager(GetWorld());
	if (Manager && !IsTemplate() && (WaterZoneIndex == INDEX_NONE))
	{
		WaterZoneIndex = Manager->AddWaterZone(this);
	}

#if WITH_EDITOR
	// Register for landscape notifications
	if (UWorld* World = GetWorld())
	{
		if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
		{
			check(!OnHeightmapStreamedHandle.IsValid());
			OnHeightmapStreamedHandle = LandscapeSubsystem->OnHeightmapStreamed().AddUObject(this, &AWaterZone::OnLandscapeHeightmapStreamed);

			check(!OnLandscapeComponentDataChangedHandle.IsValid());
			OnLandscapeComponentDataChangedHandle = LandscapeSubsystem->OnLandscapeProxyComponentDataChanged().AddUObject(this, &AWaterZone::OnLandscapeComponentDataChanged);
		}
	}
#endif // WITH_EDITOR
}

void AWaterZone::PostUnregisterAllComponents()
{
	Super::PostUnregisterAllComponents();

#if WITH_EDITOR
	if (UWorld* World = GetWorld())
	{
		if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
		{
			if (OnHeightmapStreamedHandle.IsValid())
			{
				LandscapeSubsystem->OnHeightmapStreamed().Remove(OnHeightmapStreamedHandle);
			}
			if (OnLandscapeComponentDataChangedHandle.IsValid())
			{
				LandscapeSubsystem->OnLandscapeProxyComponentDataChanged().Remove(OnLandscapeComponentDataChangedHandle);
			}
		}
	}
	OnHeightmapStreamedHandle.Reset();
	OnLandscapeComponentDataChangedHandle.Reset();
#endif // WITH_EDITOR

	// We must check for the index because UnregisterAllComponents can be called multiple times in a row by PostEditChangeProperty, etc.
	FWaterBodyManager* Manager = UWaterSubsystem::GetWaterBodyManager(GetWorld());
	if (Manager && !IsTemplate() && (WaterZoneIndex != INDEX_NONE))
	{
		Manager->RemoveWaterZone(this);
	}
	WaterZoneIndex = INDEX_NONE;
}

#if WITH_EDITORONLY_DATA
void AWaterZone::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UBoxComponent::StaticClass()));
}
#endif

void AWaterZone::MarkForRebuild(EWaterZoneRebuildFlags Flags, const FBox2D& UpdateRegion, const UObject* DebugRequestingObject)
{
	// TODO: investigate why optimizing by updating only checking the bounds of the active views dynamic extents is not working.
	const FBox WaterInfoBounds = GetZoneBounds();
	const FBox2D WaterInfoBounds2D(FVector2D(WaterInfoBounds.Min), FVector2D(WaterInfoBounds.Max));

	// Suppress updates which occur outside the bounds of the water zone.
	if ((!UpdateRegion.bIsValid || UpdateRegion.Intersect(WaterInfoBounds2D)))
	{
		if (EnumHasAnyFlags(Flags, EWaterZoneRebuildFlags::UpdateWaterMesh))
		{
			UE_LOGF(LogWater, Verbose, "AWaterZone (%ls) UpdateWaterMesh in region {%ls} (triggered by %ls)", *GetNameSafe(this), *UpdateRegion.ToString(), *GetNameSafe(DebugRequestingObject));
			WaterMesh->MarkWaterMeshGridDirty();
		}
		if (EnumHasAnyFlags(Flags, EWaterZoneRebuildFlags::UpdateWaterInfoTexture))
		{
			UE_LOGF(LogWater, Verbose, "AWaterZone (%ls) UpdateWaterInfoTexture in region {%ls} (triggered by %ls)", *GetNameSafe(this), *UpdateRegion.ToString(), *GetNameSafe(DebugRequestingObject));
			bNeedsWaterInfoRebuild = true;
		}
	}
}

void AWaterZone::MarkForRebuild(EWaterZoneRebuildFlags Flags, const UObject* DebugRequestingObject)
{
	MarkForRebuild(Flags, FBox2D(ForceInitToZero), DebugRequestingObject);
}

void AWaterZone::ForEachWaterBodyComponent(TFunctionRef<bool(UWaterBodyComponent*)> Predicate) const
{
	for (const TWeakObjectPtr<UWaterBodyComponent>& WaterBodyComponent : OwnedWaterBodies)
	{
		if (WaterBodyComponent.IsValid() && !Predicate(WaterBodyComponent.Get()))
		{
			break;
		}
	}
}

void AWaterZone::AddWaterBodyComponent(UWaterBodyComponent* WaterBodyComponent)
{
	if (OwnedWaterBodies.Find(WaterBodyComponent) == INDEX_NONE)
	{
		OwnedWaterBodies.Add(WaterBodyComponent);

		EWaterZoneRebuildFlags RebuildFlags = EWaterZoneRebuildFlags::None;
		if (WaterBodyComponent->AffectsWaterInfo())
		{
			RebuildFlags |= EWaterZoneRebuildFlags::UpdateWaterInfoTexture;
		}
		if (WaterBodyComponent->AffectsWaterMesh())
		{
			RebuildFlags |= EWaterZoneRebuildFlags::UpdateWaterMesh;
		}

		const FBox WaterBodyBounds = WaterBodyComponent->Bounds.GetBox();
		MarkForRebuild(RebuildFlags, FBox2D(FVector2D(WaterBodyBounds.Min), FVector2D(WaterBodyBounds.Max)), /* DebugRequestingObject = */ WaterBodyComponent->GetOwner());
	}
}

void AWaterZone::RemoveWaterBodyComponent(UWaterBodyComponent* WaterBodyComponent)
{
	int32 Index;
	if (OwnedWaterBodies.Find(WaterBodyComponent, Index))
	{
		OwnedWaterBodies.RemoveAtSwap(Index);

		EWaterZoneRebuildFlags RebuildFlags = EWaterZoneRebuildFlags::None;
		if (WaterBodyComponent->AffectsWaterInfo())
		{
			RebuildFlags |= EWaterZoneRebuildFlags::UpdateWaterInfoTexture;
		}
		if (WaterBodyComponent->AffectsWaterMesh())
		{
			RebuildFlags |= EWaterZoneRebuildFlags::UpdateWaterMesh;
		}
		
		const FBox WaterBodyBounds = WaterBodyComponent->Bounds.GetBox();
		MarkForRebuild(RebuildFlags, FBox2D(FVector2D(WaterBodyBounds.Min), FVector2D(WaterBodyBounds.Max)), /* DebugRequestingObject = */ WaterBodyComponent->GetOwner());
	}
}

FVector2D AWaterZone::GetZoneExtent() const
{
	return ZoneExtent * FVector2D(GetActorScale());
}

void AWaterZone::Update()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AWaterZone::Update);
	if (bNeedsWaterInfoRebuild || (ForceUpdateWaterInfoNextFrames != 0))
	{
		ForceUpdateWaterInfoNextFrames = (ForceUpdateWaterInfoNextFrames < 0) ? ForceUpdateWaterInfoNextFrames : FMath::Max(0, ForceUpdateWaterInfoNextFrames - 1);
		if (UpdateWaterInfoTexture())
		{
			bNeedsWaterInfoRebuild = false;
		}
	}
	
	if (WaterMesh)
	{
		WaterMesh->Update();
	}
}

#if	WITH_EDITOR
void AWaterZone::ForceUpdateWaterInfoTexture()
{
	UpdateWaterInfoTexture();
}

void AWaterZone::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);

	// Ensure that the water mesh is rebuilt if it moves
	EWaterZoneRebuildFlags RebuildFlags = EWaterZoneRebuildFlags::UpdateWaterInfoTexture | EWaterZoneRebuildFlags::UpdateWaterMesh;

	UpdateOverlappingWaterBodies();

	MarkForRebuild(RebuildFlags, /* DebugRequestingObject = */ this);
}

void AWaterZone::PostEditUndo()
{
	Super::PostEditUndo();

	if (IsValid(this))
	{
		UpdateOverlappingWaterBodies();
		MarkForRebuild(EWaterZoneRebuildFlags::All, /* DebugRequestingObject = */ this);
	}
}

void AWaterZone::PostEditImport()
{
	Super::PostEditImport();

	UpdateOverlappingWaterBodies();
	MarkForRebuild(EWaterZoneRebuildFlags::All, /* DebugRequestingObject = */ this);
}

void AWaterZone::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(AWaterZone, ZoneExtent))
	{
		OnExtentChanged();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWaterZone, BoundsComponent))
	{
		OnBoundsComponentModified();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWaterZone, RenderTargetResolution))
	{
		// Clamp the render resolution if the CVar is set
		const int32 RenderTargetResolutionMax = CVarWaterInfoRenderTargetResolutionMax.GetValueOnGameThread();
		if (RenderTargetResolutionMax > 0)
		{
			RenderTargetResolution.X = FMath::Clamp(RenderTargetResolution.X, 0, RenderTargetResolutionMax);
			RenderTargetResolution.Y = FMath::Clamp(RenderTargetResolution.Y, 0, RenderTargetResolutionMax);
		}
		MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture, /* DebugRequestingObject = */ this);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWaterZone, bHalfPrecisionTexture))
	{
		MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture, /* DebugRequestingObject = */ this);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWaterZone, VelocityBlurRadius))
	{
		MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture, /* DebugRequestingObject = */ this);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWaterZone, LocalTessellationExtent))
	{
		MarkForRebuild(EWaterZoneRebuildFlags::All, /* DebugRequestingObject = */ this);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWaterZone, bEnableLocalOnlyTessellation))
	{
		MarkForRebuild(EWaterZoneRebuildFlags::All, /* DebugRequestingObject = */ this);
	}
	else if (PropertyName == USceneComponent::GetRelativeScale3DPropertyName())
	{
		MarkForRebuild(EWaterZoneRebuildFlags::All, /* DebugRequestingObject = */ this);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWaterZone, bAutoIncludeLandscapesAsTerrain))
	{
		MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture, /* DebugRequestingObject = */ this);
	}
}

void AWaterZone::OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	TArray<AWaterBody*> NewWaterBodiesSelection;
	Algo::TransformIf(NewSelection, NewWaterBodiesSelection, [](UObject* Obj) { return Obj->IsA<AWaterBody>(); }, [](UObject* Obj) { return static_cast<AWaterBody*>(Obj); });
	NewWaterBodiesSelection.Sort();
	TArray<TWeakObjectPtr<AWaterBody>> NewWeakWaterBodiesSelection;
	NewWeakWaterBodiesSelection.Reserve(NewWaterBodiesSelection.Num());
	Algo::Transform(NewWaterBodiesSelection, NewWeakWaterBodiesSelection, [](AWaterBody* Body) { return TWeakObjectPtr<AWaterBody>(Body); });

	// Ensure that the water mesh is rebuilt if water body selection changed
	if (SelectedWaterBodies != NewWeakWaterBodiesSelection)
	{
		SelectedWaterBodies = NewWeakWaterBodiesSelection;
		MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterMesh | EWaterZoneRebuildFlags::UpdateWaterInfoTexture);
	}
}

TUniquePtr<class FWorldPartitionActorDesc> AWaterZone::CreateClassActorDesc() const
{
	return TUniquePtr<FWorldPartitionActorDesc>(new FWaterZoneActorDesc());
}

void AWaterZone::GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const
{
	OutRuntimeBounds = OutEditorBounds = GetZoneBounds();
}

void AWaterZone::OnLandscapeHeightmapStreamed(const FOnHeightmapStreamedContext& InContext)
{
	if (bAutoIncludeLandscapesAsTerrain)
	{
		UE_LOGF(LogWater, Verbose, "AWaterZone::OnLandscapeHeightmapStreamed() -- Rebuilding Water Info Texture...");
		// Invalidate the region corresponding the heightmap being streamed in so that the water info is always up-to-date with the current landscape (i.e. ground) height :
		const FBox2D& UpdateRegion = InContext.GetUpdateRegion();
		MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture, FBox2D(FVector2D(UpdateRegion.Min), FVector2D(UpdateRegion.Max)), reinterpret_cast<const UObject*>(InContext.GetLandscape()));
	}
}

void AWaterZone::OnLandscapeComponentDataChanged(ALandscapeProxy* InLandscape, const FLandscapeProxyComponentDataChangedParams& InChangeParams)
{
	// TODO [jonathan.bard] : if that is a problem, we could add more info to FLandscapeProxyComponentDataChangedParams and tell exactly what has changed (heightmap, weightmap, etc.)
	//  because so far, the water info only needs to be notified about heightmap changes
	if (bAutoIncludeLandscapesAsTerrain)
	{
		UE_LOGF(LogWater, Verbose, "AWaterZone::OnLandscapeComponentDataChanged() -- Rebuilding Water Info Texture...");
		// Invalidate the region corresponding to each of the changed components. Use a combined region, because 99% of times, the component are contiguous and laid out in a square :
		FBox CombinedBounds;
		InChangeParams.ForEachComponent([&CombinedBounds](const ULandscapeComponent* InLandscapeComponent)
			{
				FBox LandscapeComponentBounds = InLandscapeComponent->GetLocalBounds().TransformBy(InLandscapeComponent->GetComponentTransform()).GetBox();
				CombinedBounds += LandscapeComponentBounds;
			});
		MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture, FBox2D(FVector2D(CombinedBounds.Min), FVector2D(CombinedBounds.Max)), InLandscape);
	}
}

#endif // WITH_EDITOR

void AWaterZone::SetFarMeshMaterial(UMaterialInterface* InFarDistanceMaterial)
{
	if (WaterMesh && InFarDistanceMaterial != WaterMesh->FarDistanceMaterial)
	{
		WaterMesh->FarDistanceMaterial = InFarDistanceMaterial;
		MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterMesh, /* DebugRequestingObject = */ this);
	}
}


void AWaterZone::OnExtentChanged()
{
#if WITH_EDITOR
	BoundsComponent->SetBoxExtent(FVector(GetZoneExtent() / 2., 8192.f));
#endif // WITH_EDITOR

	UpdateOverlappingWaterBodies();

	MarkForRebuild(EWaterZoneRebuildFlags::All, /* DebugRequestingObject = */ this);
}

bool AWaterZone::UpdateOverlappingWaterBodies()
{
	TArray<TWeakObjectPtr<UWaterBodyComponent>> OldOwnedWaterBodies = OwnedWaterBodies;
	FWaterBodyManager::ForEachWaterBodyComponent(GetWorld(), [](UWaterBodyComponent* WaterBodyComponent)
		{
			WaterBodyComponent->UpdateWaterZones();
			return true;
		});
	return (OwnedWaterBodies != OldOwnedWaterBodies);
}


#if WITH_EDITOR
void AWaterZone::OnBoundsComponentModified()
{
	// BoxExtent is the radius of the box (half-extent).
	const FVector2D NewBounds = 2.0 * FVector2D(BoundsComponent->GetUnscaledBoxExtent());

	SetZoneExtent(NewBounds);
}
#endif // WITH_EDITOR

bool AWaterZone::UpdateWaterInfoTexture()
{
	if (UWorld* World = GetWorld(); World && FApp::CanEverRender())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AWaterZone::UpdateWaterInfoTexture);

		float WaterZMin(TNumericLimits<float>::Max());
		float WaterZMax(TNumericLimits<float>::Lowest());
	
		// Collect a list of all materials used in the water info render to ensure they have complete shaders maps.
		// If they do not, we must submit compile jobs for them and wait until they are finished before re-rendering.
		TArray<UMaterialInterface*> UsedMaterials;

		TArray<TWeakObjectPtr<UWaterBodyComponent>> WaterBodiesToRender;
		ForEachWaterBodyComponent([World, &WaterBodiesToRender, &WaterZMax, &WaterZMin, &UsedMaterials](UWaterBodyComponent* WaterBodyComponent)
		{
			// skip components which don't affect the water info texture
			if (!WaterBodyComponent->AffectsWaterInfo())
			{
				return true;
			}

			WaterBodiesToRender.Add(WaterBodyComponent);
			const FBox WaterBodyBounds = WaterBodyComponent->Bounds.GetBox();
			WaterZMax = FMath::Max(WaterZMax, WaterBodyBounds.Max.Z);
			WaterZMin = FMath::Min(WaterZMin, WaterBodyBounds.Min.Z);

#if WITH_EDITOR
			if (UMaterialInterface* WaterInfoMaterial = WaterBodyComponent->GetWaterInfoMaterialInstance())
			{
				UsedMaterials.Add(WaterInfoMaterial);
			}
#endif // WITH_EDITOR

			return true;
		});

		// If we don't have any water bodies we don't need to do anything.
		if (WaterBodiesToRender.Num() == 0)
		{
			return true;
		}

		// Ensure that all the PSOs for the water info materials have been pre-cached before attempting to render the water info
		// This is necessary if we have enabled the option to delay proxy creation until precache PSOs are ready, in which case
		// we'd try to render the mesh components and end up skipping the creation of the proxy (or creating a temporary proxy
		// with the fallback material fallback).
		if (IsComponentPSOPrecachingEnabled() && GetPSOPrecacheProxyCreationStrategy() != EPSOPrecacheProxyCreationStrategy::AlwaysCreate)
		{
			bool bHaveAllPSOsBeenCached = true;

			for (const TWeakObjectPtr<UWaterBodyComponent>& WaterBodyComponentPtr : WaterBodiesToRender)
			{
				if (UWaterBodyComponent* WaterBodyComponent = WaterBodyComponentPtr.Get())
				{
					// CheckPSOPrecachingAndBoostPriority returns true if PSOs are still precaching.
					if (UWaterBodyInfoMeshComponent* WaterInfoMeshComponent = WaterBodyComponent->GetWaterInfoMeshComponent())
					{
						bHaveAllPSOsBeenCached &= !WaterInfoMeshComponent->CheckPSOPrecachingAndBoostPriority();
					}
					if (UWaterBodyInfoMeshComponent* WaterInfoMeshComponent = WaterBodyComponent->GetDilatedWaterInfoMeshComponent())
					{
						bHaveAllPSOsBeenCached &= !WaterInfoMeshComponent->CheckPSOPrecachingAndBoostPriority();
					}
				}
			}

			// If the PSOs weren't fully pre-cached, we will try to render again on the next frame.
			if (!bHaveAllPSOsBeenCached)
			{
				return false;
			}
		}

		WaterHeightExtents = FVector2f(WaterZMin, WaterZMax);

		// Only compute the ground min since we can use the water max z as the ground max z for more precision.
		GroundZMin = TNumericLimits<float>::Max();
		float GroundZMax = TNumericLimits<float>::Lowest();

		TArray<TWeakObjectPtr<UPrimitiveComponent>> GroundPrimitiveComponents;

		const FBox WaterZoneBounds = GetZoneBounds();
		if (bAutoIncludeLandscapesAsTerrain)
		{
			for (ALandscapeProxy* LandscapeProxy : TActorRange<ALandscapeProxy>(World))
			{
				FBox LandscapeBox;
				// Only collect the bounds for the ULandscapeComponents to avoid growing the bounding box unnecessarily
				// if the landscape proxy has other components attached to it such as Grass or any other user-added components.
				LandscapeProxy->ForEachComponent<ULandscapeComponent>(true, [&](const ULandscapeComponent* InLandscapeComp)
					{
						// Don't include bounds for non-registered components as they will not have the correct worldspace transform
						if (InLandscapeComp->IsRegistered())
						{
							LandscapeBox += InLandscapeComp->Bounds.GetBox();
						}
					});

				// Only consider landscapes which this zone intersects with in XY and if the landscape volume is not zero sized
				if (LandscapeBox.IsValid && (LandscapeBox.GetVolume() > 0.0) && WaterZoneBounds.IntersectXY(LandscapeBox))
				{
					GroundZMin = FMath::Min(GroundZMin, LandscapeBox.Min.Z);
					GroundZMax = FMath::Max(GroundZMax, LandscapeBox.Max.Z);
					TInlineComponentArray<ULandscapeComponent*> LandscapeComponents(LandscapeProxy);
					GroundPrimitiveComponents.Append(LandscapeComponents);
				}
			}
		}

		UWaterSubsystem* WaterSubsystem = UWaterSubsystem::GetWaterSubsystem(World);
		check(WaterSubsystem);

		TArray<UWaterTerrainComponent*> WaterTerrainComponents;
		WaterSubsystem->GetWaterTerrainComponents(WaterTerrainComponents);
		
		for (UWaterTerrainComponent* TerrainComponent : WaterTerrainComponents)
		{
			if (TerrainComponent->AffectsWaterZone(this))
			{
				TArray<UPrimitiveComponent*> TerrainPrimitives = TerrainComponent->GetTerrainPrimitives();

				const FBox GroundActorBounds = TerrainComponent->GetOwner()->GetComponentsBoundingBox(true);
				GroundZMin = FMath::Min(GroundZMin, GroundActorBounds.Min.Z);
				GroundZMax = FMath::Max(GroundZMax, GroundActorBounds.Max.Z);
				GroundPrimitiveComponents.Append(TerrainPrimitives);
			}
		}

		// If we have no ground components we need to set GroundZMin to be something sensible because it won't have been set above.
		if (GroundPrimitiveComponents.Num() == 0)
		{
			GroundZMax = WaterZMax;
			GroundZMin = WaterZMin - CVarWaterFallbackDepth.GetValueOnGameThread();
		}

#if WITH_EDITOR
		// Check all the ground components have complete shader maps and are all compiled before we try to render them into the water info texture
		for (TWeakObjectPtr<UPrimitiveComponent> GroundPrimCompPtr : GroundPrimitiveComponents)
		{
			if (UPrimitiveComponent* GroundPrimComp = GroundPrimCompPtr.Get())
			{
				if (GroundPrimComp->IsCompiling())
				{
					return false;
				}
				TArray<UMaterialInterface*> TmpUsedMaterials;
				GroundPrimComp->GetUsedMaterials(TmpUsedMaterials, false);
				UsedMaterials.Append(TmpUsedMaterials);
			}
		}

		// Loop through all used materials and ensure that compile jobs are submitted for all which do not have complete shader maps before early-ing out of the info update.
		bool bHasIncompleteShaderMaps = false;
		const EShaderPlatform ShaderPlatform = World->Scene->GetShaderPlatform();
		for (UMaterialInterface* Material : UsedMaterials)
		{
			if (Material)
			{
				if (FMaterialResource* MaterialResource = Material->GetMaterialResource(ShaderPlatform))
				{
					if (!MaterialResource->IsGameThreadShaderMapComplete())
					{
						MaterialResource->SubmitCompileJobs_GameThread(EShaderCompileJobPriority::High);

						bHasIncompleteShaderMaps = true;
					}
				}
			}
		}

		if (bHasIncompleteShaderMaps)
		{
			return false;
		}
#endif // WITH_EDITOR

		// The render path for rendering the water info texture without scene captures is executed within the scene renderer.
		// If world rendering is disabled like in a loading screen, the scene renderer is not called and the water info texture will not be drawn.
		UGameViewportClient* GameViewport = World->GetGameViewport();
		// HACK [jonathan.bard] : CVarSkipWaterInfoTextureRenderWhenWorldRenderingDisabled is a temporary hack for MRQ because bDisableWorldRendering is true when capturing for MRQ. 
		//  The proper solution is to refactor the water info texture and make it per-view (also for split screen support) : FWaterViewExtension::SetupViewFamily/SetupView
		//  are called on the game thread so they might be a good place for doing this. 
		//  For now, we just use this CVar trick to let the water info texture be rendered when MRQ runs :
		if ((CVarSkipWaterInfoTextureRenderWhenWorldRenderingDisabled.GetValueOnGameThread() != 0) && GameViewport && GameViewport->bDisableWorldRendering)
		{
			return false;
		}

		const ETextureRenderTargetFormat Format = bHalfPrecisionTexture ? ETextureRenderTargetFormat::RTF_RGBA16f : RTF_RGBA32f;
		UTextureRenderTarget2DArray* OldTexture = WaterInfoTextureArray;
		WaterInfoTextureArray = FWaterUtils::GetOrCreateTransientRenderTarget2DArray(OldTexture, TEXT("WaterInfoTexture"), GetRenderTargetResolution(), WaterInfoTextureArrayNumSlices, Format);

		// The water info texture is different, we need to bind the newly created texture to all registered water bodies
		if (WaterInfoTextureArray != OldTexture)
		{
			OnWaterInfoTextureArrayCreated.Broadcast(WaterInfoTextureArray);

			ForEachWaterBodyComponent([](UWaterBodyComponent* WaterBodyComponent)
			{
				WaterBodyComponent->UpdateMaterialInstances();
				return true;
			});
		}

		UE::WaterInfo::FRenderingContext Context;
		Context.ZoneToRender = this;
		Context.WaterBodies = WaterBodiesToRender;
		Context.GroundPrimitiveComponents = MoveTemp(GroundPrimitiveComponents);
		Context.CaptureZ = FMath::Max(WaterZMax, GroundZMax) + CaptureZOffset;
		Context.TextureRenderTarget = WaterInfoTextureArray;

		if (FWaterViewExtension* WaterViewExtension = UWaterSubsystem::GetWaterViewExtension(World))
		{
			WaterViewExtension->MarkWaterInfoTextureForRebuild(Context);
		}

		UE_LOGF(LogWater, Verbose, "Water Zone (%ls) queued Water Info texture update", *GetNameSafe(this));
	}

	return true;
}

FVector AWaterZone::GetDynamicWaterInfoExtent() const
{
	if (IsLocalOnlyTessellationEnabled())
	{
		return LocalTessellationExtent * GetActorScale();
	}
	else
	{
		// #todo_water [roey]: better implementation for 3D extent
		return FVector(GetZoneExtent(), 0.0);
	}
}

bool AWaterZone::GetDynamicWaterInfoCenter(int32 PlayerIndex, FVector& OutCenter) const
{
	const UWorld* World = GetWorld();

	check(World != nullptr);

	bool bHasValidZoneLocation = false;
	OutCenter = GetActorLocation();

	if (const FWaterViewExtension* WaterViewExtension = UWaterSubsystem::GetWaterViewExtension(World))
	{
		bHasValidZoneLocation = WaterViewExtension->GetZoneLocation(this, PlayerIndex, OutCenter);
	}
	
	return bHasValidZoneLocation;
}

bool AWaterZone::GetDynamicWaterInfoBounds(int32 PlayerIndex, FBox& OutBounds) const
{
	const UWorld* World = GetWorld();

	check(World != nullptr);

	bool bHasValidZoneLocation = false;
	FVector Center = GetActorLocation();

	if (const FWaterViewExtension* WaterViewExtension = UWaterSubsystem::GetWaterViewExtension(World))
	{
		bHasValidZoneLocation = WaterViewExtension->GetZoneLocation(this, PlayerIndex, Center);
	}

	const FVector WaterInfoHalfExtents = GetDynamicWaterInfoExtent() / 2.;
	OutBounds = FBox(Center - WaterInfoHalfExtents, Center + WaterInfoHalfExtents);

	return bHasValidZoneLocation;
}


void AWaterZone::OnLevelAddedToWorld(ULevel* InLevel, UWorld* InWorld)
{
	OnLevelChanged(InLevel, InWorld);
}

void AWaterZone::OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld)
{
	OnLevelChanged(InLevel, InWorld);
}

bool AWaterZone::ContainsActorsAffectingWaterZone(ULevel* InLevel, const FBox& WaterZoneBounds) const
{
	const bool bContainsActorsAffectingWaterZone = Algo::AnyOf(InLevel->Actors, [this, &WaterZoneBounds](const AActor* Actor)
		{
			return IsAffectingWaterZone(WaterZoneBounds, Actor);
		});

	return bContainsActorsAffectingWaterZone;
}

void AWaterZone::OnLevelChanged(ULevel* InLevel, UWorld* InWorld)
{
	if ((InLevel == nullptr) || (InWorld != GetWorld()))
	{
		return;
	}

	TArray<FBox> WaterZoneViewsBounds;
	GetAllDynamicWaterInfoBounds(WaterZoneViewsBounds);

	bool bContainsActorsAffectingWaterZone = false;

	if (WaterZoneViewsBounds.Num() > 0)
	{
		for (const FBox& ZoneViewBounds : WaterZoneViewsBounds)
		{
			if (ContainsActorsAffectingWaterZone(InLevel, ZoneViewBounds))
			{
				bContainsActorsAffectingWaterZone = true;
				break;
			}
		}
	}
	else
	{
		bContainsActorsAffectingWaterZone = ContainsActorsAffectingWaterZone(InLevel, GetZoneBounds());
	}

	if (bContainsActorsAffectingWaterZone)
	{
		MarkForRebuild(EWaterZoneRebuildFlags::UpdateWaterInfoTexture, /* DebugRequestingObject = */ InLevel);
	}
}

bool AWaterZone::IsAffectingWaterZone(const FBox& InWaterZoneBounds, const AActor* InActor) const
{
	if (InActor == nullptr)
	{
		return false;
	}

	if (const AWaterBody* WaterBodyActor = Cast<const AWaterBody>(InActor))
	{
		if (const UWaterBodyComponent* WaterBodyComponent = WaterBodyActor->GetWaterBodyComponent())
		{
			if (WaterBodyComponent->GetWaterZone() == this && WaterBodyComponent->AffectsWaterInfo())
			{
				return InWaterZoneBounds.IntersectXY(WaterBodyComponent->Bounds.GetBox());
			}
		}
	}
	else if (const ALandscapeProxy* LandscapeProxy = Cast<const ALandscapeProxy>(InActor))
	{
		bool bIntersectsWaterZone = false;

		LandscapeProxy->ForEachComponent<ULandscapeComponent>(false, [&bIntersectsWaterZone, &InWaterZoneBounds](const ULandscapeComponent* LandscapeComponent)
		{
			if (!bIntersectsWaterZone && (LandscapeComponent != nullptr))
			{
				const FBox LandscapeBox = LandscapeComponent->Bounds.GetBox();

				bIntersectsWaterZone = InWaterZoneBounds.IntersectXY(LandscapeBox) && (LandscapeBox.GetVolume() > 0.0);
			}
		});

		return bIntersectsWaterZone;
	}

	return false;
}
