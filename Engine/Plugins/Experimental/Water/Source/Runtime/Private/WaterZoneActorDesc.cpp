// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterZoneActorDesc.h"

#if WITH_EDITOR
#include "WaterZoneActor.h"

FWaterZoneActorDesc::FWaterZoneActorDesc()
	: OverlapPriority(0)
{
}

void FWaterZoneActorDesc::Init(const AActor* InActor)
{
	FWorldPartitionActorDesc::Init(InActor);

	if (const AWaterZone* WaterZone = CastChecked<AWaterZone>(InActor))
	{
		OverlapPriority = WaterZone->GetOverlapPriority();
	}
}

void FWaterZoneActorDesc::Serialize(FArchive& Ar)
{
	FWorldPartitionActorDesc::Serialize(Ar);

	Ar << OverlapPriority;
}

bool FWaterZoneActorDesc::Equals(const FWorldPartitionActorDesc* Other) const
{
	if (FWorldPartitionActorDesc::Equals(Other))
	{
		const FWaterZoneActorDesc* WaterZoneActorDesc = (FWaterZoneActorDesc*)Other;
		return OverlapPriority == WaterZoneActorDesc->OverlapPriority;
	}

	return false;
}

#endif
