// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "WorldPartition/WorldPartitionActorDesc.h"

/**
 * ActorDesc for WaterZone actors.
 */
class FWaterZoneActorDesc : public FWorldPartitionActorDesc
{
public:
	FWaterZoneActorDesc();
	virtual void Init(const AActor* InActor) override;
	virtual bool Equals(const FWorldPartitionActorDesc* Other) const override;

	int32 GetOverlapPriority() const { return OverlapPriority; }
protected:
	virtual uint32 GetSizeOf() const override { return sizeof(FWaterZoneActorDesc); }
	virtual void Serialize(FArchive& Ar) override;

	int32 OverlapPriority;
};
#endif
