// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Float16.h"
#include "Components/PrimitiveComponent.h"
#include "BakedShallowWaterSimulationComponent.generated.h"

#define UE_API WATER_API

class AWaterBody;

UCLASS(MinimalAPI)
class UShallowWaterSimulationDataBase : public UObject 
{
	GENERATED_BODY()
public:
	
	/** Number of cells */
	UPROPERTY()
	FIntVector2 NumCells;
	
	/** World space center */
	UPROPERTY()
	FVector Position;
	
	/** World space size */
	UPROPERTY()
	FVector2D Size;
	
	/** Texture for baked water sim data */
	UPROPERTY()
	TObjectPtr<UTexture2D> BakedTexture;

	/** Check for a valid sim domain */
	bool HasValidData() const
	{
		return NumCells.X > 0 && NumCells.Y > 0 && GetTotalNumCells() == NumCells.X * NumCells.Y && Size.X > SMALL_NUMBER && Size.Y > SMALL_NUMBER;
	}

	/** Linear interpolate sim data at an index */
	WATER_API void SampleShallowWaterSimulationAtIndex(const FVector2D &QueryFloatIndex, FVector& OutWaterVelocity, float& OutWaterHeight, float& OutWaterDepth) const;
	
	/** Query sim data at an index */
	WATER_API void QueryShallowWaterSimulationAtIndex(const FIntVector2 &QueryIndex, FVector& OutWaterVelocity, float& OutWaterHeight, float& OutWaterDepth) const;

	/** Compute normal at a world position */
	WATER_API FVector ComputeShallowWaterSimulationNormalAtPosition(const FVector &QueryPos) const;
	
	virtual int GetTotalNumCells() const { return 0; }
	virtual int GetNumAllocatedCells() const { return 0; }
	virtual int GetMemoryInBytes() const { return 0; }

	virtual void Build(const TArray<FFloat16Color>& InputDenseGrid, FIntVector2 InDenseNumCells) {};

	virtual FVector4 QueryLowLevelGridAtIndex(const FIntVector2 Index) const { return FVector4(0,0,0,0); }
	
	/** Query shallow water grid cell values at a world position */
	void QueryShallowWaterSimulationAtPosition(const FVector &QueryPos, FVector& OutWaterVelocity, float& OutWaterHeight, float& OutWaterDepth) const
	{
		const FIntVector2 IndexPos = WorldToIndex(QueryPos);
		QueryShallowWaterSimulationAtIndex(IndexPos, OutWaterVelocity, OutWaterHeight, OutWaterDepth);
	}

	/** Linear interpolate for shallow water grid cell values at a world position */
	void SampleShallowWaterSimulationAtPosition(const FVector &QueryPos, FVector& OutWaterVelocity, float& OutWaterHeight, float& OutWaterDepth) const
	{
		FVector2D FloatIndex = WorldToFloatIndex(QueryPos);
		SampleShallowWaterSimulationAtIndex(FloatIndex, OutWaterVelocity, OutWaterHeight, OutWaterDepth);
	}

	/** Convert world space to integer index */
	FIntVector2 WorldToIndex(const FVector &WorldPos) const
	{
		FVector2D FloatIndex = WorldToFloatIndex(WorldPos);
		return FIntVector2(FloatIndex.X, FloatIndex.Y);
	}

	/** Convert world space to float index */
	FVector2D WorldToFloatIndex(const FVector &WorldPos) const
	{
		if (Size.X < 1e-5 || Size.Y < 1e-5)
		{
			return FVector2D(0.f,0.f);
		}


		const FVector LocalPos = WorldPos - Position;
		
		// unit positions are 0 - 1, Local Pos has center of grid at origin
		const FVector2D UnitPos = FVector2D(LocalPos.X, LocalPos.Y) / Size + .5;

		// cell values are stored at grid cell center
		return UnitPos * FVector2D(NumCells.X, NumCells.Y) - .5;
	}

	/** Convert a float index to world space */
	FVector FloatIndexToWorld(const FVector2D &Index) const
	{
		if (NumCells.X == 0 || NumCells.Y == 0)
		{
			return FVector(0.f, 0.f, 0.f);
		}

		// Values are stored at cell center
		const FVector2D UnitPos = (Index + .5) / FVector2D(NumCells.X, NumCells.Y);
		
		// local position origin is center of the grid
		const FVector2D LocalPos = (UnitPos - .5) * Size;

		const FVector LocalPos3D = FVector(LocalPos.X, LocalPos.Y, 0);
		return LocalPos3D + Position;
	}

	/** Convert an integer index to world space */
	FVector IndexToWorld(const FIntVector2 &Index) const
	{
		return FloatIndexToWorld(FVector2D(Index.X, Index.Y));
	}
};

UCLASS(MinimalAPI)
class UShallowWaterSimulationData : public UShallowWaterSimulationDataBase
{	
	GENERATED_BODY()

public:
	/** Raw grid data */
	UPROPERTY()
	TArray<FVector4> ArrayValues;

	virtual int GetTotalNumCells() const
	{
		return ArrayValues.Num();
	}

	virtual int GetNumAllocatedCells() const
	{
		return GetTotalNumCells();
	}

	virtual int GetMemoryInBytes() const
	{
		return ArrayValues.NumBytes();
	}
	
	virtual void Build(const TArray<FFloat16Color>& InputDenseGrid, FIntVector2 InDenseNumCells)
	{
		check(InDenseNumCells.X * InDenseNumCells.Y == InputDenseGrid.Num());

		NumCells = InDenseNumCells;

		ArrayValues.SetNum(InputDenseGrid.Num());

		int idx = 0;
		for (FFloat16Color CurrCell : InputDenseGrid)
		{
			ArrayValues[idx++] = FVector4(CurrCell.R.GetFloat(), CurrCell.G.GetFloat(),CurrCell.B.GetFloat(),CurrCell.A.GetFloat());
		}
		// should loop over cells
	}

	virtual FVector4 QueryLowLevelGridAtIndex(const FIntVector2 Index) const
	{
		return ArrayValues[Index.X + Index.Y * NumCells.X];
	}
};

UCLASS(MinimalAPI)
class UShallowWaterSimulationData16 : public UShallowWaterSimulationDataBase
{	
	GENERATED_BODY()

public:
		
	/** Raw grid data */
	UPROPERTY()
	TArray<uint16> ArrayValues;

	virtual int GetTotalNumCells() const
	{
		return ArrayValues.Num() / 4;
	}

	virtual int GetNumAllocatedCells() const
	{
		return GetTotalNumCells();
	}

	virtual int GetMemoryInBytes() const
	{
		return ArrayValues.GetAllocatedSize();
	}

	virtual void Build(const TArray<FFloat16Color>& InputDenseGrid, FIntVector2 InDenseNumCells)
	{
		check(InDenseNumCells.X * InDenseNumCells.Y == InputDenseGrid.Num());

		NumCells = InDenseNumCells;

		ArrayValues.SetNum(InputDenseGrid.Num() * 4);
		FMemory::Memcpy(ArrayValues.GetData(), InputDenseGrid.GetData(), InputDenseGrid.Num() * sizeof(FFloat16Color));
	}

	virtual FVector4 QueryLowLevelGridAtIndex(const FIntVector2 Index) const
	{
		const FFloat16Color* CurrVal = reinterpret_cast<const FFloat16Color*>(&ArrayValues[(Index.X + Index.Y * NumCells.X) * 4]);
		return FVector4(CurrVal->R, CurrVal->G, CurrVal->B, CurrVal->A);
	}
};

UCLASS(MinimalAPI)
class UShallowWaterSimulationDataSparse : public UShallowWaterSimulationDataBase
{	
	GENERATED_BODY()

public:

	/** Block size for sparse storage (16x16 cells per block) */
	static constexpr int32 BlockSize = 16;

	/** Raw grid data */
	UPROPERTY()
	TArray<uint16> BlockValues;

	UPROPERTY()
	TMap<FIntVector2, int32> GridIndexToBlockIndex;

	virtual int GetTotalNumCells() const
	{
		return NumCells.X * NumCells.Y;
	}

	virtual int GetNumAllocatedCells() const
	{
		return BlockValues.Num() / 4;
	}

	virtual int GetMemoryInBytes() const
	{
		return BlockValues.GetAllocatedSize() + GridIndexToBlockIndex.GetAllocatedSize();
	}

	virtual FVector4 QueryLowLevelGridAtIndex(const FIntVector2 Index) const
	{
		const FIntVector2 BlockIndex(Index.X / BlockSize, Index.Y / BlockSize);

		const int32 *CurrBlockDataIndex = GridIndexToBlockIndex.Find(BlockIndex);

		if (!CurrBlockDataIndex)
		{
			return FVector4(0,0,0,0);
		}

		const int32 BlockStartIndex = (*CurrBlockDataIndex);
		
		const FIntVector2 LocalBlockIndex(Index.X % BlockSize, Index.Y % BlockSize);
		const int32 LocalBlockIndexLinear = (LocalBlockIndex.X + LocalBlockIndex.Y * BlockSize) * 4;

		const FFloat16Color* CurrVal = reinterpret_cast<const FFloat16Color*>(&BlockValues[BlockStartIndex + LocalBlockIndexLinear]);
		return FVector4(CurrVal->R, CurrVal->G, CurrVal->B, CurrVal->A);
	}

	virtual void Build(const TArray<FFloat16Color>& InputDenseGrid, FIntVector2 InDenseNumCells)
	{
		check(InDenseNumCells.X * InDenseNumCells.Y == InputDenseGrid.Num());

		NumCells = InDenseNumCells;

		BlockValues.Empty();
		GridIndexToBlockIndex.Empty();

		for (int32 y = 0; y < InDenseNumCells.Y; ++y) {
		for (int32 x = 0; x < InDenseNumCells.X; ++x) {
			FIntVector2 CurrCellIndex(x,y);
			
			const FFloat16Color &CurrVal = InputDenseGrid[x + y * InDenseNumCells.X];

			// check if depth is close to zero
			if (FMath::IsNearlyZero(CurrVal.G.GetFloat(), 1e-5))				
			{
				continue;
			}

			const FIntVector2 BlockIndex(CurrCellIndex.X / BlockSize, CurrCellIndex.Y / BlockSize);

			int32 &BlockStartIndex = GridIndexToBlockIndex.FindOrAdd(BlockIndex, -1);

			if (BlockStartIndex == -1)
			{
				BlockStartIndex = BlockValues.Num();
				BlockValues.AddZeroed(BlockSizeInBytes);				
			}
					
			const FIntVector2 LocalBlockIndex(CurrCellIndex.X % BlockSize, CurrCellIndex.Y % BlockSize);
			const int32 LocalBlockIndexLinear = (LocalBlockIndex.X + LocalBlockIndex.Y * BlockSize) * 4;

			FMemory::Memcpy(&BlockValues[BlockStartIndex + LocalBlockIndexLinear], &CurrVal, sizeof(FFloat16Color));
		}}
	}
private:
	static constexpr int32 DataSizeInBytes = 4;
	static constexpr int32 BlockSizeInBytes = BlockSize * BlockSize * 4;
};

USTRUCT(BlueprintType)
struct FShallowWaterSimulationGrid_DEPRECATED
{
	GENERATED_BODY()

public:


	/** Check for a valid sim domain */
	bool IsValid() const
	{
		return NumCells.X > 0 && NumCells.Y > 0 && ArrayValues.Num() == NumCells.X * NumCells.Y && Size.X > SMALL_NUMBER && Size.Y > SMALL_NUMBER;
	}

	// All methods are removed since this is used for data migration only.  
	// Any other usage should be updated to UShallowWaterSimulationData

	/** Raw grid data */
	UPROPERTY()
	TArray<FVector4> ArrayValues;
	
	/** Number of cells */
	UPROPERTY()
	FIntVector2 NumCells = FIntVector2::ZeroValue;
	
	/** World space center */
	UPROPERTY()
	FVector Position = FVector::ZeroVector;
	
	/** World space size */
	UPROPERTY()
	FVector2D Size = FVector2D::ZeroVector;

	/** World space grid cell size */
	UPROPERTY()
	FVector2D Dx = FVector2D::ZeroVector;
	
	/** Texture for baked water sim data */
	UPROPERTY()
	TObjectPtr<UTexture2D> BakedTexture;
};

UCLASS(meta = (BlueprintSpawnableComponent), MinimalAPI)
class UBakedShallowWaterSimulationComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

public:
 #if WITH_EDITORONLY_DATA
	UPROPERTY()
	FShallowWaterSimulationGrid_DEPRECATED SimulationData_DEPRECATED;
#endif

	UPROPERTY()
	TObjectPtr<UShallowWaterSimulationDataBase> BakedSimulationData;

	UPROPERTY()
	TSet<TSoftObjectPtr<AWaterBody>> WaterBodies;

	void PostLoad() override;
};

#undef UE_API
