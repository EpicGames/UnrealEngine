// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BuoyancyTypes.h"
#include "BuoyancyComponent.generated.h"

#define UE_API WATER_API

namespace EEndPlayReason { enum Type : int; }

class UWaterBodyComponent;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPontoonEnteredWater, const FSphericalPontoon&, Pontoon);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPontoonExitedWater, const FSphericalPontoon&, Pontoon);

UCLASS(MinimalAPI, Blueprintable, Config = Game, meta = (BlueprintSpawnableComponent))
class UBuoyancyComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	UE_API UBuoyancyComponent(const FObjectInitializer& ObjectInitializer);

	//~ Begin UActorComponent Interface.	
	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~ End UActorComponent Interface

	//~ Begin UObject Interface.	
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	UE_API virtual void Update(float DeltaTime);

	UE_API virtual void ApplyForces(float DeltaTime, FVector LinearVelocity, float ForwardSpeed, float ForwardSpeedKmh, UPrimitiveComponent* PrimitiveComponent);

	virtual void FinalizeAuxData() {}

	UE_API virtual TUniquePtr<FBuoyancyComponentAsyncInput> SetCurrentAsyncInputOutput(int32 InputIdx, FBuoyancyManagerAsyncOutput* CurOutput, FBuoyancyManagerAsyncOutput* NextOutput, float Alpha, int32 BuoyancyManagerTimestamp);
	UE_API void SetCurrentAsyncInputOutputInternal(FBuoyancyComponentAsyncInput* CurInput, int32 InputIdx, FBuoyancyManagerAsyncOutput* CurOutput, FBuoyancyManagerAsyncOutput* NextOutput, float Alpha, int32 BuoyancyManagerTimestamp);
	UE_API void FinalizeSimCallbackData(FBuoyancyManagerAsyncInput& Input);
	UE_API void GameThread_ProcessIntermediateAsyncOutput(const FBuoyancyManagerAsyncOutput& AsyncOutput);
	UE_API virtual void GameThread_ProcessIntermediateAsyncOutput(const FBuoyancyComponentAsyncOutput& Output);

	UE_API bool IsUsingAsyncPath() const;

	UE_API virtual TUniquePtr<FBuoyancyComponentAsyncAux> CreateAsyncAux() const;

	UE_API virtual void SetupWaterBodyOverlaps();

	UPrimitiveComponent* GetSimulatingComponent() { return SimulatingComponent; }
	bool HasPontoons() const { return BuoyancyData.Pontoons.Num() > 0; }
	UE_API void AddCustomPontoon(float Radius, FName CenterSocketName);
	UE_API void AddCustomPontoon(float Radius, const FVector& RelativeLocation);
	UE_API virtual int32 UpdatePontoons(float DeltaTime, float ForwardSpeed, float ForwardSpeedKmh, UPrimitiveComponent* PrimitiveComponent);
	UE_API void UpdatePontoonCoefficients();
	UE_API FVector ComputeWaterForce(const float DeltaTime, const FVector LinearVelocity) const;
	UE_API FVector ComputeLinearDragForce(const FVector& PhyiscsVelocity) const;
	UE_API FVector ComputeAngularDragTorque(const FVector& AngularVelocity) const;

	UE_API void EnteredWaterBody(UWaterBodyComponent* WaterBodyComponent);
	UE_API void ExitedWaterBody(UWaterBodyComponent* WaterBodyComponent);

	UFUNCTION(BlueprintCallable, Category = Buoyancy)
	const TArray<UWaterBodyComponent*>& GetCurrentWaterBodyComponents() const { return CurrentWaterBodyComponents; }
	TArray<TObjectPtr<UWaterBodyComponent>>& GetCurrentWaterBodyComponents() { return CurrentWaterBodyComponents; }

	UFUNCTION(BlueprintCallable, Category = Buoyancy)
	bool IsOverlappingWaterBody() const { return bIsOverlappingWaterBody; }

	virtual bool IsActive() const { return bCanBeActive && IsOverlappingWaterBody(); }

	void SetCanBeActive(bool bInCanBeActive) { bCanBeActive = bInCanBeActive; }

	UFUNCTION(BlueprintCallable, Category = Buoyancy)
	bool IsInWaterBody() const { return bIsInWaterBody; }

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use BuoyancyData.Pontoons instead."))
	TArray<FSphericalPontoon> Pontoons_DEPRECATED;

	UE_API void GetWaterSplineKey(FVector Location, TMap<const UWaterBodyComponent*, float>& OutMap, TMap<const UWaterBodyComponent*, float>& OutSegmentMap) const;
	UE_API float GetWaterHeight(FVector Position, const TMap<const UWaterBodyComponent*, float>& SplineKeyMap, float DefaultHeight, UWaterBodyComponent*& OutWaterBodyComponent, float& OutWaterDepth, FVector& OutWaterPlaneLocation, FVector& OutWaterPlaneNormal, FVector& OutWaterSurfacePosition, FVector& OutWaterVelocity, int32& OutWaterBodyIdx, bool bShouldIncludeWaves = true);
	UE_API float GetWaterHeight(FVector Position, const TMap<const UWaterBodyComponent*, float>& SplineKeyMap, float DefaultHeight, bool bShouldIncludeWaves = true);

	UFUNCTION(BlueprintCallable, Category = Cosmetic)
	UE_API void OnPontoonEnteredWater(const FSphericalPontoon& Pontoon);

	UFUNCTION(BlueprintCallable, Category = Cosmetic)
	UE_API void OnPontoonExitedWater(const FSphericalPontoon& Pontoon);

	UPROPERTY(BlueprintAssignable, Category = Cosmetic)
	FOnPontoonEnteredWater OnEnteredWaterDelegate;

	UPROPERTY(BlueprintAssignable, Category = Cosmetic)
	FOnPontoonExitedWater OnExitedWaterDelegate;

	UFUNCTION(BlueprintCallable, Category = Buoyancy)
	UE_API void GetLastWaterSurfaceInfo(FVector& OutWaterPlaneLocation, FVector& OutWaterPlaneNormal,
	FVector& OutWaterSurfacePosition, float& OutWaterDepth, int32& OutWaterBodyIdx, FVector& OutWaterVelocity);

	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category = Buoyancy)
	FBuoyancyData BuoyancyData;

protected:
	UE_API virtual void ApplyBuoyancy(UPrimitiveComponent* PrimitiveComponent);
	UE_API void ComputeBuoyancy(FSphericalPontoon& Pontoon, float ForwardSpeedKmh);
	UE_API void ComputePontoonCoefficients();

	UPROPERTY(Transient)
	TArray<TObjectPtr<UWaterBodyComponent>> CurrentWaterBodyComponents;

	// Primitive component that will be used for physics simulation.
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> SimulatingComponent;

	// async data

	FBuoyancyComponentAsyncInput* CurAsyncInput;
	FBuoyancyComponentAsyncOutput* CurAsyncOutput;
	FBuoyancyComponentAsyncOutput* NextAsyncOutput;
	EAsyncBuoyancyComponentDataType CurAsyncType;
	float OutputInterpAlpha = 0.f;

	struct FAsyncOutputWrapper
	{
		int32 Idx;
		int32 Timestamp;

		FAsyncOutputWrapper()
			: Idx(INDEX_NONE)
			, Timestamp(INDEX_NONE)
		{
		}
	};

	TArray<FAsyncOutputWrapper> OutputsWaitingOn;

	// async end

	uint32 PontoonConfiguration;
	TMap<uint32, TArray<float>> ConfiguredPontoonCoefficients;
	int32 VelocityPontoonIndex;
	int8 bIsOverlappingWaterBody : 1;
	int8 bCanBeActive : 1;
	int8 bIsInWaterBody : 1;
public:
	uint8 bUseAsyncPath : 1;

};

#undef UE_API
