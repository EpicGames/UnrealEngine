// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BuoyancyTypes.h"
#include "Chaos/DebugDrawQueue.h"
#include "DrawDebugHelpers.h"
#include "Chaos/Particle/ParticleUtilities.h"

// Frequently accessed runtime physical values
struct FBuoyancyPhysicsState
{
	FBuoyancyPhysicsState()
		: UpDir(FVector::UpVector)
		, ForwardDir(FVector::ForwardVector)
		, RightDir(FVector::RightVector)
		, LinearVelocity(FVector::ZeroVector)
		, AngularVelocityRad(FVector::ZeroVector)
		, ForwardSpeed(0.f)
		, ForwardSpeedKmh(0.f)
		, RightSpeed(0.f)
		, NumPontoonsInWater(0)
		, RiverPontoonIndex(-1)
		, bIsInWaterBody(false)
	{ }
	FVector UpDir;
	FVector ForwardDir;
	FVector RightDir;
	FVector LinearVelocity;
	FVector AngularVelocityRad;
	TArray<TPair<FSphericalPontoon, EBuoyancyEvent>> Events;
	float LinearSpeed;
	float LinearSpeedKmh;
	float ForwardSpeed; // Current speed in the direction of the forward axis of the body instance's transform (not steering forward direction)
	float ForwardSpeedKmh;
	float RightSpeed; // Current speed in the direction of the right axis
	int32 NumPontoonsInWater;
	int32 RiverPontoonIndex;
	bool bIsInWaterBody;
};

struct FBuoyancyComponentBaseAsyncAux : public FBuoyancyComponentAsyncAux
{
	FBuoyancyAuxData AuxData;

	FBuoyancyComponentBaseAsyncAux()
	{ }
};
struct FBuoyancyComponentBaseAsyncInput : public FBuoyancyComponentAsyncInput
{
	TArray<UWaterBodyComponent*> WaterBodyComponents;
	TArray<FSphericalPontoon> Pontoons;
	float SmoothedWorldTimeSeconds = 0.f;

	FBuoyancyComponentBaseAsyncInput()
		: FBuoyancyComponentAsyncInput(EAsyncBuoyancyComponentDataType::AsyncBuoyancyBase)
	{ }

	FBuoyancyComponentBaseAsyncInput(EAsyncBuoyancyComponentDataType InType)
		: FBuoyancyComponentAsyncInput(InType)
	{ }

	WATER_API virtual TUniquePtr<struct FBuoyancyComponentAsyncOutput> PreSimulate(UWorld* World, const float DeltaSeconds, const float TotalSeconds, FBuoyancyComponentAsyncAux* Aux, const TMap<UWaterBodyComponent*, TUniquePtr<FSolverSafeWaterBodyData>>& WaterBodyComponentData) const override;
	virtual ~FBuoyancyComponentBaseAsyncInput() = default;
};

struct FBuoyancySimOutput
{
	FBuoyancySimOutput() = default;

	FBuoyancySimOutput(const FBuoyancyPhysicsState& State)
		: bIsInWaterBody(State.bIsInWaterBody)
		, Events(State.Events)
	{ }

	bool bIsInWaterBody;
	TArray<TPair<FSphericalPontoon, EBuoyancyEvent>> Events;
};

struct FBuoyancyComponentBaseAsyncOutput : public FBuoyancyComponentAsyncOutput
{
	FBuoyancySimOutput SimOutput;
	FBuoyancyAuxData AuxData;
	FBuoyancyComponentBaseAsyncOutput()
		: FBuoyancyComponentAsyncOutput(EAsyncBuoyancyComponentDataType::AsyncBuoyancyBase)
	{ }

	FBuoyancyComponentBaseAsyncOutput(EAsyncBuoyancyComponentDataType InType)
		: FBuoyancyComponentAsyncOutput(InType)
	{ }
};

class FBuoyancyComponentSim
{
public:

	using TParticleUtilities = Chaos::FParticleUtilitiesXR;

	FBuoyancyComponentSim()
	{ }

	template <typename TBody, typename TAux, typename TOut>
	static void Update(const float DeltaSeconds, const float TotalSeconds, const UWorld* World, TBody* Body, const FBuoyancyData& BuoyancyData, TAux& Aux, const TMap<UWaterBodyComponent*, TUniquePtr<FSolverSafeWaterBodyData>>& WaterBodyComponentData, TOut& Out)
	{
		FBuoyancyPhysicsState State;
		FBuoyancyComponentSim::UpdatePhysicsState(Body, State);
		FBuoyancyComponentSim::UpdateBuoyancy(Body, State, BuoyancyData, Aux, WaterBodyComponentData);
		//FBuoyancyComponentSim::UpdateWaterControl(Body, State, BuoyancyData, Aux);

		if (BuoyancyData.bApplyDragForcesInWater)
		{
			FBuoyancyComponentSim::ApplyLinearDrag(Body, BuoyancyData, State);
			FBuoyancyComponentSim::ApplyAngularDrag(Body, BuoyancyData, State);
		}

		FBuoyancyComponentSim::ApplyBuoyancy(Body, Aux, State);

		if (BuoyancyData.bApplyRiverForces)
		{
			State.RiverPontoonIndex = -1;

			// Attempt to use the specified pontoon
			if (Aux.Pontoons.Num() > BuoyancyData.RiverPontoonIndex)
			{
				const FSolverSafeWaterBodyData* WaterBody = Aux.Pontoons[BuoyancyData.RiverPontoonIndex].SolverWaterBody;
				if (WaterBody && WaterBody->WaterBodyType == EWaterBodyType::River)
				{
					State.RiverPontoonIndex = BuoyancyData.RiverPontoonIndex;
				}
			}

			// If it is not in a river, use any pontoon that is in a river
			if (State.RiverPontoonIndex == -1)
			{
				for (int i = 0; i < Aux.Pontoons.Num(); ++i)
				{
					const FSolverSafeWaterBodyData* WaterBody = Aux.Pontoons[i].SolverWaterBody;
					if (WaterBody && WaterBody->WaterBodyType == EWaterBodyType::River)
					{
						State.RiverPontoonIndex = i;
						break;
					}
				}
			}

			if (BuoyancyData.bApplyDownstreamAngularRotation)
			{
				FBuoyancyComponentSim::ApplyTorqueForDownstreamAngularRotation(DeltaSeconds, Body, State, BuoyancyData, Aux);
			}
	
			FBuoyancyComponentSim::ApplyWaterForce(Body, BuoyancyData, State, Aux, DeltaSeconds);
		}
		
		// Copy into the output state
		Out = TOut(State);
	}


	// Computes torque to rotate the object downstream
	template <typename TBody, typename TState, typename TAux>
	static void ApplyTorqueForDownstreamAngularRotation(const float DeltaSeconds, TBody* Body, TState& State, const FBuoyancyData& BuoyancyData, TAux& Aux)
	{
		FVector AddedTorque(FVector::ZeroVector);

		if (State.RiverPontoonIndex != -1 && Aux.Pontoons.Num() > State.RiverPontoonIndex)
		{
			const float RotationModifier = BuoyancyData.DownstreamRotationStrength;
			const float Stiffness = BuoyancyData.DownstreamRotationStiffness;
			const float Damping = BuoyancyData.DownstreamRotationAngularDamping;
			const float MaxAccel = BuoyancyData.DownstreamMaxAcceleration;

			const FVector DownstreamWaterVelocity = Aux.Pontoons[State.RiverPontoonIndex].WaterVelocity;
			const FVector DesiredRotation = GetWorldTM(Body).TransformVectorNoScale(BuoyancyData.DownstreamAxisOfRotation);
			const FVector AxisOfAlignment = DesiredRotation;
			const FQuat CurUpToTargetUp = FQuat::FindBetweenNormals(AxisOfAlignment, DownstreamWaterVelocity);
			const FVector Axis = CurUpToTargetUp.GetRotationAxis();
			
			float Angle = CurUpToTargetUp.GetAngle();
			Angle = FMath::RadiansToDegrees(FMath::UnwindRadians(Angle));
			
			float Strength = (Angle * Stiffness - FVector::DotProduct(State.AngularVelocityRad, Axis) * Damping);
			Strength *= RotationModifier;
			Strength = FMath::Clamp(Strength, -MaxAccel, MaxAccel);

			AddedTorque = Axis * Strength;
		}

		AddTorque(Body, AddedTorque);
	}

	static float GetWaterHeight(const TArray<FSolverSafeWaterBodyData*>& WaterBodies, FVector Position, float InWaveReferenceTime, const TMap<const FSolverSafeWaterBodyData*, float>& SplineKeyMap, float DefaultHeight, FSolverSafeWaterBodyData*& OutWaterBody, float& OutWaterDepth, FVector& OutWaterPlaneLocation, FVector& OutWaterPlaneNormal, FVector& OutWaterSurfacePosition, FVector& OutWaterVelocity, int32& OutWaterBodyIdx, bool bShouldIncludeWaves = true)
	{
		float WaterHeight = DefaultHeight;
		OutWaterBody = nullptr;
		OutWaterDepth = 0.f;
		OutWaterPlaneLocation = FVector::ZeroVector;
		OutWaterPlaneNormal = FVector::UpVector;

		float MaxImmersionDepth = -1.f;
		for (FSolverSafeWaterBodyData* CurrentWaterBody : WaterBodies)
		{
			if (CurrentWaterBody)
			{
				const float SplineInputKey = SplineKeyMap.FindRef(CurrentWaterBody);

				EWaterBodyQueryFlags QueryFlags =
					EWaterBodyQueryFlags::ComputeLocation
					| EWaterBodyQueryFlags::ComputeNormal
					| EWaterBodyQueryFlags::ComputeImmersionDepth
					| EWaterBodyQueryFlags::ComputeVelocity;

				if (bShouldIncludeWaves)
				{
					QueryFlags |= EWaterBodyQueryFlags::IncludeWaves;
				}

				FWaterBodyQueryResult QueryResult = CurrentWaterBody->QueryWaterInfoClosestToWorldLocation(Position, QueryFlags, InWaveReferenceTime, SplineInputKey);
				if (QueryResult.IsInWater() && QueryResult.GetImmersionDepth() > MaxImmersionDepth)
				{
					check(!QueryResult.IsInExclusionVolume());
					WaterHeight = Position.Z + QueryResult.GetImmersionDepth();
					OutWaterBody = CurrentWaterBody;
					if (EnumHasAnyFlags(QueryResult.GetQueryFlags(), EWaterBodyQueryFlags::ComputeDepth))
					{
						OutWaterDepth = QueryResult.GetWaterSurfaceDepth();
					}
					OutWaterPlaneLocation = QueryResult.GetWaterPlaneLocation();
					OutWaterPlaneNormal = QueryResult.GetWaterPlaneNormal();
					OutWaterSurfacePosition = QueryResult.GetWaterSurfaceLocation();
					OutWaterVelocity = QueryResult.GetVelocity();
					OutWaterBodyIdx = CurrentWaterBody ? CurrentWaterBody->WaterBodyIndex : 0;
					MaxImmersionDepth = QueryResult.GetImmersionDepth();
				}
			}
		}
		return WaterHeight;
	}

	static FVector ComputeWaterForce(FSphericalPontoon& Pontoon, const FBuoyancyData& BuoyancyData, const FVector& BodyVelocity, float DeltaTime)
	{
		FVector FinalAcceleration = FVector::ZeroVector;

		const FSolverSafeWaterBodyData* WaterBody = Pontoon.SolverWaterBody;
		if (WaterBody && WaterBody->WaterBodyType == EWaterBodyType::River)
		{
			float InputKey = Pontoon.SolverSplineInputKeys[Pontoon.SolverWaterBody];
			const FVector SplinePointLocation = WaterBody->WaterSpline.GetLocationAtSplineInputKey(InputKey);
			FVector LateralPushDirection = FVector::ZeroVector;

			float LateralDistanceScale = 1.0f;

			if (BuoyancyData.WaterShorePushFactor >= 0.0f)
			{
				const FVector ShoreDirection = (Pontoon.CenterLocation - SplinePointLocation).GetSafeNormal2D();
				LateralPushDirection = ShoreDirection;
			}
			else
			{
				const FVector CenterRiverLocation = FVector(SplinePointLocation.X, SplinePointLocation.Y, 0);
				const FVector CenterPontoonLocation = FVector(Pontoon.CenterLocation.X, Pontoon.CenterLocation.Y, 0);
				const FVector CenterRiverDirection = (CenterRiverLocation - CenterPontoonLocation).GetSafeNormal2D();
				LateralPushDirection = CenterRiverDirection;

				const float DistanceFromCenter = FVector::Distance(CenterRiverLocation, CenterPontoonLocation);
				const float DistanceFromPath = DistanceFromCenter - BuoyancyData.RiverTraversalPathWidth;
				
				if (DistanceFromPath > 0)
				{
					// Apply a force inwards towards the path
					LateralDistanceScale = FMath::Lerp(0.0f, 1.0f, DistanceFromPath / BuoyancyData.RiverTraversalPathWidth);
				}
				else
				{
					// Apply a drag to any horizontal movement for stabilization within the path. Stronger drag closer to center of path, and at faster speeds.
					const float BodySpeedInLateralDir = FVector::DotProduct(BodyVelocity, LateralPushDirection.GetSafeNormal());
					const float BodySpeedLateralDrag = -FMath::Lerp(0.0f, 1.0f, BodySpeedInLateralDir / BuoyancyData.MaxWaterForce);
					const float CentralProximityDragMultiplier = FMath::Lerp(0.0f, 1.0f, (2.0f * DistanceFromCenter) / BuoyancyData.RiverTraversalPathWidth);
					LateralDistanceScale = BodySpeedLateralDrag * CentralProximityDragMultiplier;
				}
			}


			const FVector WaterVelocity = WaterBody->GetWaterVelocityVectorAtSplineInputKey(InputKey) * BuoyancyData.WaterVelocityStrength;
			const float RiverWaterSpeed = WaterBody->GetWaterVelocityAtSplineInputKey(InputKey);
			const float BodySpeedInWater = FVector::DotProduct(BodyVelocity, WaterVelocity.GetSafeNormal());
			const float BodySpeedInWaterDir = BuoyancyData.bAllowCurrentWhenMovingFastUpstream ? BodySpeedInWater : FMath::Abs(BodySpeedInWater);
			const bool bApplyRiverForces = (BodySpeedInWaterDir < RiverWaterSpeed);

			if (bApplyRiverForces || BuoyancyData.bAlwaysAllowLateralPush)
			{
				const FVector LateralVelocity = LateralPushDirection * LateralDistanceScale * FMath::Abs(BuoyancyData.WaterShorePushFactor);
				const FVector LateralAcceleration = LateralVelocity / DeltaTime;
				FinalAcceleration = FinalAcceleration + LateralAcceleration.GetClampedToSize(-BuoyancyData.MaxShorePushForce, BuoyancyData.MaxShorePushForce);
			}

			if (bApplyRiverForces)
			{
				const FVector WaterAcceleration = (WaterVelocity / DeltaTime);
				FinalAcceleration = FinalAcceleration + WaterAcceleration.GetClampedToSize(-BuoyancyData.MaxWaterForce, BuoyancyData.MaxWaterForce);
			}
		}

		return FinalAcceleration;
	}

	static void ComputeBuoyancy(const FBuoyancyData& BuoyancyData, FSphericalPontoon& Pontoon, float ForwardSpeedKmh, float VelocityZ)
	{
		auto ComputeBuoyantForce = [&](FVector CenterLocation, float Radius, float InBuoyancyCoefficient, float CurrentWaterLevel) -> float
		{
			const float Bottom = CenterLocation.Z - Radius;
			const float SubDiff = FMath::Clamp(CurrentWaterLevel - Bottom, 0.f, 2.f * Radius);

			// The following was obtained by integrating the volume of a sphere
			// over a linear section of SubmersionDiff length.
			static const float Pi = (float)PI;
			const float SubDiffSq = SubDiff * SubDiff;
			const float SubVolume = (Pi / 3.f) * SubDiffSq * ((3.f * Radius) - SubDiff);

			//#if ENABLE_DRAW_DEBUG
			//			if (CVarWaterDebugBuoyancy.GetValueOnAnyThread())
			//			{
			//				const FVector WaterPoint = FVector(CenterLocation.X, CenterLocation.Y, CurrentWaterLevel);
			//				DrawDebugLine(GetWorld(), WaterPoint - 50.f * FVector::ForwardVector, WaterPoint + 50.f * FVector::ForwardVector, FColor::Blue, false, -1.f, 0, 3.f);
			//				DrawDebugLine(GetWorld(), WaterPoint - 50.f * FVector::RightVector, WaterPoint + 50.f * FVector::RightVector, FColor::Blue, false, -1.f, 0, 3.f);
			//			}
			//#endif
			const float FirstOrderDrag = BuoyancyData.BuoyancyDamp * VelocityZ;
			const float SecondOrderDrag = FMath::Sign(VelocityZ) * BuoyancyData.BuoyancyDamp2 * VelocityZ * VelocityZ;
			const float DampingFactor = -FMath::Max(FirstOrderDrag + SecondOrderDrag, 0.f);
			// The buoyant force scales with submersed volume
			return SubVolume * (InBuoyancyCoefficient)+DampingFactor;
		};

		const float MinVelocity = BuoyancyData.BuoyancyRampMinVelocity;
		const float MaxVelocity = BuoyancyData.BuoyancyRampMaxVelocity;
		const float RampFactor = FMath::Clamp((ForwardSpeedKmh - MinVelocity) / (MaxVelocity - MinVelocity), 0.f, 1.f);
		const float BuoyancyRamp = RampFactor * (BuoyancyData.BuoyancyRampMax - 1);
		float BuoyancyCoefficientWithRamp = BuoyancyData.BuoyancyCoefficient * (1 + BuoyancyRamp);

		const float BuoyantForce = FMath::Clamp(ComputeBuoyantForce(Pontoon.CenterLocation, Pontoon.Radius, BuoyancyCoefficientWithRamp, Pontoon.WaterHeight), 0.f, BuoyancyData.MaxBuoyantForce);
		Pontoon.LocalForce = FVector::UpVector * BuoyantForce * Pontoon.PontoonCoefficient;
	}

	template <typename TBody, typename TState, typename TAux>
	static void UpdateBuoyancy(const TBody* Body, TState& State, const FBuoyancyData& BuoyancyData, TAux& Aux, const TMap<UWaterBodyComponent*, TUniquePtr<FSolverSafeWaterBodyData>>& WaterBodyComponentData)
	{
		State.NumPontoonsInWater = 0;

		TArray<FSolverSafeWaterBodyData*> SolverWaterBodies;
		for (const UWaterBodyComponent* WaterBodyComponent : Aux.WaterBodyComponents)
		{
			if (const TUniquePtr<FSolverSafeWaterBodyData>* WaterDataPtrPtr = WaterBodyComponentData.Find(WaterBodyComponent))
			{
				FSolverSafeWaterBodyData& SolverWaterBody = **WaterDataPtrPtr;
				SolverWaterBodies.Add(&SolverWaterBody);
			}
		}

		int PontoonIndex = 0;
		for (FSphericalPontoon& Pontoon : Aux.Pontoons)
		{
			//if (PontoonConfiguration & (1 << PontoonIndex))
			{
				const Chaos::FRigidTransform3 CurrentTransform = TParticleUtilities::GetActorWorldTransform(Body);
				if (Pontoon.bUseCenterSocket)
				{
					Pontoon.CenterLocation = CurrentTransform.TransformPosition(Pontoon.SocketTransform.GetLocation()) + Pontoon.Offset;
					Pontoon.SocketRotation = CurrentTransform.TransformRotation(Pontoon.SocketTransform.GetRotation());
				}
				else
				{
					Pontoon.CenterLocation = CurrentTransform.TransformPosition(Pontoon.RelativeLocation);
				}

				//GetWaterSplineKey(Pontoon.CenterLocation, Pontoon.SplineInputKeys, Pontoon.SplineSegments);
				{
					Pontoon.SolverSplineInputKeys.Reset();
					for (const FSolverSafeWaterBodyData* WaterBody : SolverWaterBodies)
					{
						if (WaterBody && WaterBody->WaterBodyType == EWaterBodyType::River)
						{
							float SplineInputKey;
							//if (CVarWaterUseSplineKeyOptimization.GetValueOnAnyThread())
							//{
							//	SplineInputKey = GetWaterSplineKeyFast(Location, WaterBody, OutSegmentMap);
							//}
							//else
							{
								SplineInputKey = WaterBody->WaterSpline.FindInputKeyClosestToWorldLocation(Pontoon.CenterLocation);
							}
							Pontoon.SolverSplineInputKeys.Add(WaterBody, SplineInputKey);
						}
					}
				}

				const FVector PontoonBottom = Pontoon.CenterLocation - FVector(0, 0, Pontoon.Radius);
				const float DefaultHeight = -100000.f;
				/*Pass in large negative default value so we don't accidentally assume we're in water when we're not.*/
				Pontoon.WaterHeight = GetWaterHeight(SolverWaterBodies, PontoonBottom - FVector::UpVector * 100.f, Aux.SmoothedWorldTimeSeconds, Pontoon.SolverSplineInputKeys, DefaultHeight, Pontoon.SolverWaterBody, Pontoon.WaterDepth, Pontoon.WaterPlaneLocation, Pontoon.WaterPlaneNormal, Pontoon.WaterSurfacePosition, Pontoon.WaterVelocity, Pontoon.WaterBodyIndex);

				const bool bPrevIsInWater = Pontoon.bIsInWater;
				const float ImmersionDepth = Pontoon.WaterHeight - PontoonBottom.Z;
				/*check if the pontoon is currently in water*/
				if (ImmersionDepth >= 0.f)
				{
					Pontoon.bIsInWater = true;
					Pontoon.ImmersionDepth = ImmersionDepth;
					State.NumPontoonsInWater++;
				}
				else
				{
					Pontoon.bIsInWater = false;
					Pontoon.ImmersionDepth = 0.f;
				}

				ComputeBuoyancy(BuoyancyData, Pontoon, State.ForwardSpeedKmh, State.LinearVelocity.Z);

				if (Pontoon.bIsInWater && !bPrevIsInWater)
				{
					Pontoon.SplineSegments.Reset();
					// BlueprintImplementables don't really work on the actor component level unfortunately, so call back in to the function defined on the actor itself.
					//OnPontoonEnteredWater(Pontoon);
					State.Events.Add(MakeTuple(Pontoon, EBuoyancyEvent::EnteredWaterBody));
				}
				if (!Pontoon.bIsInWater && bPrevIsInWater)
				{
					Pontoon.SplineSegments.Reset();
					State.Events.Add(MakeTuple(Pontoon, EBuoyancyEvent::ExitedWaterBody));
					//OnPontoonExitedWater(Pontoon);
				}
			}
			PontoonIndex++;
		}
#if CHAOS_DEBUG_DRAW
		if (CVarWaterDebugBuoyancy.GetValueOnAnyThread())
		{
			const float NumPoints = CVarWaterBuoyancyDebugPoints.GetValueOnAnyThread();
			const float Size = CVarWaterBuoyancyDebugSize.GetValueOnAnyThread();
			const float StartOffset = NumPoints * 0.5f;
			const float Scale = Size / NumPoints;
			TMap<const FSolverSafeWaterBodyData*, float> DebugSplineKeyMap;
			Chaos::FDebugDrawQueue::GetInstance().SetEnabled(true);
			for (int i = 0; i < NumPoints; ++i)
			{
				for (int j = 0; j < NumPoints; ++j)
				{
					FVector Location = TParticleUtilities::GetCoMWorldPosition(Body) + (FVector::RightVector * (i - StartOffset) * Scale) + (FVector::ForwardVector * (j - StartOffset) * Scale);

					FSphericalPontoon DummyPontoon;
					for (const FSolverSafeWaterBodyData* WaterBody : SolverWaterBodies)
					{
						if (WaterBody && WaterBody->WaterBodyType == EWaterBodyType::River)
						{
							float SplineInputKey;
							//if (CVarWaterUseSplineKeyOptimization.GetValueOnAnyThread())
							//{
							//	SplineInputKey = GetWaterSplineKeyFast(Location, WaterBody, OutSegmentMap);
							//}
							//else
							{
								SplineInputKey = WaterBody->WaterSpline.FindInputKeyClosestToWorldLocation(Location);
							}
							DummyPontoon.SolverSplineInputKeys.Add(WaterBody, SplineInputKey);
						}
					}
					

					const float WaterHeight = GetWaterHeight(SolverWaterBodies, Location - FVector::UpVector * 100.f, Aux.SmoothedWorldTimeSeconds, DummyPontoon.SolverSplineInputKeys, /*DefaultHeight*/-100000.f, DummyPontoon.SolverWaterBody, DummyPontoon.WaterDepth, DummyPontoon.WaterPlaneLocation, DummyPontoon.WaterPlaneNormal, DummyPontoon.WaterSurfacePosition, DummyPontoon.WaterVelocity, DummyPontoon.WaterBodyIndex);
					const FVector DebugPoint(Location.X, Location.Y, WaterHeight);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(DebugPoint, FColor::Green, false, -1.f, 0, 5.f);
				}
			}

			for (FSphericalPontoon& Pontoon : Aux.Pontoons)
			{
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(Pontoon.CenterLocation, Pontoon.Radius, 16, Pontoon.bIsInWater ? FColor::Blue : FColor::Red, false, -1.f, 0, 1.f);
			}
		}
#endif
		
		State.bIsInWaterBody = State.NumPontoonsInWater > 0;
	}

	template <typename TBody, typename TAux, typename TState>
	static void ApplyBuoyancy(TBody* Body, TAux& Aux, const TState& State)
	{
		//int PontoonIndex = 0;
		for (const FSphericalPontoon& Pontoon : Aux.Pontoons)
		{
			//if (PontoonConfiguration & (1 << PontoonIndex))
			//{
			AddForceAtPositionWorld(Body, Pontoon.LocalForce, Pontoon.CenterLocation);
			//}
			//PontoonIndex++;
		}
	}

	template <typename TBody, typename TState>
	static void UpdatePhysicsState(const TBody* Body, TState& State)
	{
		State.LinearVelocity = GetVelocity(Body);
		State.AngularVelocityRad = GetAngularVelocity(Body);
		State.LinearSpeed = State.LinearVelocity.Size();
		State.LinearSpeedKmh = ToKmH(State.LinearSpeed);
		State.ForwardDir = GetWorldTM(Body).GetUnitAxis(EAxis::X);
		State.RightDir = GetWorldTM(Body).GetUnitAxis(EAxis::Y);
		State.UpDir = GetWorldTM(Body).GetUnitAxis(EAxis::Z);
		State.ForwardSpeed = FVector::DotProduct(State.ForwardDir, State.LinearVelocity);
		State.ForwardSpeedKmh = ToKmH(State.ForwardSpeed);
		State.RightSpeed = FVector::DotProduct(State.RightDir, State.LinearVelocity);
	}

	template <typename TBody>
	static FVector GetVelocity(const TBody* Body)
	{
		return Body->V();
	}

	template <typename TBody>
	static FVector GetAngularVelocity(const TBody* Body)
	{
		return Body->W();
	}

	static float ToKmH(float Speed)
	{
		return Speed * 0.036f;
	}

	template <typename TBody>
	static FTransform GetWorldTM(const TBody* Body)
	{
		//return FTransform::Identity;
		return TParticleUtilities::GetActorWorldTransform(Body);
	}

	template <typename TBody>
	static void AddForce(TBody* Body, const FVector& Force)
	{
		if (ensure(!Force.ContainsNaN()))
		{
			Body->AddForce((Force * Body->M()));
		}
	}

	template <typename TBody>
	static void AddForceAtPositionWorld(TBody* Body, const FVector& WorldForce, const FVector& WorldPosition)
	{
		if (ensure(!WorldForce.ContainsNaN() && !WorldPosition.ContainsNaN()))
		{
			const Chaos::FVec3 WorldCOM = TParticleUtilities::GetCoMWorldPosition(Body);
			const Chaos::FVec3 WorldTorque = Chaos::FVec3::CrossProduct(WorldPosition - WorldCOM, WorldForce);
			Body->AddForce(WorldForce);
			Body->AddTorque(WorldTorque);
		}
	}

	template <typename TBody>
	static void AddTorque(TBody* Body, const FVector& TorqueRadians)
	{
		if (ensure(!TorqueRadians.ContainsNaN()))
		{
			Body->AddTorque((TParticleUtilities::GetWorldInertia(Body) * TorqueRadians));
		}
	}

	template <typename TBody, typename TState, typename TAux>
	static void ApplyWaterForce(TBody* Body, const FBuoyancyData& BuoyancyData, const TState& State, TAux& Aux, float DeltaSeconds)
	{
		FVector WaterForce = FVector::ZeroVector;
		
		if (State.RiverPontoonIndex != -1 && Aux.Pontoons.Num() > State.RiverPontoonIndex)
		{
			WaterForce = ComputeWaterForce(Aux.Pontoons[State.RiverPontoonIndex], BuoyancyData, State.LinearVelocity, DeltaSeconds);
		}

		AddForce(Body, WaterForce);
	}

	template <typename TBody, typename TState>
	static void ApplyLinearDrag(TBody* Body, const FBuoyancyData& BuoyancyData, const TState& State)
	{
		if (State.bIsInWaterBody)
		{
			FVector DragForce = FVector::ZeroVector;

			FVector PlaneVelocity = State.LinearVelocity;
			PlaneVelocity.Z = 0;
			const FVector VelocityDir = PlaneVelocity.GetSafeNormal();
			const float SpeedKmh = ToKmH(PlaneVelocity.Size());
			const float ClampedSpeed = FMath::Clamp(SpeedKmh, -BuoyancyData.MaxDragSpeed, BuoyancyData.MaxDragSpeed);

			const float Resistance = ClampedSpeed * BuoyancyData.DragCoefficient;
			DragForce += -Resistance * VelocityDir;

			const float Resistance2 = ClampedSpeed * ClampedSpeed * BuoyancyData.DragCoefficient2;
			DragForce += -Resistance2 * VelocityDir * FMath::Sign(SpeedKmh);
			AddForce(Body, DragForce);
		}
	}

	template <typename TBody, typename TState>
	static void ApplyAngularDrag(TBody* Body, const FBuoyancyData& BuoyancyData, const TState& State)
	{
		if (State.bIsInWaterBody)
		{
			const FVector DragTorque = -State.AngularVelocityRad * BuoyancyData.AngularDragCoefficient;
			AddTorque(Body, DragTorque);
		}
	}
};
