// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BuoyancyTypes.h" // IWYU pragma: keep
#include "Chaos/SimCallbackObject.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "BuoyancyManager.generated.h"

#define UE_API WATER_API

namespace EEndPlayReason { enum Type : int; }
struct FBuoyancyComponentAsyncAux;
struct FBuoyancyManagerAsyncInput;
struct FBuoyancyManagerAsyncOutput;

class UBuoyancyComponent;
class AWaterBody;

class FBuoyancyManagerAsyncCallback : public Chaos::TSimCallbackObject<FBuoyancyManagerAsyncInput, FBuoyancyManagerAsyncOutput>
{
public:
	virtual FName GetFNameForStatId() const override;

	void CreateAsyncAux_External(Chaos::FUniqueIdx UniqueIndex, TUniquePtr<FBuoyancyComponentAsyncAux>&& Aux);
	void ClearAsyncAux_External(Chaos::FUniqueIdx UniqueIndex);

private:
	virtual void OnPreSimulate_Internal() override;

	TMap<Chaos::FUniqueIdx, TUniquePtr<FBuoyancyComponentAsyncAux>> BuoyancyComponentToAux_Internal;
};

UCLASS(MinimalAPI)
class ABuoyancyManager : public AActor
{
	GENERATED_UCLASS_BODY()

public:

	static UE_API ABuoyancyManager* Get(const UObject* WorldContextObject);

	UFUNCTION()
	static UE_API bool GetBuoyancyComponentManager(const UObject* WorldContextObject, ABuoyancyManager*& Manager);

	UE_API void OnCreatePhysics(UActorComponent* Component);
	UE_API void OnDestroyPhysics(UActorComponent* Component);

	UE_API void ClearAsyncInputs(UBuoyancyComponent* Component);

	UE_API void Register(UBuoyancyComponent* BuoyancyComponent);
	UE_API void Unregister(UBuoyancyComponent* BuoyancyComponent);

	UE_API void Update(FPhysScene* PhysScene, float DeltaTime);

#if WITH_EDITOR
	// Prevent import/export of buoyancy manager actors since they should be transient and are always spawned when the map is loaded.
	virtual bool ShouldImport(FStringView ActorPropString, bool IsMovingLevel) override { return false; }
	
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }

	virtual bool IsSelectable() const override { return false; }
#endif // WITH_EDITOR
private:
	UE_API void InitializeAsyncAux(UBuoyancyComponent* Component);
	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	TSet<const UPrimitiveComponent*> PhysicsInitializedSimulatingComponents;

	UPROPERTY()
	TArray<TObjectPtr<UBuoyancyComponent>> BuoyancyComponents;
	TArray<UBuoyancyComponent*> BuoyancyComponentsActive;
	
	// List of buoyancy components to defer registration until this buoyancy manager is fully initialized.
	TArray<TWeakObjectPtr<UBuoyancyComponent>> BuoyancyComponentsToRegister;

	FBuoyancyManagerAsyncCallback* AsyncCallback;
	int32 Timestamp;

	TArray<Chaos::TSimCallbackOutputHandle<FBuoyancyManagerAsyncOutput>> PendingOutputs;
	Chaos::TSimCallbackOutputHandle<FBuoyancyManagerAsyncOutput> LatestOutput;

	FDelegateHandle OnCreateDelegateHandle;
	FDelegateHandle OnDestroyDelegateHandle;
	FDelegateHandle OnPhysScenePreTickHandle;
};

#undef UE_API
