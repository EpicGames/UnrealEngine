// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/SimCallbackInput.h"
#include "WaterBodyTypes.h"
#include "BuoyancyTypes.generated.h"

class UBuoyancyComponent;
class AWaterBody;

extern WATER_API TAutoConsoleVariable<int32> CVarWaterDebugBuoyancy;
extern WATER_API TAutoConsoleVariable<int32> CVarWaterBuoyancyDebugPoints;
extern WATER_API TAutoConsoleVariable<int32> CVarWaterBuoyancyDebugSize;

USTRUCT(Blueprintable)
struct FSphericalPontoon
{
	GENERATED_BODY()

	/** The socket to center this pontoon on. Also used as the name of the pontoon for effects */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Buoyancy)
	FName CenterSocket;

	/** Relative Location of pontoon WRT parent actor. Overridden by Center Socket. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Buoyancy)
	FVector RelativeLocation;

	/** The radius of the pontoon */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Buoyancy)
	float Radius;

	/** Should this pontoon be considered as a candidate location for visual/audio effects upon entering water for burst cues? To be implemented by user*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Buoyancy)
	bool bFXEnabled = true;

	UPROPERTY(BlueprintReadOnly, Category = Buoyancy)
	FVector LocalForce;

	UPROPERTY(BlueprintReadOnly, Category = Buoyancy)
	FVector CenterLocation;

	UPROPERTY(BlueprintReadOnly, Category = Buoyancy)
	FQuat SocketRotation;

	UPROPERTY(BlueprintReadOnly, Category = Buoyancy)
	FVector Offset;

	float PontoonCoefficient;

	UPROPERTY(BlueprintReadOnly, Category = Buoyancy)
	float WaterHeight;

	UPROPERTY(BlueprintReadOnly, Category = Buoyancy)
	float WaterDepth;

	UPROPERTY(BlueprintReadOnly, Category = Buoyancy)
	float ImmersionDepth;

	UPROPERTY(BlueprintReadOnly, Category = Buoyancy)
	FVector WaterPlaneLocation;

	UPROPERTY(BlueprintReadOnly, Category = Buoyancy)
	FVector WaterPlaneNormal;

	UPROPERTY(BlueprintReadOnly, Category = Buoyancy)
	FVector WaterSurfacePosition;

	UPROPERTY(BlueprintReadOnly, Category = Buoyancy)
	FVector WaterVelocity;

	UPROPERTY(BlueprintReadOnly, Category = Buoyancy)
	int32 WaterBodyIndex;

	UPROPERTY(BlueprintReadOnly, Category = Buoyancy)
	bool bIsInWater;
	
	FTransform SocketTransform;

	TMap<const UWaterBodyComponent*, float> SplineInputKeys;
	TMap<const UWaterBodyComponent*, float> SplineSegments;

	TMap<const FSolverSafeWaterBodyData*, float> SolverSplineInputKeys;
	TMap<const FSolverSafeWaterBodyData*, float> SolverSplineSegments;

	uint8 bEnabled : 1;
	uint8 bUseCenterSocket : 1;

	UPROPERTY(Transient, BlueprintReadOnly, Category = Buoyancy)
	TObjectPtr<UWaterBodyComponent> CurrentWaterBodyComponent;

	FSolverSafeWaterBodyData* SolverWaterBody;

	FSphericalPontoon()
		: RelativeLocation(FVector::ZeroVector)
		, Radius(100.f)
		, LocalForce(FVector::ZeroVector)
		, CenterLocation(FVector::ZeroVector)
		, SocketRotation(FQuat::Identity)
		, Offset(FVector::ZeroVector)
		, PontoonCoefficient(1.f)
		, WaterHeight(-10000.f)
		, WaterDepth(0.f)
		, ImmersionDepth(0.f)
		, WaterPlaneLocation(FVector::ZeroVector)
		, WaterPlaneNormal(FVector::UpVector)
		, WaterSurfacePosition(FVector::ZeroVector)
		, WaterVelocity(FVector::ZeroVector)
		, WaterBodyIndex(0)
		, bIsInWater(false)
		, SocketTransform(FTransform::Identity)
		, bEnabled(true)
		, bUseCenterSocket(false)
		, CurrentWaterBodyComponent(nullptr)
		, SolverWaterBody(nullptr)
	{
	}

	void CopyDataFromPT(const FSphericalPontoon& PTPontoon)
	{
		LocalForce = PTPontoon.LocalForce;
		CenterLocation = PTPontoon.CenterLocation;
		SocketRotation = PTPontoon.SocketRotation;
		WaterHeight = PTPontoon.WaterHeight;
		bIsInWater = PTPontoon.bIsInWater;
		ImmersionDepth = PTPontoon.ImmersionDepth;
		WaterDepth = PTPontoon.WaterDepth;
		WaterPlaneLocation = PTPontoon.WaterPlaneLocation;
		WaterPlaneNormal = PTPontoon.WaterPlaneNormal;
		WaterSurfacePosition = PTPontoon.WaterSurfacePosition;
		WaterVelocity = PTPontoon.WaterVelocity;
		WaterBodyIndex = PTPontoon.WaterBodyIndex;
		CurrentWaterBodyComponent = PTPontoon.CurrentWaterBodyComponent;
	}

	//void CopyDataToPT(const FSphericalPontoon& GTPontoon)
	//{
	//	CenterSocket = GTPontoon.CenterSocket;
	//	RelativeLocation = GTPontoon.RelativeLocation;
	//	Radius = GTPontoon.Radius;
	//	Offset = GTPontoon.Offset;
	//	PontoonCoefficient = GTPontoon.PontoonCoefficient;
	//	SocketTransform = GTPontoon.SocketTransform;
	//	bEnabled = GTPontoon.bEnabled;
	//	bUseCenterSocket = GTPontoon.bUseCenterSocket;
	//}

	void Serialize(FArchive& Ar)
	{
		Ar << CenterSocket;
		Ar << RelativeLocation;
		Ar << Radius;
		Ar << LocalForce;
		Ar << CenterLocation;
		Ar << SocketRotation;
		Ar << Offset;
		Ar << PontoonCoefficient;
		Ar << WaterHeight;
		Ar << WaterDepth;
		Ar << ImmersionDepth;
		Ar << WaterPlaneLocation;
		Ar << WaterPlaneNormal;
		Ar << WaterSurfacePosition;
		Ar << WaterVelocity;
		Ar << WaterBodyIndex;
		Ar << bIsInWater;
		Ar << SocketTransform;
		uint8 Enabled = bEnabled;
		uint8 UseCenterSocket = bUseCenterSocket;
		Ar << Enabled;
		Ar << UseCenterSocket;
		bEnabled = Enabled;
		bUseCenterSocket = UseCenterSocket;
	}
};

USTRUCT(Blueprintable)
struct FBuoyancyData
{
	GENERATED_BODY()

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Buoyancy)
	TArray<FSphericalPontoon> Pontoons;

	/** If true, center pontoons around center of mass when using relative locations
		(not used when pontoon locations are specified via sockets) */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Buoyancy)
	bool bCenterPontoonsOnCOM = true;

	/** Increases buoyant force applied on each pontoon. */
	UPROPERTY(EditDefaultsOnly, Category = Buoyancy)
	float BuoyancyCoefficient;

	/** Damping factor to scale damping based on Z velocity. */
	UPROPERTY(EditDefaultsOnly, Category = Buoyancy)
	float BuoyancyDamp;

	/**Second Order Damping factor to scale damping based on Z velocity. */
	UPROPERTY(EditDefaultsOnly, Category = Buoyancy)
	float BuoyancyDamp2;

	/** Minimum velocity to start applying a ramp to buoyancy. */
	UPROPERTY(EditDefaultsOnly, Category = Buoyancy)
	float BuoyancyRampMinVelocity;

	/** Maximum velocity until which the buoyancy can ramp up. */
	UPROPERTY(EditDefaultsOnly, Category = Buoyancy)
	float BuoyancyRampMaxVelocity;

	/** Maximum value that buoyancy can ramp to (at or beyond max velocity). */
	UPROPERTY(EditDefaultsOnly, Category = Buoyancy)
	float BuoyancyRampMax;

	/** Maximum buoyant force in the Up direction. */
	UPROPERTY(EditDefaultsOnly, Category = Buoyancy)
	float MaxBuoyantForce;

	UPROPERTY(EditDefaultsOnly, Category = Buoyancy)
	bool bApplyDragForcesInWater = false;

	/** Coefficient for applying linear drag based on speed */
	UPROPERTY(EditDefaultsOnly, Category = Buoyancy, Meta = (EditCondition = "bApplyDragForcesInWater"))
	float DragCoefficient = 20.f;

	/** Coefficient for applying linear drag based on the square of the speed */
	UPROPERTY(EditDefaultsOnly, Category = Buoyancy, Meta = (EditCondition = "bApplyDragForcesInWater"))
	float DragCoefficient2 = 0.01f;

	/** Coefficient for applying angular drag that resists the rotation of the object */
	UPROPERTY(EditDefaultsOnly, Category = Buoyancy, Meta = (EditCondition = "bApplyDragForcesInWater"))
	float AngularDragCoefficient = 1.f;

	/** Max speed for which drag force is applied */
	UPROPERTY(EditDefaultsOnly, Category = Buoyancy, Meta = (EditCondition = "bApplyDragForcesInWater"))
	float MaxDragSpeed = 15.f;

	/** Whether we should apply river forces such as downstream push and shore push */
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior")
	bool bApplyRiverForces = true;

	/** Pontoon to calculate water forces from. Used to calculate lateral push/pull, to grab water velocity for main force calculations from for downstream calculation if possible.*/
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior", Meta = (EditCondition = "bApplyRiverForces", ClampMin = 0))
	int RiverPontoonIndex = 0;

	/** Coefficient for nudging objects to shore in Rivers (for perf reasons). Or, set negative to push towards center of river. */
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior", Meta = (EditCondition = "bApplyRiverForces"))
	float WaterShorePushFactor;

	/** Path width along the inside of the river which the object should traverse */
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior", Meta = (EditCondition = "bApplyRiverForces", ClampMin = 1.0f))
	float RiverTraversalPathWidth = 300.0f;

	/** Maximum push force that can be applied by riverths towards the center or edge. */
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior", Meta = (EditCondition = "bApplyRiverForces"))
	float MaxShorePushForce;

	/** Coefficient for applying push force in rivers. */
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior", Meta = (EditCondition = "bApplyRiverForces"))
	float WaterVelocityStrength;

	/** Maximum push force that can be applied by rivers. */
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior", Meta = (EditCondition = "bApplyRiverForces"))
	float MaxWaterForce;

	/** Allow an object to be pushed laterally regardless of the forward movement speed through the river */
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior", Meta = (EditCondition = "bApplyRiverForces"))
	bool bAlwaysAllowLateralPush = false;
	
	/** Apply the current when moving at high speeds upstream. Disable for vehicles to have more control*/
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior", Meta = (EditCondition = "bApplyRiverForces"))
	bool bAllowCurrentWhenMovingFastUpstream = false;

	/** Apply torque to align the object along the downstream direction of the river */
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior", Meta = (EditCondition = "bApplyRiverForces"))
	bool bApplyDownstreamAngularRotation = false;

	/** The axis with respect to the object that the downstream angular rotation should be aligned */
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior", Meta = (EditCondition = "bApplyRiverForces && bApplyDownstreamAngularRotation"))
	FVector DownstreamAxisOfRotation;

	/** Strength of the downstream angular rotation application */
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior", Meta = (EditCondition = "bApplyRiverForces && bApplyDownstreamAngularRotation", ClampMin = 0.0f, ClampMax = 1.0f))
	float DownstreamRotationStrength = 0.05f;

	/** Stiffness of the spring used to align the object along the downstream direction */
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior", Meta = (EditCondition = "bApplyRiverForces && bApplyDownstreamAngularRotation"))
	float DownstreamRotationStiffness = 20.f;

	/** Damping of the spring used to align the object along the downstream direction */
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior", Meta = (EditCondition = "bApplyRiverForces && bApplyDownstreamAngularRotation"))
	float DownstreamRotationAngularDamping = 5.0f;

	/** Maximum torque to apply per update for downstream rotation */
	UPROPERTY(EditDefaultsOnly, Category = "Buoyancy | River Behavior", Meta = (EditCondition = "bApplyRiverForces && bApplyDownstreamAngularRotation"))
	float DownstreamMaxAcceleration = 10.0f;

	FBuoyancyData()
		: BuoyancyCoefficient(0.1f)
		, BuoyancyDamp(1000.f)
		, BuoyancyDamp2(1.f)
		, BuoyancyRampMinVelocity(20.f)
		, BuoyancyRampMaxVelocity(50.f)
		, BuoyancyRampMax(1.f)
		, MaxBuoyantForce(5000000.f)
		, WaterShorePushFactor(0.3f)
		, RiverTraversalPathWidth(300.0f)
		, MaxShorePushForce(300.0f)
		, WaterVelocityStrength(0.01f)
		, MaxWaterForce(10000.f)
		, DownstreamAxisOfRotation(FVector::ZeroVector)
	{
	}

	void Serialize(FArchive& Ar)
	{
		int32 NumPontoons = Pontoons.Num();
		Ar << NumPontoons;
		if (Ar.IsLoading())
		{
			Pontoons.SetNum(NumPontoons);
		}
		for (int32 Index = 0; Index < NumPontoons; ++Index)
		{
			Pontoons[Index].Serialize(Ar);
		}
		Ar << RiverPontoonIndex;
		Ar << BuoyancyCoefficient;
		Ar << BuoyancyDamp;
		Ar << BuoyancyDamp2;
		Ar << BuoyancyRampMinVelocity;
		Ar << BuoyancyRampMaxVelocity;
		Ar << BuoyancyRampMax;
		Ar << MaxBuoyantForce;
		Ar << bApplyDragForcesInWater;
		Ar << DragCoefficient;
		Ar << DragCoefficient2;
		Ar << AngularDragCoefficient;
		Ar << MaxDragSpeed;
		Ar << bApplyRiverForces;
		Ar << WaterShorePushFactor;
		Ar << WaterVelocityStrength;
		Ar << MaxWaterForce;
		Ar << bAlwaysAllowLateralPush;
		Ar << bAllowCurrentWhenMovingFastUpstream;
		Ar << RiverTraversalPathWidth;
		Ar << bApplyDownstreamAngularRotation;
		Ar << DownstreamAxisOfRotation;
		Ar << DownstreamRotationStrength;
		Ar << DownstreamRotationStiffness;
		Ar << DownstreamRotationAngularDamping;
		Ar << DownstreamMaxAcceleration;
	}
};

/* async structs */

enum EAsyncBuoyancyComponentDataType : int8
{
	AsyncBuoyancyInvalid,
	AsyncBuoyancyBase,
	AsyncBuoyancyVehicle,
	AsyncBuoyancyBoat
};

UENUM()
enum class EBuoyancyEvent : uint8
{
	EnteredWaterBody,
	ExitedWaterBody
};

struct FBuoyancyAuxData
{
	FBuoyancyAuxData()
		: SmoothedWorldTimeSeconds(0.f)
	{ }

	TArray<FSphericalPontoon> Pontoons;
	TArray<UWaterBodyComponent*> WaterBodyComponents;
	float SmoothedWorldTimeSeconds;
};

// Auxiliary, persistent data which the update can use
struct FBuoyancyComponentAsyncAux
{
	FBuoyancyData BuoyancyData;

	FBuoyancyComponentAsyncAux() = default;
	virtual ~FBuoyancyComponentAsyncAux() = default;
};

struct FBuoyancyComponentAsyncInput
{
	const EAsyncBuoyancyComponentDataType Type;
	const UBuoyancyComponent* BuoyancyComponent;

	Chaos::FSingleParticlePhysicsProxy* Proxy;

	virtual TUniquePtr<struct FBuoyancyComponentAsyncOutput> PreSimulate(UWorld* World, const float DeltaSeconds, const float TotalSeconds, FBuoyancyComponentAsyncAux* Aux, const TMap<UWaterBodyComponent*, TUniquePtr<FSolverSafeWaterBodyData>>& WaterBodyComponentData) const = 0;

	FBuoyancyComponentAsyncInput(EAsyncBuoyancyComponentDataType InType = EAsyncBuoyancyComponentDataType::AsyncBuoyancyInvalid)
		: Type(InType)
		, BuoyancyComponent(nullptr)
	{
		Proxy = nullptr;	//indicates async/sync task not needed. This can happen due to various logic when update is not needed
	}

	virtual ~FBuoyancyComponentAsyncInput() = default;
};

struct FBuoyancyManagerAsyncInput : public Chaos::FSimCallbackInput
{
	TArray<TUniquePtr<FBuoyancyComponentAsyncInput>> Inputs;
	TMap<UWaterBodyComponent*, TUniquePtr<FSolverSafeWaterBodyData>> WaterBodyComponentToSolverData;
	TWeakObjectPtr<UWorld> World;
	int32 Timestamp = INDEX_NONE;

	void Reset()
	{
		Inputs.Reset();
		World.Reset();
		WaterBodyComponentToSolverData.Reset();
	}
};

struct FBuoyancyComponentAsyncOutput
{
	const EAsyncBuoyancyComponentDataType Type;
	bool bValid;	//indicates no work was actually done. This is here because it can early out due to a lot of internal logic and we still want to go wide

	FBuoyancyComponentAsyncOutput(EAsyncBuoyancyComponentDataType InType = EAsyncBuoyancyComponentDataType::AsyncBuoyancyInvalid)
		: Type(InType)
		, bValid(false)
	{ }

	virtual ~FBuoyancyComponentAsyncOutput() = default;
};

struct FBuoyancyManagerAsyncOutput : public Chaos::FSimCallbackOutput
{
	TArray<TUniquePtr<FBuoyancyComponentAsyncOutput>> Outputs;
	int32 Timestamp = INDEX_NONE;

	void Reset()
	{
		Outputs.Reset();
	}
};
/* async structs end here */
