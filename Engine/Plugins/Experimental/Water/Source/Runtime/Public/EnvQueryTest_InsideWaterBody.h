// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvQueryTest_InsideWaterBody.generated.h"

UCLASS()
class UEnvQueryTest_InsideWaterBody : public UEnvQueryTest
{
	GENERATED_UCLASS_BODY()

	/** Whether waves should be taken into account in the query. */
	UPROPERTY(EditDefaultsOnly, Category = Water)
	bool bIncludeWaves = false;

	/** Use the simple (faster) version of waves computation. */
	UPROPERTY(EditDefaultsOnly, Category = Water, meta = (EditCondition = "!bIncludeWaves"))
	bool bSimpleWaves = true;

	/** Whether water body exclusion volumes should be taken into account in the query. */
	UPROPERTY(EditDefaultsOnly, Category = Water)
	bool bIgnoreExclusionVolumes = false;

	virtual void RunTest(FEnvQueryInstance& QueryInstance) const override;

	virtual FText GetDescriptionDetails() const override;
};
