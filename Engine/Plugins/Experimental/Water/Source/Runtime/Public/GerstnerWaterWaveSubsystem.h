// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "GerstnerWaterWaveSubsystem.generated.h"

struct FResourceSizeEx;

class FWaterViewExtension;

/** UGerstnerWaterWaveSubsystem manages all UGerstnerWaterWaves objects, regardless of which world they belong to (it's a UEngineSubsystem) */
UCLASS(MinimalAPI)
class UGerstnerWaterWaveSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	UGerstnerWaterWaveSubsystem();

	// UEngineSubsystem implementation
	WATER_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	WATER_API virtual void Deinitialize() override;

	WATER_API void Register(FWaterViewExtension* ViewExtension);
	WATER_API void Unregister(FWaterViewExtension* ViewExtension);

	void RebuildGPUData() { bRebuildGPUData = true; }

	//~ Begin UObject Interface.	
	WATER_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject Interface

private:
	void BeginFrameCallback();

private:
	TArray<FWaterViewExtension*> WaterViewExtensions;
	bool bRebuildGPUData = true;
};
