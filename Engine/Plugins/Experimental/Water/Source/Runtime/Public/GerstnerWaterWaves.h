// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterWaves.h"
#include "GerstnerWaterWaves.generated.h"

/** Raw wave parameters for one gerstner wave */
USTRUCT(BlueprintType)
struct FGerstnerWave
{
	GENERATED_BODY()

public:
	/** Manually call Recompute to recompute FGerstnerWave's internals after one of its properties has changed : */
	void Recompute();

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Wave)
	float WaveLength = 25.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Wave)
	float Amplitude = 10.0f;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Wave)
	float Steepness = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Wave)
	FVector Direction = FVector::ForwardVector;

	UPROPERTY()
	FVector2D WaveVector = FVector2D::ZeroVector;

	UPROPERTY()
	float WaveSpeed = 0.0f;

	UPROPERTY()
	float WKA = 0.0f;

	UPROPERTY()
	float Q = 0.0f;

	UPROPERTY()
	float PhaseOffset = 0.0f;
};

USTRUCT(BlueprintType)
struct FGerstnerWaveOctave
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Octave)
	int32 NumWaves = 16;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Octave)
	float AmplitudeScale = 1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Octave | Direction")
	float MainDirection = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Octave | Direction")
	float SpreadAngle = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Octave | Direction")
	bool bUniformSpread = false;
};

UENUM()
enum class EWaveSpectrumType : uint8
{
	Phillips UMETA(DisplayName = "Phillips"),
	PiersonMoskowitz UMETA(DisplayName = "Pierson-Moskowitz"),
	JONSWAP UMETA(DisplayName = "JONSWAP"),
};

/** 
	Base class for the gerstner water wave generation. This can be overridden by either C++ classes or Blueprint classes.
	Simply implement GenerateGerstnerWaves (or GenerateGerstnerWaves_Implementation in C++) to return the set of waves to be used. Waves will automatically be sorted based on wave length.
*/
UCLASS(EditInlineNew, BlueprintType, MinimalAPI, Abstract, Blueprintable)
class UGerstnerWaterWaveGeneratorBase : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintNativeEvent, Category = "Generation")
	void GenerateGerstnerWaves(TArray<FGerstnerWave>& OutWaves) const;

	virtual void GenerateGerstnerWaves_Implementation(TArray<FGerstnerWave>& OutWaves) const {}
};

/**
	Default implementation of a gerstner wave generator using a simple custom range based set of parameters to generate waves.
*/
UCLASS(EditInlineNew, BlueprintType, MinimalAPI, NotBlueprintable)
class UGerstnerWaterWaveGeneratorSimple : public UGerstnerWaterWaveGeneratorBase
{
	GENERATED_BODY()

public:
	virtual void GenerateGerstnerWaves_Implementation(TArray<FGerstnerWave>& OutWaves) const override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 1, ClampMin = 1, UIMax = 128, ClampMax = 4096, Category = "Default"))
	int32 NumWaves = 16;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Default")
	int32 Seed = 0;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, meta = (UIMin = 0, ClampMin = 0, Category = "Default"))
	float Randomness = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 0, ClampMin = 0, UIMax = 10000.0, Category = "Wavelengths"))
	float MinWavelength = 521.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 0, ClampMin = 0, UIMax = 10000.0, Category = "Wavelengths"))
	float MaxWavelength = 6000.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 0, ClampMin = 0, UIMax = 100.0, Category = "Wavelengths"))
	float WavelengthFalloff = 2.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 0.0001, ClampMin = 0.0001, UIMax = 1000.0, Category = "Amplitude"))
	float MinAmplitude = 4.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 0.0001, ClampMin = 0.0001, UIMax = 1000.0, Category = "Amplitude"))
	float MaxAmplitude = 80.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 0, ClampMin = 0, UIMax = 100.0, Category = "Amplitude"))
	float AmplitudeFalloff = 2.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Dominant Wind Angle", Category = "Directions", UIMin = -180, ClampMin = -180, UIMax = 180, ClampMax = 180, Units = deg))
	float WindAngleDeg = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Direction Angular Spread", Category = "Directions", UIMin = 0, ClampMin = 0, Units = deg))
	float DirectionAngularSpreadDeg = 1325.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 0, ClampMin = 0, UIMax = 1.0, Category = "Steepness"))
	float SmallWaveSteepness = 0.4f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 0, ClampMin = 0, UIMax = 1.0, Category = "Steepness"))
	float LargeWaveSteepness = 0.2f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 0, ClampMin = 0, UIMax = 100.0, Category = "Steepness"))
	float SteepnessFalloff = 1.0f;
};

/**
	Default implementation of a gerstner wave generator using known wave spectra from oceanology. 
	Edited using octaves, where each octave is a set of waves with 2x longer wave length than the previous octave
*/
UCLASS(EditInlineNew, BlueprintType, MinimalAPI, NotBlueprintable, HideDropdown)
class UGerstnerWaterWaveGeneratorSpectrum : public UGerstnerWaterWaveGeneratorBase
{
	GENERATED_BODY()

public:
	virtual void GenerateGerstnerWaves_Implementation(TArray<FGerstnerWave>& OutWaves) const override;

	UPROPERTY(EditAnywhere, Category = "Wave Parameters")
	EWaveSpectrumType SpectrumType = EWaveSpectrumType::Phillips;

	UPROPERTY(EditAnywhere, Category = "Wave Parameters")
	TArray<FGerstnerWaveOctave> Octaves;
};

/**
	Waves implementation based off Gerstner waves. 
	They provide a decent look, are relatively cheap to evaluate and have the advantage of being only dependent on the time variable, which makes them perfect for online simulations. 
*/
UCLASS(EditInlineNew, BlueprintType, MinimalAPI)
class UGerstnerWaterWaves : public UWaterWaves
{
	GENERATED_BODY()

	friend class UGerstnerWaterWaveSubsystem;

public:
	UGerstnerWaterWaves();
	
	/** Returns the maximum wave height that can be reached by those waves */
	virtual float GetMaxWaveHeight() const override { return MaxWaveHeight; }

	/** Computes the raw wave perturbation of the water height/normal */
	WATER_API virtual float GetWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime, FVector& OutNormal) const override;

	/** Computes the raw wave perturbation of the water height only (simple version : faster computation) */
	WATER_API virtual float GetSimpleWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime) const override;

	/** Computes the attenuation factor to apply to the raw wave perturbation. Attenuates : normal/wave height/max wave height. 
	Should match the GPU version (ComputeWaveDepthAttenuationFactor) */
	WATER_API virtual float GetWaveAttenuationFactor(const FVector& InPosition, float InWaterDepth, float InTargetWaveMaskDepth) const override;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Instanced, Category = Waves)
	TObjectPtr<UGerstnerWaterWaveGeneratorBase> GerstnerWaveGenerator;

protected:
	UPROPERTY(BlueprintReadOnly, Category = "Wave")
	TArray<FGerstnerWave> GerstnerWaves;
	
	UPROPERTY(BlueprintReadOnly, Category = "Wave")
	float MaxWaveHeight;

public:
	const TArray<FGerstnerWave>& GetGerstnerWaves() const { return GerstnerWaves; }

	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/** Call RecomputeWaves whenever wave data changes, so that all cached data can be recomputed (do not call OnPostLoad... can call BP script internally) */
	WATER_API void RecomputeWaves(bool bAllowBPScript);

};
