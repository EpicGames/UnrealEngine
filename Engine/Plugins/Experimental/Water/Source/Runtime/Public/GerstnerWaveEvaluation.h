// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FGerstnerWave;

/**
 * Shared functions for Gerstner wave evaluation.
 * Called by UGerstnerWaterWaves, FSolverSafeWaterBodyData, and FBuoyancyGerstnerWaveData.
 *
 * When bBlendLWCTiles is true, wave sin/cos values are blended near LWC tile
 * boundaries to avoid visible seams. Only UGerstnerWaterWaves passes true (for
 * rendering-matching queries on the game thread); physics-thread callers pass
 * false (the default).
 */
namespace GerstnerWaveEvaluation
{
	/** Per-wave offset (XYZ displacement + normal contribution + 1D horizontal offset). */
	WATER_API FVector GetWaveOffsetAtPosition(const FGerstnerWave& InWaveParams, 
		const FVector& InPosition, float InTime, FVector& OutNormal, float& OutOffset1D, 
		bool bBlendLWCTiles = false);

	/** Per-wave simple offset (height only, no normal). */
	WATER_API float GetSimpleWaveOffsetAtPosition(const FGerstnerWave& InWaveParams, 
		const FVector& InPosition, float InTime, bool bBlendLWCTiles = false);

	/** Aggregate wave height over array (steepness two-sample lerp for Q != 0). */
	WATER_API float GetWaveHeightAtPosition(TArrayView<const FGerstnerWave> WaveParams, 
		const FVector& InPosition, float InTime, FVector& OutNormal, bool bBlendLWCTiles = false);

	/** Aggregate simple wave height over array (height only). */
	WATER_API float GetSimpleWaveHeightAtPosition(TArrayView<const FGerstnerWave> WaveParams, 
		const FVector& InPosition, float InTime, bool bBlendLWCTiles = false);

	/** Pure attenuation formula (shore dampening). */
	WATER_API float GetWaveAttenuationFactor(float InWaterDepth, float InTargetWaveMaskDepth);
}
