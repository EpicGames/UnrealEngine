// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "LakeCollisionComponent.generated.h"

#define UE_API WATER_API

UCLASS(MinimalAPI, ClassGroup = (Custom))
class ULakeCollisionComponent : public UPrimitiveComponent
{
	friend class FLakeCollisionSceneProxy;

	GENERATED_UCLASS_BODY()
public:
	UE_API void UpdateCollision(FVector InBoxExtent, bool bSplinePointsChanged);
	
	virtual bool IsZeroExtent() const override { return BoxExtent.IsZero(); }
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const;
	UE_API virtual UBodySetup* GetBodySetup() override;

#if UE_ENABLE_DEBUG_DRAWING
	// The scene proxy is only for debug purposes :
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
#endif // UE_ENABLE_DEBUG_DRAWING

	/** Collects custom navigable geometry of component.
    *   Substract the MaxWaveHeight to the Lake collision so nav mesh geometry is exported a ground level
	*	@return true if regular navigable geometry exporting should be run as well */
	UE_API virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;

protected:
	UE_API void UpdateBodySetup();
	UE_API void CreateLakeBodySetupIfNeeded();
private:
	UPROPERTY(NonPIEDuplicateTransient)
	TObjectPtr<class UBodySetup> CachedBodySetup;

	UPROPERTY()
	FVector BoxExtent;
};

#undef UE_API
