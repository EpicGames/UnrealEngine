// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceWater.generated.h"

#define UE_API WATER_API

class UWaterBodyComponent;

UCLASS(MinimalAPI, EditInlineNew, Category = "Water", meta = (DisplayName = "Water"))
class UNiagaraDataInterfaceWater : public UNiagaraDataInterface
{
	GENERATED_BODY()

public:
	UE_API virtual void PostInitProperties() override;
	virtual bool CanBeInCluster() const override { return false; }	// Note: Due to BP functionality we can change a UObject property on this DI we can not put into a cluster

	/** UNiagaraDataInterface interface */
	UE_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	UE_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	UE_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	UE_API virtual int32 PerInstanceDataSize() const override;
	UE_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	UE_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	UE_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return Target == ENiagaraSimTarget::CPUSim; }

#if WITH_EDITORONLY_DATA
	UE_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif

#if WITH_NIAGARA_DEBUGGER
	UE_API virtual void DrawDebugHud(FNDIDrawDebugHudContext& DebugHudContext) const override;
#endif

	UE_API void GetWaterDataAtPoint(FVectorVMExternalFunctionContext& Context);

	UE_API void GetWaveParamLookupTableOffset(FVectorVMExternalFunctionContext& Context);

	/** Sets the current water body to be used by this data interface */
	UE_API void SetWaterBodyComponent(UWaterBodyComponent* InWaterBodyComponent);

protected:
#if WITH_EDITORONLY_DATA
	UE_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

private:
	UPROPERTY(EditAnywhere, Category = "Water")
	bool bFindWaterBodyOnSpawn = false;

	/** When enabled the owning system instance position will be used to sample the depth of the water. */
	UPROPERTY(EditAnywhere, Category = "Water")
	bool bEvaluateSystemDepth = true;

	/** If bEvaluateSystemDepth is enabled the depth will be updated each frame. */
	UPROPERTY(EditAnywhere, Category = "Water", meta = (EditCondition = "bEvaluateSystemDepth"))
	bool bEvaluateSystemDepthPerFrame = true;

	UPROPERTY(EditAnywhere, Category = "Water", meta = (DisplayName = "Source Actor Or Component", AllowedClasses = "/Script/Engine.WaterBodyComponent,/Script/Engine.Actor"))
	TSoftObjectPtr<UObject> SourceBodyComponent;

	uint32 SourceBodyChangeId = 0;
};

#undef UE_API
