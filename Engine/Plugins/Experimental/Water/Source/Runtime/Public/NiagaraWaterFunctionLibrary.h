// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "NiagaraWaterFunctionLibrary.generated.h"

class UNiagaraComponent;
class AWaterBody;
class UWaterBodyComponent;

UCLASS(MinimalAPI)
class UNiagaraWaterFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	/** Sets the water body on the Niagra water data interface on a Niagara particle system*/
	UE_DEPRECATED(all, "Use SetWaterBodyComponent instead")
	UFUNCTION(BlueprintCallable, Category = Water, meta = (DisplayName = "Set Water Body"), meta = (DeprecatedFunction, DeprecationMessage="Use SetWaterBodyComponent instead"))
	static void SetWaterBody(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") const FString & OverrideName, AWaterBody* WaterBody) {}

	/** Sets the water body component on the Niagra water data interface on a Niagara particle system*/
	UFUNCTION(BlueprintCallable, Category = Water)
	static void SetWaterBodyComponent(UNiagaraComponent* NiagaraSystem, UPARAM(DisplayName = "Parameter Name") const FString& OverrideName, UWaterBodyComponent* WaterBodyComponent);
};
