// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/BoxComponent.h"
#include "OceanCollisionComponent.generated.h"

#define UE_API WATER_API

struct FKConvexElem;

UCLASS(MinimalAPI, ClassGroup = (Custom))
class UOceanCollisionComponent : public UPrimitiveComponent
{
	friend class FOceanCollisionSceneProxy;

	GENERATED_UCLASS_BODY()
public:

	UE_API void InitializeFromConvexElements(const TArray<FKConvexElem>& ConvexElements);

	virtual bool IsZeroExtent() const override { return BoundingBox.GetExtent().IsZero(); }
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const;
	UE_API virtual UBodySetup* GetBodySetup() override;

#if UE_ENABLE_DEBUG_DRAWING
	// The scene proxy is only for debug purposes :
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
#endif // UE_ENABLE_DEBUG_DRAWING

	/** Collects custom navigable geometry of component.
	*   Substract the MaxWaveHeight to the Ocean collision so nav mesh geometry is exported a ground level
	*	@return true if regular navigable geometry exporting should be run as well */
	UE_API virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;

protected:
	UE_API void UpdateBodySetup(const TArray<FKConvexElem>& ConvexElements);
	UE_API void CreateOceanBodySetupIfNeeded();
private:
	UPROPERTY(NonPIEDuplicateTransient)
	TObjectPtr<class UBodySetup> CachedBodySetup;

	FBox BoundingBox;
};



UCLASS(MinimalAPI)
class UOceanBoxCollisionComponent : public UBoxComponent
{
	GENERATED_UCLASS_BODY()
public:
	UE_API virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;
};

#undef UE_API
