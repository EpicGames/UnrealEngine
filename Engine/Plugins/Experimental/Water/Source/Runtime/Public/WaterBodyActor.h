// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TerrainCarvingSettings.h"
#include "WaterBrushActorInterface.h"
#include "WaterBodyComponent.h"
#include "WaterBodyActor.generated.h"

#define UE_API WATER_API

class UWaterSplineComponent;
class UWaterBodyInfoMeshComponent;
class AWaterBodyIsland;
class AWaterBodyExclusionVolume;
class ALandscapeProxy;
class UMaterialInstanceDynamic;

class UWaterBodyComponent;
class UWaterBodyRiverComponent;
class UWaterBodyLakeComponent;
class UWaterBodyOceanComponent;
class UWaterBodyCustomComponent;

// ----------------------------------------------------------------------------------

// For internal use.
UCLASS(MinimalAPI, Abstract, Deprecated, Within = WaterBody)
class UDEPRECATED_WaterBodyGenerator : public UObject
{
	GENERATED_UCLASS_BODY()
};

// ----------------------------------------------------------------------------------

/**
 * Base class for all water body actors.
 *
 * WaterBodyActors provide a spline-based workflow to create lakes, rivers, and oceans which automatically create meshes,
 * carve landscapes, and support physics interactions.
 *
 * To create a new water body this class must be derived (native or blueprint) and have the `WaterBodyType` property changed to the specific water type.
 * The new class will automatically have a corresponding UWaterBodyComponent specific to that water body type.
 * The component class for each water body type can be defined in the Editor Settings.
 */
UCLASS(MinimalAPI, Blueprintable, config = Engine, Abstract, HideCategories = (Tags, Activation, Cooking, Replication, Input, Actor, AssetUserData))
class AWaterBody : public AActor, public IWaterBrushActorInterface
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin IWaterBrushActorInterface interface
	virtual bool AffectsLandscape() const override { return WaterBodyComponent->AffectsLandscape(); }
	virtual bool AffectsWaterMesh() const override { return WaterBodyComponent->AffectsWaterMesh(); }
	virtual bool CanEverAffectWaterMesh() const override { return WaterBodyComponent->CanEverAffectWaterMesh(); }
	virtual bool CanEverAffectWaterInfo() const override { return WaterBodyComponent->CanEverAffectWaterInfo(); }

#if WITH_EDITOR
	virtual const FWaterCurveSettings& GetWaterCurveSettings() const override { return WaterBodyComponent->GetWaterCurveSettings(); }
	virtual const FWaterBodyHeightmapSettings& GetWaterHeightmapSettings() const override { return WaterBodyComponent->GetWaterHeightmapSettings(); }
	virtual const TMap<FName, FWaterBodyWeightmapSettings>& GetLayerWeightmapSettings() const override { return WaterBodyComponent->GetLayerWeightmapSettings(); }
	virtual ETextureRenderTargetFormat GetBrushRenderTargetFormat() const override { return WaterBodyComponent->GetBrushRenderTargetFormat(); }
	virtual void GetBrushRenderDependencies(TSet<UObject*>& OutDependencies) const override { return WaterBodyComponent->GetBrushRenderDependencies(OutDependencies); }
	virtual TArray<UPrimitiveComponent*> GetBrushRenderableComponents() const override { return WaterBodyComponent->GetBrushRenderableComponents(); }
#endif //WITH_EDITOR
	//~ End IWaterBrushActorInterface interface

	/** Actor Interface */
	UE_API virtual void NotifyActorBeginOverlap(AActor* OtherActor) override;
	UE_API virtual void NotifyActorEndOverlap(AActor* OtherActor) override;
	UE_API virtual void PreRegisterAllComponents() override;
	UE_API virtual void UnregisterAllComponents(bool bForReregister = false) override;
	UE_API virtual void PreInitializeComponents() override;
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostRegisterAllComponents() override;
	UE_API virtual bool IsHLODRelevant() const override;
	UE_API virtual bool IsComponentRelevantForNavigation(UActorComponent* Component) const override;

#if WITH_EDITOR
	UE_API virtual void SetActorHiddenInGame(bool bNewHidden) override;
	UE_API virtual void SetIsTemporarilyHiddenInEditor(bool bIsHidden) override;
	UE_API virtual bool SetIsHiddenEdLayer(bool bIsHiddenEdLayer) override;
	UE_API virtual void GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const override;

	UE_API virtual void PostActorCreated() override;

	UE_API virtual void PreDuplicateFromRoot(FObjectDuplicationParameters& DupParams) override;

	UE_API virtual void PostEditMove(bool bFinished) override;
	UE_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
#endif // WITH_EDITOR
	
	/** Returns the type of body */
	UFUNCTION(BlueprintCallable, Category=Water)
	virtual EWaterBodyType GetWaterBodyType() const { return IsTemplate() ? WaterBodyType : GetClass()->GetDefaultObject<AWaterBody>()->WaterBodyType; }
	
	/** Returns water spline component */
	UFUNCTION(BlueprintCallable, Category=Water)
	UWaterSplineComponent* GetWaterSpline() const { return SplineComp; }

	UWaterSplineMetadata* GetWaterSplineMetadata() { return WaterSplineMetadata; }

	const UWaterSplineMetadata* GetWaterSplineMetadata() const { return WaterSplineMetadata; }

	UFUNCTION(BlueprintCallable, Category = Wave)
	UE_API void SetWaterWaves(UWaterWavesBase* InWaterWaves);
	UWaterWavesBase* GetWaterWaves() const { return WaterWaves; }

	/** Returns the water body component */
	UFUNCTION(BlueprintCallable, Category=Water)
	UWaterBodyComponent* GetWaterBodyComponent() const { return WaterBodyComponent; }

protected:
	/** Initializes the water body by creating the respective component for this water body type. */
	UE_API virtual void InitializeBody();

	UE_API virtual void DeprecateData();

	/** The spline data attached to this water type. */
	UPROPERTY(Category = Water, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UWaterSplineComponent> SplineComp;

	UPROPERTY(Instanced)
	TObjectPtr<UWaterSplineMetadata> WaterSplineMetadata;

	UPROPERTY(Category = Water, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Water,Wave,Rendering,Terrain,Navigation,Physics,Collision,Debug", AllowPrivateAccess = "true"))
	TObjectPtr<UWaterBodyComponent> WaterBodyComponent;

	/** Unique Id for accessing (wave, ... ) data in GPU buffers */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional, BlueprintReadOnly, Category = Water, meta = (AllowPrivateAccess = "true"))
	int32 WaterBodyIndex = INDEX_NONE;
	
	UPROPERTY(Category = Water, EditDefaultsOnly, meta = (AllowPrivateAccess = "true"))
	EWaterBodyType WaterBodyType;

	// #todo_water: This should be moved to the component when component subobjects are supported
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Instanced, Category = Wave, DisplayName = "Waves Source", meta = (Tooltip = ""))
	TObjectPtr<UWaterWavesBase> WaterWaves = nullptr;

	UPROPERTY(TextExportTransient, NonPIEDuplicateTransient)
	TObjectPtr<UWaterBodyInfoMeshComponent> WaterInfoMeshComponent;

	UPROPERTY(TextExportTransient, NonPIEDuplicateTransient)
	TObjectPtr<UWaterBodyInfoMeshComponent> DilatedWaterInfoMeshComponent;

	UPROPERTY(TextExportTransient, NonPIEDuplicateTransient)
	TArray<TObjectPtr<UWaterBodyStaticMeshComponent>> WaterBodyStaticMeshComponents;

	UE_API void SetWaterWavesInternal(UWaterWavesBase* InWaterWaves);

	const TArray<TObjectPtr<UWaterBodyStaticMeshComponent>>& GetWaterBodyStaticMeshComponents() const { return WaterBodyStaticMeshComponents; }

	/** Removes all invalid references in the WaterBodyStaticMeshComponents list. */
	UE_API void CleanupInvalidStaticMeshComponents();

	/** Sets up a new list of water body static mesh components. */
	UE_API void SetWaterBodyStaticMeshComponents(TArrayView<TObjectPtr<UWaterBodyStaticMeshComponent>> NewComponentList, TConstArrayView<TObjectPtr<UWaterBodyStaticMeshComponent>> ComponentsToUnregister = {});

	UPROPERTY(EditDefaultsOnly, Config, Category = Water,  meta = (MetaClass = "/Script/Water.WaterBodyRiverComponent"))
	TSubclassOf<UWaterBodyRiverComponent> WaterBodyRiverComponentClass;

	UPROPERTY(EditDefaultsOnly, Config, Category = Water,  meta = (MetaClass = "/Script/Water.WaterBodyLakeComponent"))
	TSubclassOf<UWaterBodyLakeComponent> WaterBodyLakeComponentClass;

	UPROPERTY(EditDefaultsOnly, Config, Category = Water,  meta = (MetaClass = "/Script/Water.WaterBodyOceanComponent"))
	TSubclassOf<UWaterBodyOceanComponent> WaterBodyOceanComponentClass;

	UPROPERTY(EditDefaultsOnly, Config, Category = Water,  meta = (MetaClass = "/Script/Water.WaterBodyCustomComponent"))
	TSubclassOf<UWaterBodyCustomComponent> WaterBodyCustomComponentClass;

// ----------------------------------------------------------------------------------
// Deprecated

#pragma region Deprecated
public:
	UE_DEPRECATED(all, "Moved to WaterBodyComponent")
	UFUNCTION(BlueprintCallable, Category = Rendering, meta = (DeprecatedFunction))
	virtual UMaterialInstanceDynamic* GetRiverToLakeTransitionMaterialInstance() final { return WaterBodyComponent->GetRiverToLakeTransitionMaterialInstance(); }

	UE_DEPRECATED(all, "Moved to WaterBodyComponent")
	UFUNCTION(BlueprintCallable, Category = Rendering, meta = (DeprecatedFunction))
	virtual UMaterialInstanceDynamic* GetRiverToOceanTransitionMaterialInstance() final { return WaterBodyComponent->GetRiverToOceanTransitionMaterialInstance(); }

	UE_DEPRECATED(all, "Moved to WaterBodyComponent")
	UFUNCTION(BlueprintCallable, Category = Rendering, meta = (DeprecatedFunction))
	void SetWaterMaterial(UMaterialInterface* InMaterial) { WaterBodyComponent->SetWaterMaterial(InMaterial); }

	UE_DEPRECATED(all, "Moved to WaterBodyComponent")
	UFUNCTION(BlueprintCallable, Category = Rendering, meta = (DeprecatedFunction))
	UMaterialInstanceDynamic* GetWaterMaterialInstance() { return WaterBodyComponent->GetWaterMaterialInstance(); }

	UE_DEPRECATED(all, "Moved to WaterBodyComponent")
	UFUNCTION(BlueprintCallable, Category = WaterBody, meta = (DeprecatedFunction))
	virtual float GetWaterVelocityAtSplineInputKey(float InKey) const { return WaterBodyComponent->GetWaterVelocityAtSplineInputKey(InKey); }

	UE_DEPRECATED(all, "Moved to WaterBodyComponent")
	UFUNCTION(BlueprintCallable, Category = WaterBody, meta = (DeprecatedFunction))
	virtual FVector GetWaterVelocityVectorAtSplineInputKey(float InKey) const { return WaterBodyComponent->GetWaterVelocityVectorAtSplineInputKey(InKey); }

	UE_DEPRECATED(all, "Moved to WaterBodyComponent")
	UFUNCTION(BlueprintCallable, Category = WaterBody, meta = (DeprecatedFunction))
	virtual float GetAudioIntensityAtSplineInputKey(float InKey) const { return WaterBodyComponent->GetAudioIntensityAtSplineInputKey(InKey); }

	UE_DEPRECATED(all, "Moved to WaterBodyComponent")
	UFUNCTION(BlueprintCallable, Category = Water, meta = (DeprecatedFunction))
	TArray<AWaterBodyIsland*> GetIslands() const { return WaterBodyComponent->GetIslands(); }

	UE_DEPRECATED(all, "Moved to WaterBodyComponent")
	UFUNCTION(BlueprintCallable, Category = Water, meta = (DeprecatedFunction))
	TArray<AWaterBodyExclusionVolume*> GetExclusionVolumes() const { return WaterBodyComponent->GetExclusionVolumes(); }

	UE_DEPRECATED(all, "Moved to WaterBodyComponent")
	UFUNCTION(BlueprintCallable, Category=Water, meta = (DeprecatedFunction))
	void OnWaterBodyChanged(bool bShapeOrPositionChanged, bool bWeightmapSettingsChanged = false) {}

protected:
	friend class UWaterBodyComponent;
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UPhysicalMaterial> PhysicalMaterial_DEPRECATED;

	UPROPERTY()
	float TargetWaveMaskDepth_DEPRECATED;

	UPROPERTY()
	float MaxWaveHeightOffset_DEPRECATED = 0.f;

	UPROPERTY()
	bool bFillCollisionUnderWaterBodiesForNavmesh_DEPRECATED;

	UPROPERTY()
	FUnderwaterPostProcessSettings UnderwaterPostProcessSettings_DEPRECATED;

	UPROPERTY()
	FWaterCurveSettings CurveSettings_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> WaterMaterial_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> UnderwaterPostProcessMaterial_DEPRECATED;

	UPROPERTY()
	FLandmassTerrainCarvingSettings TerrainCarvingSettings_DEPRECATED;

	UPROPERTY()
	FWaterBodyHeightmapSettings WaterHeightmapSettings_DEPRECATED;

	UPROPERTY()
	TMap<FName, FWaterBodyWeightmapSettings> LayerWeightmapSettings_DEPRECATED;

	UPROPERTY()
	bool bAffectsLandscape_DEPRECATED;
	
	UPROPERTY()
	bool bGenerateCollisions_DEPRECATED = true;

	UPROPERTY()
	bool bOverrideWaterMesh_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UStaticMesh> WaterMeshOverride_DEPRECATED;

	UPROPERTY()
	int32 OverlapMaterialPriority_DEPRECATED = 0;

	UPROPERTY()
	FName CollisionProfileName_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> WaterMID_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> UnderwaterPostProcessMID_DEPRECATED;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	TArray<TLazyObjectPtr<AWaterBodyIsland>> Islands_DEPRECATED;

	UPROPERTY()
	TArray<TLazyObjectPtr<AWaterBodyExclusionVolume>> ExclusionVolumes_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	bool bCanAffectNavigation_DEPRECATED;

	UPROPERTY()
	TSubclassOf<UNavAreaBase> WaterNavAreaClass_DEPRECATED;

	UPROPERTY()
	float ShapeDilation_DEPRECATED = 4096.0f;
#endif // WITH_EDITORONLY_DATA

#pragma endregion // Deprecated
};

#undef UE_API
