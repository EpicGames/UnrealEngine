// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/BlendableInterface.h"
#include "Engine/Scene.h"
#include "WaterBodyWeightmapSettings.h"
#include "WaterBodyHeightmapSettings.h"
#include "WaterCurveSettings.h"
#include "WaterBodyStaticMeshSettings.h"
#include "WaterSplineMetadata.h"
#include "BakedShallowWaterSimulationComponent.h"
#include "WaterZoneActor.h"
#include "WaterBodyTypes.h"
#include "Templates/ValueOrError.h"

class AWaterBody;
class UStaticMesh;
class UWaterWavesBase;
enum ETextureRenderTargetFormat : int;
struct FPostProcessVolumeProperties;

#include "WaterBodyComponent.generated.h"

class UWaterSplineComponent;
struct FOnWaterSplineDataChangedParams;
class UWaterBodyStaticMeshComponent;
class UWaterBodyInfoMeshComponent;
class AWaterBodyIsland;
class AWaterBodyExclusionVolume;
class AWaterZone;
class ALandscapeProxy;
class UMaterialInstanceDynamic;
class FTokenizedMessage;
class UNavAreaBase;
namespace UE::Geometry { class FDynamicMesh3; }
struct FMeshDescription;

// ----------------------------------------------------------------------------------

USTRUCT(BlueprintType)
struct FUnderwaterPostProcessSettings
{
	GENERATED_BODY()

	FUnderwaterPostProcessSettings()
		: bEnabled(true)
		, Priority(0)
		, BlendRadius(100.f)
		, BlendWeight(1.0f)
		, UnderwaterPostProcessMaterial_DEPRECATED(nullptr)
	{}
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Rendering)
	bool bEnabled;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Rendering)
	float Priority;

	/** World space radius around the volume that is used for blending (only if not unbound).			*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Rendering, meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "6000.0"))
	float BlendRadius;

	/** 0:no effect, 1:full effect */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Rendering, meta = (UIMin = "0.0", UIMax = "1.0"))
	float BlendWeight;

	/** List of all post-process settings to use when underwater : note : use UnderwaterPostProcessMaterial for setting the actual post process material. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Rendering)
	FPostProcessSettings PostProcessSettings;

	/** This is the parent post process material for the PostProcessSettings */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> UnderwaterPostProcessMaterial_DEPRECATED;
};

// ----------------------------------------------------------------------------------

enum class EWaterBodyStatus : uint8
{
	Valid,
	MissingWaterZone,
	MissingLandscape,
	InvalidWaveData,
};


// ----------------------------------------------------------------------------------

struct FOnWaterBodyChangedParams
{
	FOnWaterBodyChangedParams(const FPropertyChangedEvent& InPropertyChangedEvent = FPropertyChangedEvent(/*InProperty = */nullptr))
		: PropertyChangedEvent(InPropertyChangedEvent)
	{}

	/** Provides some additional context about how the water body data has changed (property, type of change...) */
	FPropertyChangedEvent PropertyChangedEvent;

	/** Indicates that property related to the water body's visual shape has changed */
	bool bShapeOrPositionChanged = false;

	/** Indicates that a property affecting the terrain weightmaps has changed */
	bool bWeightmapSettingsChanged = false;

	/** Indicates user initiated Parameter change */
	bool bUserTriggered = false;
};


// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI, Abstract, HideCategories = (Tags, Activation, Cooking, Physics, Replication, Input, AssetUserData, Mesh))
class UWaterBodyComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

	friend class AWaterBody;
	friend class FWaterBodySceneProxy;
	friend class FWaterBodyMeshBuilder;
public:
	WATER_API virtual bool AffectsLandscape() const;
	WATER_API virtual bool AffectsWaterMesh() const;
	WATER_API virtual bool AffectsWaterInfo() const;
	virtual bool CanEverAffectWaterMesh() const { return true; }
	virtual bool CanEverAffectWaterInfo() const { return true; }

	WATER_API void UpdateAll(const FOnWaterBodyChangedParams& InParams);

#if WITH_EDITOR
	const FWaterCurveSettings& GetWaterCurveSettings() const { return CurveSettings; }
	const FWaterBodyHeightmapSettings& GetWaterHeightmapSettings() const { return WaterHeightmapSettings; }
	const TMap<FName, FWaterBodyWeightmapSettings>& GetLayerWeightmapSettings() const { return LayerWeightmapSettings; }
	WATER_API virtual ETextureRenderTargetFormat GetBrushRenderTargetFormat() const;
	WATER_API virtual void GetBrushRenderDependencies(TSet<UObject*>& OutDependencies) const;
	virtual TArray<UPrimitiveComponent*> GetBrushRenderableComponents() const { return TArray<UPrimitiveComponent*>(); }
	
	WATER_API virtual void UpdateWaterSpriteComponent();

	WATER_API virtual FMeshDescription GetHLODMeshDescription() const;
	WATER_API virtual UMaterialInterface* GetHLODMaterial() const;
	WATER_API virtual void SetHLODMaterial(UMaterialInterface* InMaterial);

	WATER_API void UpdateWaterBodyRenderData();

	UFUNCTION(BlueprintCallable, Category = Water)
	WATER_API void SetWaterBodyStaticMeshEnabled(bool bEnabled);
#endif //WITH_EDITOR

	/** Returns whether the body supports waves */
	WATER_API virtual bool IsWaveSupported() const;

	/** Returns true if there are valid water waves */
	WATER_API bool HasWaves() const;

	/** Returns whether the body is baked (false) at save-time or needs to be dynamically regenerated at runtime (true) and is therefore transient. */
	virtual bool IsBodyDynamic() const { return false; }
	
	/** Returns body's collision components */
	UFUNCTION(BlueprintCallable, Category = Collision)
	virtual TArray<UPrimitiveComponent*> GetCollisionComponents(bool bInOnlyEnabledComponents = true) const { return TArray<UPrimitiveComponent*>(); }

	/** Retrieves the list of primitive components that this water body uses when not being rendered by the water mesh (e.g. the static mesh component used when WaterMeshOverride is specified) */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	virtual TArray<UPrimitiveComponent*> GetStandardRenderableComponents() const { return {}; }

	/** Returns the body's collision component bounds */
	WATER_API virtual FBox GetCollisionComponentBounds() const;

	/** Returns the type of body */
	virtual EWaterBodyType GetWaterBodyType() const PURE_VIRTUAL(UWaterBodyComponent::GetWaterBodyType, return EWaterBodyType::Transition; )

	/** Returns collision half-extents */
	virtual FVector GetCollisionExtents() const { return FVector::ZeroVector; }

	/** Returns the physical material of this water body. */
	WATER_API UPhysicalMaterial* GetPhysicalMaterial() const;

	/** Sets an additional water height (For internal use. Please use AWaterBodyOcean instead.) */
	virtual void SetHeightOffset(float InHeightOffset) { check(false); }

	/** Returns the additional water height added to the body (For internal use. Please use AWaterBodyOcean instead.) */
	virtual float GetHeightOffset() const { return 0.f; }

	/** Sets a static mesh to use as a replacement for the water mesh (for water bodies that are being rendered by the water mesh) */
	WATER_API void SetWaterMeshOverride(UStaticMesh* InMesh);

	/** Returns River to lake transition material instance (For internal use. Please use AWaterBodyRiver instead.) */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	virtual UMaterialInstanceDynamic* GetRiverToLakeTransitionMaterialInstance() { return nullptr; }

	/** Returns River to ocean transition material instance (For internal use. Please use AWaterBodyRiver instead.) */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	virtual UMaterialInstanceDynamic* GetRiverToOceanTransitionMaterialInstance() { return nullptr; }

	/** Returns the WaterBodyActor who owns this component */
	UFUNCTION(BlueprintCallable, Category = Water)
	WATER_API AWaterBody* GetWaterBodyActor() const;
	
	/** Returns water spline component */
	UFUNCTION(BlueprintCallable, Category = Water)
	WATER_API UWaterSplineComponent* GetWaterSpline() const;

	UFUNCTION(BlueprintCallable, Category = Water)
	WATER_API UWaterWavesBase* GetWaterWaves() const;

	/** Returns the unique id of this water body for accessing data in GPU buffers */
	int32 GetWaterBodyIndex() const { return WaterBodyIndex; }
	
	/** Returns water mesh override */
	UStaticMesh* GetWaterMeshOverride() const { return WaterMeshOverride; }

	/** Returns water material */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	UMaterialInterface* GetWaterMaterial() const { return WaterMaterial; }

	/** Returns river to lake transition water material */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	virtual UMaterialInterface* GetRiverToLakeTransitionMaterial() const { return nullptr; }

	/** Returns river to ocean transition water material */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	virtual UMaterialInterface* GetRiverToOceanTransitionMaterial() const { return nullptr; }

	/** Returns material used to render the water as a static mesh */
	UMaterialInterface* GetWaterStaticMeshMaterial() const { return WaterStaticMeshMaterial; }

	/** Sets water material */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	WATER_API void SetWaterMaterial(UMaterialInterface* InMaterial);

	/** Sets the material used to draw the Water Info Texture for this water body */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	WATER_API void SetWaterInfoMaterial(UMaterialInterface* InMaterial);

	/** Sets water static mesh material */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	WATER_API void SetWaterStaticMeshMaterial(UMaterialInterface* InMaterial);

	/** Returns water MID */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	WATER_API UMaterialInstanceDynamic* GetWaterMaterialInstance();

	/** Returns water static mesh MID */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	WATER_API UMaterialInstanceDynamic* GetWaterStaticMeshMaterialInstance();

	/** Returns water LOD MID */
	UE_DEPRECATED(all, "GetWaterLODMaterialInstance has been renamed to GetWaterStaticMeshMaterialInstance.")
	UFUNCTION(BlueprintCallable, Category = Rendering, meta=(DeprecationMessage="GetWaterLODMaterialInstance has been renamed to GetWaterStaticMeshMaterialInstance"))
	UMaterialInstanceDynamic* GetWaterLODMaterialInstance() { return GetWaterStaticMeshMaterialInstance(); };

	/** Returns under water post process MID */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	WATER_API UMaterialInstanceDynamic* GetUnderwaterPostProcessMaterialInstance();
	
	/** Returns water info MID */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	WATER_API UMaterialInstanceDynamic* GetWaterInfoMaterialInstance();

	/** Sets under water post process material */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	WATER_API void SetUnderwaterPostProcessMaterial(UMaterialInterface* InMaterial);

	UFUNCTION(BlueprintCallable, Category = Rendering)
	WATER_API void SetWaterAndUnderWaterPostProcessMaterial(UMaterialInterface* InWaterMaterial, UMaterialInterface* InUnderWaterPostProcessMaterial);

	/** Returns water spline metadata */
	UWaterSplineMetadata* GetWaterSplineMetadata() { return WaterSplineMetadata; }

	/** Returns water spline metadata */
	const UWaterSplineMetadata* GetWaterSplineMetadata() const { return WaterSplineMetadata; }

	/** Is this water body rendered with the WaterMeshComponent, with the quadtree-based water renderer? */
	WATER_API bool ShouldGenerateWaterMeshTile() const;

	/** Returns nav collision offset */
	FVector GetWaterNavCollisionOffset() const { return FVector(0.0f, 0.0f, -GetMaxWaveHeight()); }

	/** Returns overlap material priority */
	int32 GetOverlapMaterialPriority() const { return OverlapMaterialPriority; }

	/** Returns channel depth */
	float GetChannelDepth() const { return CurveSettings.ChannelDepth; }

	WATER_API void AddIsland(AWaterBodyIsland* Island);
	WATER_API void RemoveIsland(AWaterBodyIsland* Island);
	WATER_API void UpdateIslands();

	/** Adds WaterBody exclusion volume */
	WATER_API void AddExclusionVolume(AWaterBodyExclusionVolume* InExclusionVolume);

	/** Removes WaterBody exclusion volume */
	WATER_API void RemoveExclusionVolume(AWaterBodyExclusionVolume* InExclusionVolume);

	/** Returns post process properties */
	WATER_API FPostProcessVolumeProperties GetPostProcessProperties() const;

	/** Returns the requested water info closest to this world location
	- InWorldLocation: world-space location closest to which the function returns the water info
	- InQueryFlags: flags to indicate which info is to be computed
	- InSplineInputKey: (optional) location on the spline, in case it has already been computed.
	*/
	WATER_API virtual TValueOrError<FWaterBodyQueryResult, EWaterBodyQueryError> TryQueryWaterInfoClosestToWorldLocation(const FVector& InWorldLocation, EWaterBodyQueryFlags InQueryFlags, const TOptional<float>& InSplineInputKey = TOptional<float>()) const;

	UE_DEPRECATED(5.7, "Deprecated in favor of TryQueryWaterInfoClosestToWorldLocation to ensure caller correctly handles the cases where this function fails")
	WATER_API virtual FWaterBodyQueryResult QueryWaterInfoClosestToWorldLocation(const FVector& InWorldLocation, EWaterBodyQueryFlags InQueryFlags, const TOptional<float>& InSplineInputKey = TOptional<float>()) const;

	UFUNCTION(BlueprintCallable, Category = WaterBody)
	WATER_API bool GetWaterSurfaceInfoAtLocation(const FVector& InLocation, FVector& OutWaterSurfaceLocation, FVector& OutWaterSurfaceNormal, FVector& OutWaterVelocity, float& OutWaterDepth, bool bIncludeDepth = false) const;

	/** Spline query helper. It's faster to get the spline key once then query properties using that key, rather than querying repeatedly by location etc. */
	WATER_API float FindInputKeyClosestToWorldLocation(const FVector& WorldLocation) const;

	/*
	 * Spline queries specific to metadata type
	 */
	UFUNCTION(BlueprintCallable, Category = WaterBody)
	WATER_API virtual float GetWaterVelocityAtSplineInputKey(float InKey) const;

	UFUNCTION(BlueprintCallable, Category = WaterBody)
	WATER_API virtual FVector GetWaterVelocityVectorAtSplineInputKey(float InKey) const;

	UFUNCTION(BlueprintCallable, Category = WaterBody)
	WATER_API virtual float GetAudioIntensityAtSplineInputKey(float InKey) const;


	UFUNCTION(BlueprintCallable, Category = WaterBody)
	WATER_API void SetWaterVelocityAtSplineInputKey(float InKey, float InVelocity);

	UFUNCTION(BlueprintCallable, Category = WaterBody)
	WATER_API void SetAudioIntensityAtSplineInputKey(float InKey, float InAudioIntensity);

	/**
	 * Gets the islands that influence this water body
	 */
	UFUNCTION(BlueprintCallable, Category = Water)
	WATER_API TArray<AWaterBodyIsland*> GetIslands() const;

	bool ContainsIsland(TSoftObjectPtr<AWaterBodyIsland> Island) const { return WaterBodyIslands.Contains(Island); }

	/**
	 * Gets the exclusion volume that influence this water body
	 */
	UFUNCTION(BlueprintCallable, Category = Water)
	WATER_API TArray<AWaterBodyExclusionVolume*> GetExclusionVolumes() const;

	bool ContainsExclusionVolume(TSoftObjectPtr<AWaterBodyExclusionVolume> InExclusionVolume) const { return WaterBodyExclusionVolumes.Contains(InExclusionVolume); }

	/** Component interface */
	WATER_API virtual void OnRegister() override;
	WATER_API virtual void OnUnregister() override;
	WATER_API virtual void PostDuplicate(bool bDuplicateForPie) override;

	WATER_API void OnWaterBodyChanged(const FOnWaterBodyChangedParams& InParams);

	/** Fills wave-related information at the given world position and for this water depth.
	 - InPosition : water surface position at which to query the wave information
	 - InWaterDepth : water depth at this location
	 - bSimpleWaves : true for the simple version (faster computation, lesser accuracy, doesn't perturb the normal)
	 - FWaveInfo : input/output : the structure's field must be initialized prior to the call (e.g. InOutWaveInfo.Normal is the unperturbed normal)
	 Returns true if waves are supported, false otherwise. */
	WATER_API bool GetWaveInfoAtPosition(const FVector& InPosition, float InWaterDepth, bool bInSimpleWaves, FWaveInfo& InOutWaveInfo) const;

	/** Returns the max height that this water body's waves can hit. Can be called regardless of whether the water body supports waves or not */
	UFUNCTION(BlueprintCallable, Category = Wave)
	WATER_API float GetMaxWaveHeight() const;

	/** Sets the dynamic parameters needed by the material instance for rendering. Returns true if the operation was successfull */
	WATER_API virtual bool SetDynamicParametersOnMID(UMaterialInstanceDynamic* InMID);

	/** Sets the dynamic parameters needed by the underwater post process material instance for rendering. Returns true if the operation was successfull */
	WATER_API virtual bool SetDynamicParametersOnUnderwaterPostProcessMID(UMaterialInstanceDynamic* InMID);

	/** Sets the dynamic parameters needed by the material instance for rendering the water info texture. Returns true if the operation was successfull */
	WATER_API virtual bool SetDynamicParametersOnWaterInfoMID(UMaterialInstanceDynamic* InMID);

	/** Returns true if the location is within one of this water body's exclusion volumes */
	WATER_API bool IsWorldLocationInExclusionVolume(const FVector& InWorldLocation) const;

	/** Updates the bVisible/bHiddenInGame flags on the component and eventually the child renderable components (e.g. custom water body) */
	WATER_API void UpdateVisibility();

	/** Creates/Destroys/Updates necessary MIDS */
	WATER_API virtual void UpdateMaterialInstances();

	/** Returns the time basis to use in waves computation (must be unique for all water bodies currently, to ensure proper transitions between water tiles) */
	WATER_API virtual float GetWaveReferenceTime() const;

	/** Returns the minimum and maximum Z of the water surface, including waves */
	WATER_API virtual void GetSurfaceMinMaxZ(float& OutMinZ, float& OutMaxZ) const;

	WATER_API virtual ALandscapeProxy* FindLandscape() const;

	/** Returns what can be considered the single water depth of the water surface.
	Only really make sense for EWaterBodyType::Transition water bodies for which we don't really have a way to evaluate depth. */
	WATER_API virtual float GetConstantDepth() const;

	/** Returns what can be considered the single water velocity of the water surface.
	Only really make sense for EWaterBodyType::Transition water bodies for which we don't really have a way to evaluate velocity. */
	WATER_API virtual FVector GetConstantVelocity() const;

	/** Returns what can be considered the single base Z of the water surface.
	Doesn't really make sense for non-flat water bodies like EWaterBodyType::Transition or EWaterBodyType::River but can still be useful when using FixedZ for post-process, for example. */
	WATER_API virtual float GetConstantSurfaceZ() const;

	virtual void Reset() {}

	/** Gets the water zone to which this component belongs */
	WATER_API AWaterZone* GetWaterZone() const;

	/** Override the default behavior of water bodies finding their water zone based on bounds and set a specific water zone to which this water body should register. */
	UFUNCTION(BlueprintCallable, Category=Water)
	WATER_API void SetWaterZoneOverride(const TSoftObjectPtr<AWaterZone>& InWaterZoneOverride);

	/** 
	 * Registers or this water body with corresponding overlapping water zones and unregisters it from any old zones if they are no longer overlapping.
	 *
	 * @param bAllowChangesDuringCook When disabled, this function will not make any changes during cook and just trust that the serialized pointer is correct. 
	 */
	WATER_API void UpdateWaterZones(bool bAllowChangesDuringCook = false);

	/** Set the navigation area class */
	void SetNavAreaClass(TSubclassOf<UNavAreaBase> NewWaterNavAreaClass) { WaterNavAreaClass = NewWaterNavAreaClass; }

	/* Generates the mesh representation of the water body */
	WATER_API virtual bool GenerateWaterBodyMesh(UE::Geometry::FDynamicMesh3& OutMesh, UE::Geometry::FDynamicMesh3* OutDilatedMesh = nullptr) const;

	WATER_API UWaterBodyInfoMeshComponent* GetWaterInfoMeshComponent() const;
	WATER_API UWaterBodyInfoMeshComponent* GetDilatedWaterInfoMeshComponent() const;

	const FWaterBodyStaticMeshSettings& GetWaterBodyStaticMeshSettings() const { return StaticMeshSettings; }
	
	UE_DEPRECATED(all, "Use version which takes FOnWaterBodyChangedParams")
	UFUNCTION(BlueprintCallable, Category=Water, meta=(Deprecated = "5.2"))
	void OnWaterBodyChanged(bool bShapeOrPositionChanged, bool bWeightmapSettingsChanged = false, bool bUserTriggeredChanged = false) {}

	/** Get the baked shallow water simulation for this water body */
	UBakedShallowWaterSimulationComponent* GetBakedShallowWaterSimulation() const { return BakedShallowWaterSim.Get(); }
	
	/** Set the baked shallow water simulation for this water body */
	void SetBakedShallowWaterSimulation(TObjectPtr<UBakedShallowWaterSimulationComponent> BakedSim) { BakedShallowWaterSim = BakedSim; }

	/** Set toggle to use baked simulations if they are valid */
	void SetUseBakedSimulationForQueriesAndPhysics(bool bUseBakedSimulation) { bUseBakedSimForQueriesAndPhysics = bUseBakedSimulation;  }
	
	/** Query for if the baked simulations is valid for use */
	bool UseBakedSimulationForQueriesAndPhysics() const { return bUseBakedSimForQueriesAndPhysics && BakedShallowWaterSim.IsValid() && BakedShallowWaterSim->BakedSimulationData != nullptr && BakedShallowWaterSim->BakedSimulationData->HasValidData(); }

	/** 
	 * Marks the owning water zone for rebuild. 
	 * If bOnlyWithinWaterBodyBounds is set, updates to the water zone that aren't relevant within the bounds of the water body are suppressed.
	 */
	WATER_API void MarkOwningWaterZoneForRebuild(EWaterZoneRebuildFlags InRebuildFlags, bool bInOnlyWithinWaterBodyBounds = true) const;

protected:
	//~ Begin UActorComponent interface.
	WATER_API virtual bool IsHLODRelevant() const override;
	//~ End UActorComponent interface.

	//~ Begin USceneComponent Interface.
	WATER_API virtual void OnVisibilityChanged();
	WATER_API virtual void OnHiddenInGameChanged();
	//~ End USceneComponent Interface.

	/** Returns whether the body has a flat surface or not */
	WATER_API virtual bool IsFlatSurface() const;

	/** Returns whether the body's spline is closed */
	WATER_API virtual bool IsWaterSplineClosedLoop() const;

	/** Returns whether the body support a height offset */
	WATER_API virtual bool IsHeightOffsetSupported() const;

	/** Called every time UpdateAll is called on WaterBody (prior to UpdateWaterBody) */
	WATER_API virtual void BeginUpdateWaterBody();

	/** Updates WaterBody (called 1st with bWithExclusionVolumes = false, then with true */
	WATER_API virtual void UpdateWaterBody(bool bWithExclusionVolumes);

	virtual void OnUpdateBody(bool bWithExclusionVolumes) {}

	/** Called when the WaterBodyActor has had all its components registered. */
	WATER_API virtual void OnPostRegisterAllComponents();

	/** Returns navigation area class */
	TSubclassOf<UNavAreaBase> GetNavAreaClass() const { return WaterNavAreaClass; }

	/** Copies the relevant collision settings from the water body component to the component passed in parameter (useful when using external components for collision) */
	WATER_API void CopySharedCollisionSettingsToComponent(UPrimitiveComponent* InComponent);

	/** Copies the relevant navigation settings from the water body component to the component passed in parameter (useful when using external components for navigation) */
	WATER_API void CopySharedNavigationSettingsToComponent(UPrimitiveComponent* InComponent);

	/** Computes the raw wave perturbation of the water height/normal */
	WATER_API virtual float GetWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime, FVector& OutNormal) const;

	/** Computes the raw wave perturbation of the water height only (simple version : faster computation) */
	WATER_API virtual float GetSimpleWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime) const;

	/** Computes the attenuation factor to apply to the raw wave perturbation. Attenuates : normal/wave height/max wave height. */
	WATER_API virtual float GetWaveAttenuationFactor(const FVector& InPosition, float InWaterDepth) const;

	/** Called by the owning actor when it receives a PostActorCreated callback */
	virtual void OnPostActorCreated() {}

#if WITH_EDITOR
	/** Called by UWaterBodyComponent::PostEditChangeProperty. */
	WATER_API virtual void OnPostEditChangeProperty(FOnWaterBodyChangedParams& InOutOnWaterBodyChangedParams);

	/** Validates this component's data */
	WATER_API virtual void CheckForErrors() override;

	WATER_API virtual TArray<TSharedRef<FTokenizedMessage>> CheckWaterBodyStatus();

	virtual const TCHAR* GetWaterSpriteTextureName() const { return TEXT("/Water/Icons/WaterSprite"); }

	virtual bool IsIconVisible() const { return true; }

	virtual FVector GetWaterSpriteLocation() const { return GetComponentLocation(); }

	WATER_API virtual void OnWaterBodyRenderDataUpdated();

	/** Fixup any invalid transformations made to the water body in the editor that may have been made through the transform gizmo or property changes. */
	WATER_API void FixupEditorTransform();
#endif // WITH_EDITOR

	WATER_API EWaterBodyQueryFlags CheckAndAjustQueryFlags(EWaterBodyQueryFlags InQueryFlags) const;
	WATER_API void UpdateSplineComponent();
	WATER_API void UpdateExclusionVolumes();
	WATER_API bool UpdateWaterHeight();
	WATER_API virtual void CreateOrUpdateWaterMID();
	WATER_API void CreateOrUpdateWaterStaticMeshMID();
	WATER_API void CreateOrUpdateUnderwaterPostProcessMID();
	WATER_API void CreateOrUpdateWaterInfoMID();
	WATER_API void PrepareCurrentPostProcessSettings();
	WATER_API void ApplyNavigationSettings();
	WATER_API void ApplyCollisionSettings();
	WATER_API void RequestGPUWaveDataUpdate();
	WATER_API EObjectFlags GetTransientMIDFlags() const; 
	WATER_API void DeprecateData();

	WATER_API virtual void Serialize(FArchive& Ar) override;
	WATER_API virtual void PostLoad() override;
	WATER_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	WATER_API virtual void OnComponentCollisionSettingsChanged(bool bUpdateOverlaps) override;
	WATER_API virtual void OnGenerateOverlapEventsChanged() override;
	WATER_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

#if WITH_EDITOR
	WATER_API virtual void PreEditUndo() override;
	WATER_API virtual void PostEditUndo() override;
	WATER_API virtual void PostEditImport() override;
	WATER_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	WATER_API void OnWaterSplineDataChanged(const FOnWaterSplineDataChangedParams& InParams);
	WATER_API void RegisterOnUpdateWavesData(UWaterWavesBase* InWaterWaves, bool bRegister);
	WATER_API void OnWavesDataUpdated(UWaterWavesBase* InWaterWaves, EPropertyChangeType::Type InChangeType);
	WATER_API void OnWaterSplineMetadataChanged(const FOnWaterSplineMetadataChangedParams& InParams);
	WATER_API void RegisterOnChangeWaterSplineData(bool bRegister);

	WATER_API void CreateWaterSpriteComponent();

	WATER_API void UpdateWaterInfoMeshComponents();
	WATER_API void UpdateWaterBodyStaticMeshComponents();

	WATER_API virtual TSubclassOf<class UHLODBuilder> GetCustomHLODBuilderClass() const override;
#endif // WITH_EDITOR

public:
	// INavRelevantInterface start
	WATER_API virtual void GetNavigationData(struct FNavigationRelevantData& Data) const override;
	WATER_API virtual FBox GetNavigationBounds() const override;
	WATER_API virtual bool IsNavigationRelevant() const override;
	// INavRelevantInterface end

	/** Public static constants : */
	static WATER_API const FName WaterBodyIndexParamName;
	static WATER_API const FName WaterZoneIndexParamName;
	static WATER_API const FName WaterBodyZOffsetParamName;
	static WATER_API const FName WaterVelocityAndHeightName;
	static WATER_API const FName GlobalOceanHeightName;
	static WATER_API const FName MaxFlowVelocityParamName;

	/** Water depth at which waves start being attenuated. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Wave, DisplayName = "Wave Attenuation Water Depth", meta = (UIMin = 0, ClampMin = 0, UIMax = 10000.0))
	float TargetWaveMaskDepth;

	/** Offset added to the automatically calculated max wave height bounds. Use this in case the automatically calculated max height bounds don't match your waves. This can happen if the water surface is manually altered through World Position Offset or other means.*/
	UPROPERTY(EditAnywhere, Category = Wave)
	float MaxWaveHeightOffset = 0.0f;

	/** Post process settings to apply when the camera goes underwater (only available when collisions are enabled as they are needed to detect if the camera is under water).
	Note: Underwater post process material is setup using UnderwaterPostProcessMaterial. */
	UPROPERTY(Category = Rendering, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta = (EditCondition = "UnderwaterPostProcessMaterial != nullptr", DisplayAfter = "UnderwaterPostProcessMaterial"))
	FUnderwaterPostProcessSettings UnderwaterPostProcessSettings;

	UPROPERTY(Category = Terrain, EditAnywhere, BlueprintReadWrite, meta = (EditCondition = "bAffectsLandscape"))
	FWaterCurveSettings CurveSettings;

	UPROPERTY(Category = Rendering, EditAnywhere, BlueprintReadOnly)
	TObjectPtr<UMaterialInterface> WaterMaterial;

	UPROPERTY(Category = HLOD, EditAnywhere, BlueprintReadOnly, meta = (DisplayName = "Water HLOD Material"))
	TObjectPtr<UMaterialInterface> WaterHLODMaterial;

	UPROPERTY(Category = Rendering, EditAnywhere, BlueprintReadOnly)
	TObjectPtr<UMaterialInterface> WaterStaticMeshMaterial;

	/** Post process material to apply when the camera goes underwater (only available when bGenerateCollisions is true because collisions are needed to detect if it's under water). */
	UPROPERTY(Category = Rendering, EditAnywhere, BlueprintReadOnly, meta = (EditCondition = "bGenerateOverlapEvents", DisplayAfter = "WaterMaterial"))
	TObjectPtr<UMaterialInterface> UnderwaterPostProcessMaterial;

	UPROPERTY(Category = Rendering, EditAnywhere, BlueprintReadOnly, meta = (DisplayAfter = "WaterMaterial"))
	TObjectPtr<UMaterialInterface> WaterInfoMaterial;
	
	UPROPERTY(Category = Terrain, EditAnywhere, BlueprintReadWrite, meta = (EditCondition = "bAffectsLandscape"))
	FWaterBodyHeightmapSettings WaterHeightmapSettings;

	UPROPERTY(Category = Terrain, EditAnywhere, BlueprintReadWrite, meta = (EditCondition = "bAffectsLandscape"))
	TMap<FName, FWaterBodyWeightmapSettings> LayerWeightmapSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Rendering)
	float ShapeDilation = 4096.0f;

	/** The distance above the surface of the water where collision checks should still occur. Useful if the post process effect is not activating under really high waves. */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Collision)
	float CollisionHeightOffset = 0.f;

	/** If enabled, landscape will be deformed based on this water body placed on top of it and landscape height will be considered when determining water depth at runtime */
	UPROPERTY(Category = Terrain, EditAnywhere, BlueprintReadWrite)
	bool bAffectsLandscape;

	UPROPERTY(Category = Water, EditAnywhere)
	FWaterBodyStaticMeshSettings StaticMeshSettings;

protected:

	/** Unique Id for accessing (wave, ... ) data in GPU buffers */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional, VisibleAnywhere, BlueprintReadOnly, Category = Water)
	int32 WaterBodyIndex = INDEX_NONE;

	UPROPERTY(Category = Rendering, EditAnywhere, BlueprintReadWrite, Getter, Setter)
	TObjectPtr<UStaticMesh> WaterMeshOverride;

	/** 
	 * When this is set to true, the water mesh will always generate tiles for this water body. 
	 * For example, this can be useful to generate water tiles even when the water material is invalid, for the case where "empty" water tiles are actually desirable.
	 */
	UPROPERTY(Category = Rendering, EditAnywhere, BlueprintReadOnly, AdvancedDisplay)
	bool bAlwaysGenerateWaterMeshTiles = false;

	/** Higher number is higher priority. If two water bodies overlap and they don't have a transition material specified, this will be used to determine which water body to use the material from. Valid range is -8192 to 8191 */
	UPROPERTY(Category = Rendering, EditAnywhere, BlueprintReadOnly, meta = (ClampMin = "-8192", ClampMax = "8191"))
	int32 OverlapMaterialPriority = 0;

	UPROPERTY(Transient)
	TObjectPtr<UWaterSplineMetadata> WaterSplineMetadata;

	UPROPERTY(Category = Debug, VisibleInstanceOnly, Transient, NonPIEDuplicateTransient, TextExportTransient, meta = (DisplayAfter = "WaterMaterial"))
	TObjectPtr<UMaterialInstanceDynamic> WaterMID;

	UPROPERTY(Category = Debug, VisibleInstanceOnly, Transient, NonPIEDuplicateTransient, TextExportTransient, meta = (DisplayAfter = "WaterStaticMeshMaterial"))
	TObjectPtr<UMaterialInstanceDynamic> WaterStaticMeshMID;

	UPROPERTY(Category = Debug, VisibleInstanceOnly, Transient, NonPIEDuplicateTransient, TextExportTransient, meta = (DisplayAfter = "UnderwaterPostProcessMaterial"))
	TObjectPtr<UMaterialInstanceDynamic> UnderwaterPostProcessMID;
	
	UPROPERTY(Category = Debug, VisibleInstanceOnly, Transient, NonPIEDuplicateTransient, TextExportTransient, meta = (DisplayAfter = "WaterInfoMaterial"))
	TObjectPtr<UMaterialInstanceDynamic> WaterInfoMID;

	/** Islands in this water body*/
	UPROPERTY(Category = Water, EditAnywhere, AdvancedDisplay)
	TArray<TSoftObjectPtr<AWaterBodyIsland>> WaterBodyIslands;

	UPROPERTY(Category = Water, EditAnywhere, AdvancedDisplay)
	TArray<TSoftObjectPtr<AWaterBodyExclusionVolume>> WaterBodyExclusionVolumes;

	UPROPERTY(Transient)
	mutable TWeakObjectPtr<ALandscapeProxy> Landscape;

	UPROPERTY(Category = Water, VisibleInstanceOnly, AdvancedDisplay, TextExportTransient)
	TSoftObjectPtr<AWaterZone> OwningWaterZone;

	UPROPERTY(Category = Water, EditAnywhere, BlueprintReadOnly, AdvancedDisplay)
	TSoftObjectPtr<AWaterZone> WaterZoneOverride;

	UPROPERTY(Transient)
	FPostProcessSettings CurrentPostProcessSettings;

	// The navigation area class that will be generated on nav mesh
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Navigation, meta = (EditCondition = "bCanEverAffectNavigation"))
	TSubclassOf<UNavAreaBase> WaterNavAreaClass;

	/** If the Water Material assigned to this component has Fixed Depth enabled, this is the depth that is passed. */
	UPROPERTY(Category = Water, EditAnywhere, AdvancedDisplay)
	double FixedWaterDepth = 512.0;

	/**  Baked simulation data for this water body, owned by a UShallowWaterRiverComponent */
	UPROPERTY(Category = BakedSimulation, AdvancedDisplay, VisibleAnywhere)
	TWeakObjectPtr<UBakedShallowWaterSimulationComponent> BakedShallowWaterSim;

	/**  Override to disable use of the baked shallow water simulation for collisions and other uses */
	UPROPERTY(Category = BakedSimulation, AdvancedDisplay, EditAnywhere)
	bool bUseBakedSimForQueriesAndPhysics = true;

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	TArray<TLazyObjectPtr<AWaterBodyIsland>> Islands_DEPRECATED;
	UPROPERTY()
	TArray<TLazyObjectPtr<AWaterBodyExclusionVolume>> ExclusionVolumes_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	TObjectPtr<UMaterialInterface> WaterLODMaterial_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UPhysicalMaterial> PhysicalMaterial_DEPRECATED;

	UPROPERTY()
	bool bFillCollisionUnderWaterBodiesForNavmesh_DEPRECATED;

	UPROPERTY()
	FName CollisionProfileName_DEPRECATED;

	UPROPERTY()
	bool bGenerateCollisions_DEPRECATED = true;

	UPROPERTY()
	bool bCanAffectNavigation_DEPRECATED;

	UPROPERTY()
	bool bOverrideWaterMesh_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

private:
	/** Boolean to keep track of whether the water body is visible in the water mesh. This avoids unnecessary calls to rebuild the water mesh whenever UpdateComponentVisibility is called */
	bool bIsRenderedByWaterMeshAndVisible = false;

};
