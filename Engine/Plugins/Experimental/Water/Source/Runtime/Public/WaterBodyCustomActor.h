// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterBodyActor.h"
#include "WaterBodyCustomActor.generated.h"

#define UE_API WATER_API

class UDEPRECATED_CustomMeshGenerator;
class UStaticMeshComponent;

// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI)
class UDEPRECATED_CustomMeshGenerator : public UDEPRECATED_WaterBodyGenerator
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(NonPIEDuplicateTransient)
	TObjectPtr<UStaticMeshComponent> MeshComp;
};

// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI, Blueprintable)
class AWaterBodyCustom : public AWaterBody
{
	GENERATED_UCLASS_BODY()
protected:
	UE_API virtual void PostLoad() override;

#if WITH_EDITORONLY_DATA
	UPROPERTY(NonPIEDuplicateTransient)
	TObjectPtr<UDEPRECATED_CustomMeshGenerator> CustomGenerator_DEPRECATED;
#endif
};

#undef UE_API
