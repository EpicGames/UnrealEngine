// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterBodyComponent.h"
#include "WaterBodyCustomComponent.generated.h"

#define UE_API WATER_API

class UStaticMeshComponent;

// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI, Blueprintable)
class UWaterBodyCustomComponent : public UWaterBodyComponent
{
	GENERATED_UCLASS_BODY()
	friend class AWaterBodyCustom;
public:
	/** UWaterBodyComponent Interface */
	virtual EWaterBodyType GetWaterBodyType() const override { return EWaterBodyType::Transition; }
	UE_API virtual TArray<UPrimitiveComponent*> GetCollisionComponents(bool bInOnlyEnabledComponents = true) const override;
	UE_API virtual TArray<UPrimitiveComponent*> GetStandardRenderableComponents() const override;
	virtual bool CanEverAffectWaterMesh() const { return false; }
	virtual bool CanEverAffectWaterInfo() const { return false; }

	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

protected:
	/** UWaterBodyComponent Interface */
	UE_API virtual void Reset() override;
	UE_API virtual void BeginUpdateWaterBody() override;
	UE_API virtual void OnUpdateBody(bool bWithExclusionVolumes) override;
	UE_API virtual void CreateOrUpdateWaterMID() override;
	
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
#if WITH_EDITOR
	UE_API virtual TArray<TSharedRef<FTokenizedMessage>> CheckWaterBodyStatus() override;

	UE_API virtual const TCHAR* GetWaterSpriteTextureName() const override;

	UE_API virtual bool IsIconVisible() const override;

	UE_API virtual void PostLoad() override;
#endif // WITH_EDITOR

protected:
	UPROPERTY(NonPIEDuplicateTransient)
	TObjectPtr<UStaticMeshComponent> MeshComp;
};

#undef UE_API
