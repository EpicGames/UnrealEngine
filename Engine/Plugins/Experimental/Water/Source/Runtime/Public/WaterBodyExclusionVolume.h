// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/PhysicsVolume.h"
#include "WaterBodyExclusionVolume.generated.h"

#define UE_API WATER_API

class AWaterBody;

UENUM()
enum class EWaterExclusionMode
{
	/**
	 * Adds all water bodies specified in the WaterBodies list to the exclusion volume.
	 * If none are specified, no water body overlapped by this volume will be part of the exclusion.
	 */
	AddWaterBodiesListToExclusion,
	/**
	 * Removes all water bodies specified in the WaterBodies list from the exclusion volume.
	 * If none are specified, every water body overlapped by this volume will be part of the exclusion.
	 */
	RemoveWaterBodiesListFromExclusion,
};

struct FWaterExclusionVolumeChangedParams
{
	FWaterExclusionVolumeChangedParams(const FPropertyChangedEvent& InPropertyChangedEvent = FPropertyChangedEvent(/*InProperty = */nullptr))
		: PropertyChangedEvent(InPropertyChangedEvent)
	{}

	/** Provides some additional context about how the water exclusion volume data has changed (property, type of change...) */
	FPropertyChangedEvent PropertyChangedEvent;

	/** Indicates user initiated Parameter change */
	bool bUserTriggered = false;
};

/**
 * WaterBodyExclusionVolume allows players not enter surface swimming when touching a water volume
 */
UCLASS(MinimalAPI)
class AWaterBodyExclusionVolume : public APhysicsVolume
{
	GENERATED_UCLASS_BODY()

public:
	UE_API void UpdateOverlappingWaterBodies(const FWaterExclusionVolumeChangedParams& Params);

#if WITH_EDITOR
	UE_API void UpdateActorIcon();
#endif // WITH_EDITOR

	UE_API virtual void PostRegisterAllComponents() override;
protected:
	UE_API virtual void PostLoad() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void Destroyed() override;

#if WITH_EDITOR
	UE_API virtual void PostEditMove(bool bFinished) override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual FName GetCustomIconName() const override;
#endif // WITH_EDITOR

	/** Updates all water bodies affected by this exclusion volume to rebuild due to a change in this exclusion volume. */
	UE_API void UpdateAffectedWaterBodyCollisions(const FWaterExclusionVolumeChangedParams& Params);
public:
	/** Determines the behavior of the WaterBodies list. */
	UPROPERTY(EditAnywhere, Category = Water)
	EWaterExclusionMode ExclusionMode = EWaterExclusionMode::RemoveWaterBodiesListFromExclusion;

	/** List of water bodies that will be added or removed from the exclusion volume based on the ExclusionMode parameter. */
	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = Water, meta = (DisplayAfter=ExclusionMode))
	TArray<TSoftObjectPtr<AWaterBody>> WaterBodies;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecationMessage = "Property changed from boolean to EWaterExclusionMode enum. a value of AddWaterBodiesListToExclusion is equivalent to a value of false."))
	bool bExcludeAllOverlappingWaterBodies_DEPRECATED = true;

	UPROPERTY(meta = (DeprecationMessage = "Property renamed to WaterBodies"))
	TArray<TObjectPtr<AWaterBody>> WaterBodiesToExclude_DEPRECATED;

	UPROPERTY(meta = (DeprecationMessage = "Property renamed to bExcludeAllOverlapping"))
	bool bIgnoreAllOverlappingWaterBodies_DEPRECATED = false;

	UPROPERTY(meta = (DeprecationMessage = "Property renamed to WaterBodiesToExclude"))
	TArray<TObjectPtr<AWaterBody>> WaterBodiesToIgnore_DEPRECATED;

	UPROPERTY(meta = (DeprecatedProperty))
	TObjectPtr<AWaterBody> WaterBodyToIgnore_DEPRECATED;

	UPROPERTY(Transient)
	TObjectPtr<class UBillboardComponent> ActorIcon;
#endif // WITH_EDITORONLY_DATA
};

#undef UE_API
