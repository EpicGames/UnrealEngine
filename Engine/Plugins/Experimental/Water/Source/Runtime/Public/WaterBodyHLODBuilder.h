// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "WorldPartition/HLOD/HLODBuilder.h"
#include "WaterBodyHLODBuilder.generated.h"

UCLASS(HideDropdown)
class UWaterBodyHLODBuilder : public UHLODBuilder
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	virtual bool ComputeHLODHash(FHLODHashBuilder& HashBuilder, const UActorComponent* InSourceComponent) const override;

	/**
	 * Components created with this method need to be properly outered & assigned to your target actor.
	 */
	virtual TArray<UActorComponent*> Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const override;
#endif
};
