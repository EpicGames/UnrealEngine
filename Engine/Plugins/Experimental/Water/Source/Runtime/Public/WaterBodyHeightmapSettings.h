// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "WaterFalloffSettings.h"
#include "WaterBrushEffects.h"
#include "WaterBodyHeightmapSettings.generated.h"

/** The blend mode changes how the brush material is applied to the terrain. */
UENUM(BlueprintType)
enum class EWaterBrushBlendType : uint8
{
	/** Alpha Blend will affect the heightmap both upwards and downwards. */
	AlphaBlend,
	/** Limits the brush to only lowering the terrain. */
	Min,
	/** Limits the brush to only raising the terrain. */
	Max,
	/** Performs an additive blend, using a flat Z=0 terrain as the input. Useful when you want to preserve underlying detail or ramps. */
	Additive,
};

USTRUCT(BlueprintType)
struct FWaterBodyHeightmapSettings
{
	GENERATED_BODY()

	WATER_API FWaterBodyHeightmapSettings();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = TerrainCarvingSettings)
	EWaterBrushBlendType BlendMode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = TerrainCarvingSettings)
	FWaterFalloffSettings FalloffSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = TerrainCarvingSettings)
	FWaterBrushEffects Effects;

	UPROPERTY()
	bool bInvertShape_DEPRECATED = false;

	UPROPERTY()
	int32 Priority_DEPRECATED = 0;
};
