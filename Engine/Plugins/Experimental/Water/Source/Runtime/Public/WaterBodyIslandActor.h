// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "WaterBrushActorInterface.h"
#include "WaterBodyHeightmapSettings.h"
#include "WaterBodyWeightmapSettings.h"
#include "WaterCurveSettings.h"
#include "WaterBodyIslandActor.generated.h"

#define UE_API WATER_API

class UWaterSplineComponent;
struct FOnWaterSplineDataChangedParams;
class USplineMeshComponent;
class UBillboardComponent;
class AWaterBody;

// ----------------------------------------------------------------------------------

struct FOnWaterBodyIslandChangedParams
{
	FOnWaterBodyIslandChangedParams(const FPropertyChangedEvent& InPropertyChangedEvent = FPropertyChangedEvent(/*InProperty = */nullptr))
		: PropertyChangedEvent(InPropertyChangedEvent)
	{}

	/** Provides some additional context about how the water body island data has changed (property, type of change...) */
	FPropertyChangedEvent PropertyChangedEvent;

	/** Indicates that property related to the water body island's visual shape has changed */
	bool bShapeOrPositionChanged = false;

	/** Indicates that a property affecting the terrain weightmaps has changed */
	bool bWeightmapSettingsChanged = false;

	/** Indicates user initiated change*/
	bool bUserTriggered = false;
};


// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI, Blueprintable)
class AWaterBodyIsland : public AActor, public IWaterBrushActorInterface
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin IWaterBrushActorInterface interface
	virtual bool AffectsLandscape() const override { return true; }
	virtual bool AffectsWaterMesh() const override { return false; }
	virtual bool CanEverAffectWaterMesh() const override { return false; }
	virtual bool CanEverAffectWaterInfo() const override { return false; }

#if WITH_EDITOR
	virtual const FWaterCurveSettings& GetWaterCurveSettings() const { return WaterCurveSettings; }
	virtual const FWaterBodyHeightmapSettings& GetWaterHeightmapSettings() const override { return WaterHeightmapSettings; }
	virtual const TMap<FName, FWaterBodyWeightmapSettings>& GetLayerWeightmapSettings() const override { return WaterWeightmapSettings; }
	UE_API virtual ETextureRenderTargetFormat GetBrushRenderTargetFormat() const override;
	UE_API virtual void GetBrushRenderDependencies(TSet<UObject*>& OutDependencies) const override;
#endif //WITH_EDITOR
	//~ End IWaterBrushActorInterface interface
	
	//~ Begin AActor Interface
#if WITH_EDITOR
	UE_API virtual void PostEditMove(bool bFinished) override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	// ~End AActor Interface

	UFUNCTION(BlueprintCallable, Category=Water)
	UWaterSplineComponent* GetWaterSpline() const { return SplineComp; }

	UE_API void UpdateHeight();

	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	
#if WITH_EDITOR
	UE_API virtual void GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const override;
	UE_API void UpdateOverlappingWaterBodyComponents();
	UE_API void UpdateActorIcon();
#endif // WITH_EDITOR

	UPROPERTY(Category = Terrain, EditAnywhere, BlueprintReadWrite)
	FWaterCurveSettings WaterCurveSettings;

	UPROPERTY(Category = Terrain, EditAnywhere, BlueprintReadWrite)
	FWaterBodyHeightmapSettings WaterHeightmapSettings;

	UPROPERTY(Category = Terrain, EditAnywhere, BlueprintReadWrite)
	TMap<FName, FWaterBodyWeightmapSettings> WaterWeightmapSettings;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<UBillboardComponent> ActorIcon;
#endif

protected:
	UE_API virtual void Destroyed() override;

#if WITH_EDITOR
	UE_API void UpdateAll(const FOnWaterBodyIslandChangedParams& InParams);
	
	UE_API void OnWaterSplineDataChanged(const FOnWaterSplineDataChangedParams& InParams);
	UE_API void OnWaterBodyIslandChanged(const FOnWaterBodyIslandChangedParams& InParams);

#endif

protected:
	/**
	 * The spline data attached to this water type.
	 */
	UPROPERTY(VisibleAnywhere, Category = Water, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UWaterSplineComponent> SplineComp;
};

#undef UE_API
