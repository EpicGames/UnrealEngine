// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterBodyActor.h"
#include "WaterBodyLakeActor.generated.h"

#define UE_API WATER_API

class UBoxComponent;
class ULakeCollisionComponent;

// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI)
class UDEPRECATED_LakeGenerator : public UDEPRECATED_WaterBodyGenerator
{
	GENERATED_UCLASS_BODY()
public:
	UPROPERTY(NonPIEDuplicateTransient)
	TObjectPtr<UStaticMeshComponent> LakeMeshComp;

	UPROPERTY()
	TObjectPtr<UBoxComponent> LakeCollisionComp_DEPRECATED;

	UPROPERTY(NonPIEDuplicateTransient)
	TObjectPtr<ULakeCollisionComponent> LakeCollision;
};

// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI, Blueprintable)
class AWaterBodyLake : public AWaterBody
{
	GENERATED_UCLASS_BODY()
protected:
	UE_API virtual void PostLoad() override;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UDEPRECATED_LakeGenerator> LakeGenerator_DEPRECATED;
#endif
};

#undef UE_API
