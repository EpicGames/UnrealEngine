// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterBodyComponent.h"
#include "WaterBodyLakeComponent.generated.h"

#define UE_API WATER_API

class UBoxComponent;
class ULakeCollisionComponent;
class UDEPRECATED_LakeGenerator;

// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI, Blueprintable)
class UWaterBodyLakeComponent : public UWaterBodyComponent
{
	GENERATED_UCLASS_BODY()

	friend class AWaterBodyLake;
public:
	/** UWaterBodyComponent Interface */
	virtual EWaterBodyType GetWaterBodyType() const override { return EWaterBodyType::Lake; }
	UE_API virtual TArray<UPrimitiveComponent*> GetCollisionComponents(bool bInOnlyEnabledComponents = true) const override;
	UE_API virtual TArray<UPrimitiveComponent*> GetStandardRenderableComponents() const override;
	UE_API virtual bool GenerateWaterBodyMesh(UE::Geometry::FDynamicMesh3& OutMesh, UE::Geometry::FDynamicMesh3* OutDilatedMesh = nullptr) const override;

	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

protected:
	/** UWaterBodyComponent Interface */
	UE_API virtual void Reset() override;
	UE_API virtual void OnUpdateBody(bool bWithExclusionVolumes) override;

#if WITH_EDITOR
	UE_API virtual const TCHAR* GetWaterSpriteTextureName() const override;

	UE_API virtual FVector GetWaterSpriteLocation() const override;
#endif // WITH_EDITOR

	UPROPERTY(NonPIEDuplicateTransient)
	TObjectPtr<UStaticMeshComponent> LakeMeshComp;

	UPROPERTY(NonPIEDuplicateTransient)
	TObjectPtr<ULakeCollisionComponent> LakeCollision;
};

#undef UE_API
