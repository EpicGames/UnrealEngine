// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "WaterBodyComponent.h"
#include "WaterZoneActor.h"

#define UE_API WATER_API

class FWaterViewExtension;

DECLARE_MULTICAST_DELEGATE_OneParam(FWaterBodyEvent, UWaterBodyComponent*);


class FWaterBodyManager
{
public:
	UE_API void Initialize(UWorld* World);
	UE_API void Deinitialize();

	/** 
	 * Register any water body component upon addition to the world
	 * @param InWaterBodyComponent
	 * @return int32 the unique sequential index assigned to this water body component
	 */
	UE_API int32 AddWaterBodyComponent(UWaterBodyComponent* InWaterBodyComponent);

	/** Unregister any water body upon removal to the world */
	UE_API void RemoveWaterBodyComponent(UWaterBodyComponent* InWaterBodyComponent);

	UE_API int32 AddWaterZone(AWaterZone* InWaterZone);
	UE_API void RemoveWaterZone(AWaterZone* InWaterZone);

	/** Recomputes water gpu data whenever it changes on one of the managed water types. */
	UE_API void RequestGPUDataRebuild();

	/** Recomputes wave-related data whenever it changes on one of water bodies. */
	UE_API void RequestWaveDataRebuild();

	/** Returns the maximum of all MaxWaveHeight : */
	float GetGlobalMaxWaveHeight() const { return GlobalMaxWaveHeight; }

	/** Execute a predicate function on each valid water body. Predicate should return false for early exit. */
	UE_API void ForEachWaterBodyComponent (TFunctionRef<bool(UWaterBodyComponent*)> Pred) const;

	/** Execute a predicate function on each valid water body. Predicate should return false for early exit. */
	static UE_API void ForEachWaterBodyComponent (const UWorld* World, TFunctionRef<bool(UWaterBodyComponent*)> Pred);

	UE_API void ForEachWaterZone(TFunctionRef<bool(AWaterZone*)> Pred) const;
	static UE_API void ForEachWaterZone(const UWorld* World, TFunctionRef<bool(AWaterZone*)> Pred);

	bool HasAnyWaterBodies() const { return WaterBodyComponents.Num() > 0; }

	int32 NumWaterBodies() const { return WaterBodyComponents.Num(); }
	int32 MaxWaterBodyIndex() const { return WaterBodyComponents.GetMaxIndex(); }

	/** Shrinks the sparse array storage for water body components and water zones. Ensures that MaxIndex == MaxAllocatedIndex */
	UE_API void Shrink();

	int32 NumWaterZones() const { return WaterZones.Num(); }

	FWaterViewExtension* GetWaterViewExtension() { return WaterViewExtension.Get(); }

	TWeakPtr<FWaterViewExtension, ESPMode::ThreadSafe> GetWaterViewExtensionWeakPtr() { return WaterViewExtension; }

	FWaterBodyEvent OnWaterBodyAdded;

	FWaterBodyEvent OnWaterBodyRemoved;

private:
	/** List of components registered to this manager. */
	TSparseArray<TWeakObjectPtr<UWaterBodyComponent>> WaterBodyComponents;

	/** List of Water zones registered to this manager. */
	TSparseArray<TWeakObjectPtr<AWaterZone>> WaterZones;

	float GlobalMaxWaveHeight = 0.0f;

	TSharedPtr<FWaterViewExtension, ESPMode::ThreadSafe> WaterViewExtension;
};

#undef UE_API
