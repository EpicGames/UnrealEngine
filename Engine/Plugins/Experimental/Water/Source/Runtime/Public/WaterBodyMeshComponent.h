// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/StaticMeshComponent.h"

#include "WaterBodyMeshComponent.generated.h"

class UWaterBodyComponent;
struct FMeshDescription;
class UMaterialInterface;

/**
 * Base class for meshes used to render water bodies without relying on the water zone/water mesh.
 */
UCLASS(MinimalAPI)
class UWaterBodyMeshComponent : public UStaticMeshComponent
{
	GENERATED_UCLASS_BODY()
public:

protected:
	virtual bool CanCreateSceneProxy() const;

	// Exclude water mesh components from HLOD since we have a custom HLOD builder for water bodies.
	virtual bool IsHLODRelevant() const override { return false; }

	// Exclude water meshes from loading on the server since we don't render at all.
	virtual bool NeedsLoadForServer() const override { return false; }

#if WITH_EDITOR
	virtual void PostLoad() override;
	virtual void PostStaticMeshCompilation() override;
	void FixupCollisionOnBodySetup();
#endif
};

