// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterBodyActor.h"
#include "WaterBodyOceanActor.generated.h"

#define UE_API WATER_API

class UOceanBoxCollisionComponent;
class UOceanCollisionComponent;

// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI)
class UDEPRECATED_OceanGenerator : public UDEPRECATED_WaterBodyGenerator
{
	GENERATED_UCLASS_BODY()
public:
	UPROPERTY(NonPIEDuplicateTransient)
	TArray<TObjectPtr<UOceanBoxCollisionComponent>> CollisionBoxes;

	UPROPERTY(NonPIEDuplicateTransient)
	TArray<TObjectPtr<UOceanCollisionComponent>> CollisionHullSets;
};

// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI, Blueprintable)
class AWaterBodyOcean : public AWaterBody
{
	GENERATED_UCLASS_BODY()
protected:
	UE_API virtual void PostLoad() override;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UDEPRECATED_OceanGenerator> OceanGenerator_DEPRECATED;
	
	UPROPERTY()
	FVector CollisionExtents_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
};

#undef UE_API
