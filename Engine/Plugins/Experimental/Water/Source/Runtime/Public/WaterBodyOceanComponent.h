// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterBodyComponent.h"
#include "WaterBodyOceanComponent.generated.h"

#define UE_API WATER_API

class UOceanCollisionComponent;
class UOceanBoxCollisionComponent;

// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI, Blueprintable)
class UWaterBodyOceanComponent : public UWaterBodyComponent
{
	GENERATED_UCLASS_BODY()

	friend class AWaterBodyOcean;
public:
	/** UWaterBodyComponent Interface */
	virtual EWaterBodyType GetWaterBodyType() const override { return EWaterBodyType::Ocean; }
	UE_API virtual TArray<UPrimitiveComponent*> GetCollisionComponents(bool bInOnlyEnabledComponents = true) const override;
	virtual FVector GetCollisionExtents() const override { return CollisionExtents; }
	UE_API virtual void SetHeightOffset(float InHeightOffset) override;
	virtual float GetHeightOffset() const override { return HeightOffset; }
	
#if WITH_EDITOR
	UE_API void SetCollisionExtents(const FVector& NewExtents);

	UE_API void SetOceanExtent(const FVector2D& NewExtents);

	/** Rebuilds the ocean mesh to completely fill the zone to which it belongs. */
	UFUNCTION(CallInEditor, Category = Water)
	UE_API void FillWaterZoneWithOcean();
#endif // WITH_EDITOR

protected:
	/** UWaterBodyComponent Interface */
	virtual bool IsBodyDynamic() const override { return true; }
	UE_API virtual void BeginUpdateWaterBody() override;
	UE_API virtual void OnUpdateBody(bool bWithExclusionVolumes) override;
	UE_API virtual void Reset() override;
	UE_API virtual bool GenerateWaterBodyMesh(UE::Geometry::FDynamicMesh3& OutMesh, UE::Geometry::FDynamicMesh3* OutDilatedMesh = nullptr) const override;

	UE_API virtual void PostLoad() override;
	UE_API virtual void OnPostRegisterAllComponents() override;
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const;

	UE_API virtual void OnPostActorCreated() override;

#if WITH_EDITOR
	UE_API virtual void OnPostEditChangeProperty(FOnWaterBodyChangedParams& InOutOnWaterBodyChangedParams) override;

	UE_API virtual const TCHAR* GetWaterSpriteTextureName() const override;

	UE_API virtual TArray<TSharedRef<FTokenizedMessage>> CheckWaterBodyStatus();

	UE_API virtual void OnWaterBodyRenderDataUpdated() override;
#endif
protected:
	UPROPERTY(NonPIEDuplicateTransient)
	TArray<TObjectPtr<UOceanBoxCollisionComponent>> CollisionBoxes;

	UPROPERTY(NonPIEDuplicateTransient)
	TArray<TObjectPtr<UOceanCollisionComponent>> CollisionHullSets;

	UPROPERTY(Category = Collision, EditAnywhere, BlueprintReadOnly)
	FVector CollisionExtents;

	/** The extent of the ocean, centered around water zone to which the ocean belongs. */
	UPROPERTY(Category = Water, EditAnywhere, BlueprintReadOnly)
	FVector2D OceanExtents;

	/** Saved water zone location so that the ocean mesh can be regenerated relative to it and match it perfectly without being loaded. */
	UPROPERTY()
	FVector2D SavedZoneLocation;

	/** If enabled, oceans will always center their mesh/bounds on the owning water zone by using a saved location that is updated whenever the ocean mesh is rebuilt. */
	UPROPERTY(EditAnywhere, Category = Water, AdvancedDisplay)
	bool bCenterOnWaterZone = true;

	UPROPERTY(Transient)
	float HeightOffset = 0.0f;

#if WITH_EDITORONLY_DATA

	UPROPERTY()
	FVector2D VisualExtents_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
};

#undef UE_API
