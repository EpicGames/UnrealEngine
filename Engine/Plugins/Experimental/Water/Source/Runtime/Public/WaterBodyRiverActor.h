// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterBodyActor.h"
#include "WaterBodyRiverActor.generated.h"

#define UE_API WATER_API

class URiverGenerator;
class UMaterialInstanceDynamic;
class USplineMeshComponent;

// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI)
class UDEPRECATED_RiverGenerator : public UDEPRECATED_WaterBodyGenerator
{
	GENERATED_UCLASS_BODY()
public:
	UPROPERTY(NonPIEDuplicateTransient)
	TArray<TObjectPtr<USplineMeshComponent>> SplineMeshComponents;
};

// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI, Blueprintable)
class AWaterBodyRiver : public AWaterBody
{
	GENERATED_UCLASS_BODY()
protected:
	UE_API virtual void PostLoad() override;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UDEPRECATED_RiverGenerator> RiverGenerator_DEPRECATED;
	
	/** Material used when a river is overlapping a lake. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> LakeTransitionMaterial_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> LakeTransitionMID_DEPRECATED;

	/** This is the material used when a river is overlapping the ocean. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> OceanTransitionMaterial_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> OceanTransitionMID_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
};

#undef UE_API
