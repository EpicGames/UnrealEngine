// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterBodyComponent.h"
#include "WaterBodyRiverComponent.generated.h"

#define UE_API WATER_API

class UMaterialInstanceDynamic;
class USplineMeshComponent;

// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI, Blueprintable)
class UWaterBodyRiverComponent : public UWaterBodyComponent
{
	GENERATED_UCLASS_BODY()
	friend class AWaterBodyRiver;
public:
	/** UWaterBodyComponent Interface */
	virtual EWaterBodyType GetWaterBodyType() const override { return EWaterBodyType::River; }
	UE_API virtual TArray<UPrimitiveComponent*> GetCollisionComponents(bool bInOnlyEnabledComponents = true) const override;
	UE_API virtual TArray<UPrimitiveComponent*> GetStandardRenderableComponents() const override;
	UE_API virtual UMaterialInstanceDynamic* GetRiverToLakeTransitionMaterialInstance() override;
	UE_API virtual UMaterialInstanceDynamic* GetRiverToOceanTransitionMaterialInstance() override;
	UE_API virtual UMaterialInterface* GetRiverToLakeTransitionMaterial() const override;
	UE_API virtual UMaterialInterface* GetRiverToOceanTransitionMaterial() const override;

#if WITH_EDITOR
	UE_API virtual TArray<UPrimitiveComponent*> GetBrushRenderableComponents() const override;
#endif //WITH_EDITOR

	UFUNCTION(BlueprintCallable, Category = Rendering)
	UE_API void SetLakeTransitionMaterial(UMaterialInterface* InMat);

	UFUNCTION(BlueprintCallable, Category = Rendering)
	UE_API void SetOceanTransitionMaterial(UMaterialInterface* InMat);

	UFUNCTION(BlueprintCallable, Category = Rendering)
	UE_API void SetLakeAndOceanTransitionMaterials(UMaterialInterface* InLakeTransition, UMaterialInterface* InOceanTransition);


	UFUNCTION(BlueprintCallable, Category = WaterBody)
	UE_API float GetRiverWidthAtSplineInputKey(float InKey) const;

	UFUNCTION(BlueprintCallable, Category = WaterBody)
	UE_API float GetRiverDepthAtSplineInputKey(float InKey) const;

	UFUNCTION(BlueprintCallable, Category = WaterBody)
	UE_API void SetRiverWidthAtSplineInputKey(float InKey, float InWidth);

	UFUNCTION(BlueprintCallable, Category = WaterBody)
	UE_API void SetRiverDepthAtSplineInputKey(float InKey, float InDepth);

protected:
	/** UWaterBodyComponent Interface */
	UE_API virtual void Reset() override;
	UE_API virtual void UpdateMaterialInstances() override;
	UE_API virtual void OnUpdateBody(bool bWithExclusionVolumes) override;
	UE_API virtual bool GenerateWaterBodyMesh(UE::Geometry::FDynamicMesh3& OutMesh, UE::Geometry::FDynamicMesh3* OutDilatedMesh = nullptr) const override;

	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
#if WITH_EDITOR
	UE_API virtual void OnPostEditChangeProperty(FOnWaterBodyChangedParams& InOutOnWaterBodyChangedParams) override;

	UE_API virtual const TCHAR* GetWaterSpriteTextureName() const override;
#endif

	UE_API void CreateOrUpdateLakeTransitionMID();
	UE_API void CreateOrUpdateOceanTransitionMID();

	UE_API void GenerateMeshes();
	UE_API void UpdateSplineMesh(USplineMeshComponent* MeshComp, int32 SplinePointIndex);

protected:
	UPROPERTY(NonPIEDuplicateTransient)
	TArray<TObjectPtr<USplineMeshComponent>> SplineMeshComponents;

	/** Material used when a river is overlapping a lake. */
	UPROPERTY(Category = Rendering, EditAnywhere, BlueprintReadOnly, meta = (DisplayName = "River to Lake Transition"))
	TObjectPtr<UMaterialInterface> LakeTransitionMaterial;

	UPROPERTY(Category = Debug, VisibleInstanceOnly, Transient, NonPIEDuplicateTransient, TextExportTransient, meta = (DisplayAfter = "LakeTransitionMaterial"))
	TObjectPtr<UMaterialInstanceDynamic> LakeTransitionMID;

	/** This is the material used when a river is overlapping the ocean. */
	UPROPERTY(Category = Rendering, EditAnywhere, BlueprintReadOnly, meta = (DisplayName = "River to Ocean Transition"))
	TObjectPtr<UMaterialInterface> OceanTransitionMaterial;

	UPROPERTY(Category = Debug, VisibleInstanceOnly, Transient, NonPIEDuplicateTransient, TextExportTransient, meta = (DisplayAfter = "OceanTransitionMaterial"))
	TObjectPtr<UMaterialInstanceDynamic> OceanTransitionMID;
};

#undef UE_API
