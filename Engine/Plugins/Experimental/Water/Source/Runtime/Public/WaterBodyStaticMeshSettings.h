// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterBodyStaticMeshSettings.generated.h"

USTRUCT()
struct FWaterBodyStaticMeshSettings
{
	GENERATED_BODY()

	/** If enabled, the water body will generate static meshes to use outside of the dynamic mesh generated by the water zone. */
	UPROPERTY(Category = StaticMesh, EditAnywhere)
	bool bEnableWaterBodyStaticMesh = false;

	/** If enabled, the water body static mesh will be split into separate meshes to allow them to stream in and out as needed to save memory. */
	UPROPERTY(Category = StaticMesh, EditAnywhere, meta = (EditCondition = "bEnableWaterBodyStaticMesh"))
	bool bSectionWaterBodyStaticMesh = false;

	/** The size of each section tile if sectioning is enabled. */
	UPROPERTY(Category = StaticMesh, EditAnywhere, AdvancedDisplay, meta = (EditCondition = "bEnableWaterBodyStaticMesh && bSectionWaterBodyStaticMesh"))
	double SectionSize = 8192.0;
};
