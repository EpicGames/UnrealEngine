// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GerstnerWaterWaves.h"
#include "Components/SplineComponent.h"
#include "WaterBodyTypes.generated.h"

class UWaterSplineComponent;
class UWaterSplineMetadata;
class UWaterBodyComponent;
struct FWaterSplineDataPhysics;

extern TAutoConsoleVariable<float> CVarWaterOceanFallbackDepth;

UENUM(BlueprintType)
enum class EWaterBodyType : uint8
{
	/** Rivers defined by a spline down the middle */
	River,
	/** Lakes defined by a close loop spline around the shore with water in the middle*/
	Lake,
	/* Ocean defined by a shoreline spline and rendered out to a far distance */
	Ocean,
	/** A custom water body that can be used for gameplay reasons.  Uses a spline down the middle to encode gameplay data. Requires a custom mesh to render, doesn't affect landscape */
	Transition UMETA(DisplayName = "Custom"),
	/** Max value */
	Num UMETA(Hidden),
};

enum class EWaterBodyQueryFlags
{
	None = 0,
	/** Computes the location on the water plane (+ optionally the location on the water surface if IncludeWaves is passed) */
	ComputeLocation = (1 << 0),
	/** Computes the normal on the water plane (+ optionally the normal on the water surface if IncludeWaves is passed) */
	ComputeNormal = (1 << 1),
	/** Computes the water velocity */
	ComputeVelocity = (1 << 2),
	/** Compute the water depth */
	ComputeDepth = (1 << 3),
	/** Computes the how much under the water surface the point is (implies ComputeLocation) : > 0 if underwater */
	ComputeImmersionDepth = (1 << 4),
	/** Includes waves in water depth computation (requires ComputeLocation or ComputeNormal or ComputeDepth or ComputeImmersionDepth) */
	IncludeWaves = (1 << 5),
	/** Use simple version of waves computation (requires IncludeWaves) */
	SimpleWaves = (1 << 6),
	/** Ignore the water exclusion volumes (it is expected to check for IsInExclusionVolume() when not using this flag as the returned information will be invalid) */
	IgnoreExclusionVolumes = (1 << 7),
};
ENUM_CLASS_FLAGS(EWaterBodyQueryFlags);

// ----------------------------------------------------------------------------------
/** Struct holding wave computation result : */
struct FWaveInfo
{
	/** Time that has been used in the wave computation */
	float ReferenceTime = 0.0f;
	/** Factor between 0 and 1 to dampen the waves (e.g. near the shore, based on a mask, etc.) */
	float AttenuationFactor = 1.0f;
	/** Current wave amplitude at this location (includes AttenuationFactor dampening) */
	float Height = 0.0f;
	/** Maximum wave amplitude at this location (includes AttenuationFactor dampening) */
	float MaxHeight = 0.0f;
	/** Current wave normal at this location (includes AttenuationFactor dampening) */
	FVector Normal = FVector::UpVector;
};

// ----------------------------------------------------------------------------------

UENUM()
enum class EWaterBodyQueryError : uint8
{
	UnknownError,
	// Invalid to query the water surface info for a water body which is not yet registered to the parent water body actor.
	WaterBodyNotRegistered,
	// Invalid to query the water surface info for a water body which has a null WaterSplineMetadata.
	// This should be impossible in normal lifetime but could be null during the early parts of the component lifecycle before registration.
	NullWaterSplineMetadata,
};

/** Struct holding the result from water queries :  */
struct FWaterBodyQueryResult
{
	FWaterBodyQueryResult(const TOptional<float>& InSplineInputKey = TOptional<float>())
		: SplineInputKey(InSplineInputKey)
	{
	}

	EWaterBodyQueryFlags GetQueryFlags() const { return QueryFlags; }
	const FVector& GetWaterPlaneLocation() const { check(QueryFlags & EWaterBodyQueryFlags::ComputeLocation); return WaterPlaneLocation; }
	const FVector& GetWaterSurfaceLocation() const { check(QueryFlags & EWaterBodyQueryFlags::ComputeLocation); return WaterSurfaceLocation; }
	const FVector& GetWaterPlaneNormal() const { check(QueryFlags & EWaterBodyQueryFlags::ComputeNormal); return WaterPlaneNormal; }
	const FVector& GetWaterSurfaceNormal() const { check(QueryFlags & EWaterBodyQueryFlags::ComputeNormal); return WaterSurfaceNormal; }
	float GetWaterPlaneDepth() const { check(QueryFlags & EWaterBodyQueryFlags::ComputeDepth); return WaterPlaneDepth; }
	float GetWaterSurfaceDepth() const { check(QueryFlags & EWaterBodyQueryFlags::ComputeDepth); return WaterSurfaceDepth; }
	const FWaveInfo& GetWaveInfo() const { check(EnumHasAnyFlags(QueryFlags, EWaterBodyQueryFlags::IncludeWaves)); return WaveInfo; }
	float GetImmersionDepth() const { check(QueryFlags & EWaterBodyQueryFlags::ComputeImmersionDepth); return ImmersionDepth; }
	bool IsInWater() const { check(QueryFlags & EWaterBodyQueryFlags::ComputeImmersionDepth); return (ImmersionDepth > 0.f); }
	const FVector& GetVelocity() const { check(QueryFlags & EWaterBodyQueryFlags::ComputeVelocity); return Velocity; }
	const TOptional<float>& GetSplineInputKey() const { return SplineInputKey; }
	bool IsInExclusionVolume() const { return bIsInExclusionVolume; }

	void SetQueryFlags(EWaterBodyQueryFlags InFlags) { QueryFlags = InFlags; }
	float LazilyComputeSplineKey(const UWaterBodyComponent& InWaterBodyComponent, const FVector& InWorldLocation);
	float LazilyComputeSplineKey(const FWaterSplineDataPhysics& InWaterSpline, const FVector& InWorldLocation);
	void SetWaterPlaneLocation(const FVector& InValue) { check(QueryFlags & EWaterBodyQueryFlags::ComputeLocation); WaterPlaneLocation = InValue; }
	void SetWaterSurfaceLocation(const FVector& InValue) { check(QueryFlags & EWaterBodyQueryFlags::ComputeLocation); WaterSurfaceLocation = InValue; }
	void SetWaterPlaneNormal(const FVector& InValue) { check(QueryFlags & EWaterBodyQueryFlags::ComputeNormal); WaterPlaneNormal = InValue; }
	void SetWaterSurfaceNormal(const FVector& InValue) { check(QueryFlags & EWaterBodyQueryFlags::ComputeNormal); WaterSurfaceNormal = InValue; }
	void SetWaterPlaneDepth(float InValue) { check(QueryFlags & EWaterBodyQueryFlags::ComputeDepth); WaterPlaneDepth = InValue; }
	void SetWaterSurfaceDepth(float InValue) { check(QueryFlags & EWaterBodyQueryFlags::ComputeDepth); WaterSurfaceDepth = InValue; }
	void SetWaveInfo(const FWaveInfo& InValue) { check(EnumHasAllFlags(QueryFlags, EWaterBodyQueryFlags::IncludeWaves)); WaveInfo = InValue; }
	void SetImmersionDepth(float InValue) { check(QueryFlags & EWaterBodyQueryFlags::ComputeImmersionDepth); ImmersionDepth = InValue; }
	void SetVelocity(const FVector& InValue) { check(QueryFlags & EWaterBodyQueryFlags::ComputeVelocity); Velocity = InValue; }
	void SetIsInExclusionVolume(bool bInValue) { bIsInExclusionVolume = bInValue; }

private:
	// ----------------------------------------------------------------------------------
	// Always set :
	EWaterBodyQueryFlags QueryFlags = EWaterBodyQueryFlags::None;

	// ----------------------------------------------------------------------------------
	// Optionally set :
	// Sample point on the spline used in the query (may or may not be computed depending on the query/water body type) :
	TOptional<float> SplineInputKey;
	// Location of the water plane (i.e. the water surface, excluding waves) : 
	FVector WaterPlaneLocation = FVector::ZeroVector;
	// (Optionally) includes wave perturbation (if IncludeWaves is passed), otherwise, same as WaterPlaneLocation : 
	FVector WaterSurfaceLocation = FVector::ZeroVector;
	// Normal of the water plane (i.e. the water surface, excluding waves) :
	FVector WaterPlaneNormal = FVector::UpVector;
	// (Optionally) includes wave perturbation (if IncludeWaves is passed), otherwise, same as WaterPlaneNormal :
	FVector WaterSurfaceNormal = FVector::UpVector;
	// Depth of the water plane (i.e. the water surface, excluding waves) : 
	float WaterPlaneDepth = 0.0f;
	// (Optionally) includes wave perturbation (if IncludeWaves is passed), otherwise, same as WaterPlaneDepth :
	float WaterSurfaceDepth = 0.0f;
	// Wave perturbation information (if IncludeWaves is passed) :
	FWaveInfo WaveInfo;
	// Difference between the query location depth and the water surface depth, (optionally) includes wave perturbation (if IncludeWaves is passed)
	float ImmersionDepth = 0.0f;
	// Velocity of the water at this location
	FVector Velocity = FVector::ZeroVector;
	// Whether the requested location is within an exclusion volme (optionally ignored)
	bool bIsInExclusionVolume = false;
};

/* Structs used for async sim */

struct FWaterSplineMetadataPhysics
{
	FInterpCurveFloat Depth;
	FInterpCurveFloat WaterVelocityScalar;

	FWaterSplineMetadataPhysics() {};
	FWaterSplineMetadataPhysics& operator=(const UWaterSplineMetadata* Metadata);
};

struct FWaterSplineDataPhysics
{
	FTransform ComponentTransform;
	FSplineCurves SplineCurves;
	FVector DefaultUpVector;

	FWaterSplineDataPhysics() {};
	FWaterSplineDataPhysics(UWaterSplineComponent* WaterSplineComponent);
	FWaterSplineDataPhysics& operator=(const UWaterSplineComponent* WaterSplineComponent);
	WATER_API float FindInputKeyClosestToWorldLocation(const FVector& WorldLocation) const;
	WATER_API FVector GetLocationAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace = ESplineCoordinateSpace::World) const;
	FVector GetUpVectorAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace = ESplineCoordinateSpace::World) const;
	FQuat GetQuaternionAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace = ESplineCoordinateSpace::World) const;
	WATER_API FVector GetDirectionAtSplineInputKey(float InKey, ESplineCoordinateSpace::Type CoordinateSpace = ESplineCoordinateSpace::World) const;
};

struct FSolverSafeWaterBodyData
{
	UWorld* World;
	TArray<UPrimitiveComponent*> LandscapeCollisionComponents;
	FWaterSplineDataPhysics WaterSpline;
	FWaterSplineMetadataPhysics WaterSplineMetadata;
	FVector Location;
	EWaterBodyType WaterBodyType;
	float OceanHeightOffset; // needs to be updated when ocean height changes
	TArray<FGerstnerWave> WaveParams;
	float WaveSpeedFactor;
	float TargetWaveMaskDepth;
	float MaxWaveHeight;
	int32 WaterBodyIndex;

	FSolverSafeWaterBodyData() {}
	WATER_API FSolverSafeWaterBodyData(UWaterBodyComponent* WaterBodyComponent);

	WATER_API FWaterBodyQueryResult QueryWaterInfoClosestToWorldLocation(const FVector& InWorldLocation, EWaterBodyQueryFlags InQueryFlags, float InWaveReferenceTime, const TOptional<float>& InSplineInputKey = TOptional<float>()) const;
	WATER_API float GetWaterVelocityAtSplineInputKey(float InKey) const;
	WATER_API FVector GetWaterVelocityVectorAtSplineInputKey(float InKey) const;
	bool WaterBodyTypeSupportsWaves() const { return (WaterBodyType == EWaterBodyType::Lake || WaterBodyType == EWaterBodyType::Ocean); }
	EWaterBodyQueryFlags CheckAndAjustQueryFlags(EWaterBodyQueryFlags InQueryFlags) const;
	/** Fills wave-related information at the given world position and for this water depth.
	- InPosition : water surface position at which to query the wave information
	- InWaterDepth : water depth at this location
	- bSimpleWaves : true for the simple version (faster computation, lesser accuracy, doesn't perturb the normal)
	- FWaveInfo : input/output : the structure's field must be initialized prior to the call (e.g. InOutWaveInfo.Normal is the unperturbed normal)
	Returns true if waves are supported, false otherwise. */
	bool GetWaveInfoAtPosition(const FVector& InPosition, float InWaterDepth, float InWaveReferenceTime, bool bInSimpleWaves, FWaveInfo& InOutWaveInfo) const;
	float GetWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime, FVector& OutNormal) const;
	/** Computes the raw wave perturbation of the water height only (simple version : faster computation) */
	float GetSimpleWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime) const;
	/** Computes the attenuation factor to apply to the raw wave perturbation. Attenuates : normal/wave height/max wave height. */
	float GetWaveAttenuationFactor(const FVector& InPosition, float InWaterDepth) const;
};

/* async structs end here */
