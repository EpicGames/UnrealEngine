// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "WaterBodyWeightmapSettings.generated.h"

class UTexture2D;

USTRUCT(BlueprintType)
struct FWaterBodyWeightmapSettings
{
	GENERATED_BODY()

	FWaterBodyWeightmapSettings()
		: FalloffWidth(512.f)
		, EdgeOffset(0)
		, ModulationTexture(nullptr)
		, TextureTiling(4.0f)
		, TextureInfluence(.5f)
		, Midpoint(0.5f)
		, FinalOpacity(1.0f)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category= WaterBodyWeightmapSettings, meta = (ClampMin = "0.1"))
	float FalloffWidth;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WaterBodyWeightmapSettings)
	float EdgeOffset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WaterBodyWeightmapSettings)
	TSoftObjectPtr<UTexture2D> ModulationTexture;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WaterBodyWeightmapSettings)
	float TextureTiling;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WaterBodyWeightmapSettings)
	float TextureInfluence;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WaterBodyWeightmapSettings)
	float Midpoint;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WaterBodyWeightmapSettings)
	float FinalOpacity;
};
