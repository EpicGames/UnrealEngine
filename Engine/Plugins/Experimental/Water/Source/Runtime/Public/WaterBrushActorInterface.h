// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "WaterBrushActorInterface.generated.h"

#define UE_API WATER_API

struct FWaterCurveSettings;
struct FWaterBodyHeightmapSettings;
struct FWaterBodyWeightmapSettings;
class UPrimitiveComponent;
enum ETextureRenderTargetFormat : int;

/** Dummy class needed to support Cast<IWaterBrushActorInterface>(Object). */
UINTERFACE(MinimalAPI)
class UWaterBrushActorInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

/**
 * Interface implemented by actors which can affect the water brush
 */
class IWaterBrushActorInterface
{
	GENERATED_IINTERFACE_BODY()
	
	/** 
	 * Returns true if this water actor is currently setup to modify the landscape :
	 */
	virtual bool AffectsLandscape() const = 0;

	/**
	 * Returns true if this water actor is currently affecting (i.e. being rendered by) a AWaterZone actor :
	 */
	virtual bool AffectsWaterMesh() const = 0;

	/**
	 * Returns true if this water actor could potentially be affecting (i.e. being rendered by) a AWaterZone actor :
	 */
	virtual bool CanEverAffectWaterMesh() const = 0;

	/**
	 * Returns true if this water actor could potentially be affecting the water info texture of an AWaterZone actor:
	 */
	virtual bool CanEverAffectWaterInfo() const = 0;

#if WITH_EDITOR
	/** 
	 * Returns the curve settings for this water actor
	 */
	virtual const FWaterCurveSettings& GetWaterCurveSettings() const = 0;
	
	/**
	 * Returns the landscape heightmap settings for this water actor
	 */
	virtual const FWaterBodyHeightmapSettings& GetWaterHeightmapSettings() const = 0;
	
	/**
	 * Returns the landscape weightmap settings, per layer, for this water actor
	 */
	virtual const TMap<FName, FWaterBodyWeightmapSettings>& GetLayerWeightmapSettings() const = 0;

	/** 
	 * Returns the format of the render target used to render this actor in the water brush
	 */
	virtual ETextureRenderTargetFormat GetBrushRenderTargetFormat() const = 0;

	/** 
	 * Returns an ordered list of components to render in the actor brush render pass (e.g. USplineMeshComponents for rivers)
	 */
	UE_API virtual TArray<UPrimitiveComponent*> GetBrushRenderableComponents() const;

	/**
	 * Returns the list of objects this actor depends on to render its brush (textures, materials...)
	 */
	virtual void GetBrushRenderDependencies(TSet<UObject*>& OutDependencies) const = 0;

	/** 
	 * Struct indicating what type of change occurrend on the water brush actor
	 */
	struct FWaterBrushActorChangedEventParams
	{
		FWaterBrushActorChangedEventParams(IWaterBrushActorInterface* InWaterBrushActor, const FPropertyChangedEvent& InPropertyChangedEvent = FPropertyChangedEvent(/*InProperty = */nullptr))
			: WaterBrushActor(InWaterBrushActor)
			, PropertyChangedEvent(InPropertyChangedEvent)
		{}

		/** The water brush actor that has changed */
		IWaterBrushActorInterface* WaterBrushActor = nullptr;

		/** Provides some additional context about how the water brush actor data has changed (property, type of change...) */
		FPropertyChangedEvent PropertyChangedEvent;

		/** Indicates that property related to the water brush actor's visual shape has changed */
		bool bShapeOrPositionChanged = false;

		/** Indicates that a property affecting the terrain weightmaps has changed */
		bool bWeightmapSettingsChanged = false;

		/** Indicates user initiated Parameter change */
		bool bUserTriggered = false;
	};

	/** 
	 * Event sent whenever a data change occurs on a water brush actor
	 */
	DECLARE_EVENT_OneParam(IWaterBrushActorInterface, FWaterBrushActorChangedEvent, const FWaterBrushActorChangedEventParams&);
	static UE_API FWaterBrushActorChangedEvent& GetOnWaterBrushActorChangedEvent();

	void BroadcastWaterBrushActorChangedEvent(const FWaterBrushActorChangedEventParams& InParams)
	{
		if (GIsEditor)
		{
			FWaterBrushActorChangedEvent& Event = GetOnWaterBrushActorChangedEvent();
			if (Event.IsBound())
			{
				Event.Broadcast(InParams);
			}
		}
	}
#endif // WITH_EDITOR
	
};

#undef UE_API
