// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterBrushEffects.generated.h"

class UTexture2D;
class UCurveFloat;

USTRUCT(BlueprintType)
struct FWaterBrushEffectBlurring
{
	GENERATED_BODY()

	FWaterBrushEffectBlurring()
		: bBlurShape(true)
		, Radius(2)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	bool bBlurShape;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	int32 Radius;
};


USTRUCT(BlueprintType)
struct FWaterBrushEffectCurlNoise
{
	GENERATED_BODY()

	FWaterBrushEffectCurlNoise()
		: Curl1Amount(0)
		, Curl2Amount(0)
		, Curl1Tiling(16.0)
		, Curl2Tiling(3.0)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float Curl1Amount;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float Curl2Amount;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float Curl1Tiling;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float Curl2Tiling;
};

USTRUCT(BlueprintType)
struct FWaterBrushEffectCurves
{
	GENERATED_BODY()

	FWaterBrushEffectCurves()
		: bUseCurveChannel(true)
		, ElevationCurveAsset(nullptr)
		, ChannelEdgeOffset(0.0f)
		, ChannelDepth(0.0f)
		, CurveRampWidth(512.0f)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	bool bUseCurveChannel;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	TObjectPtr<UCurveFloat> ElevationCurveAsset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float ChannelEdgeOffset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float ChannelDepth;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float CurveRampWidth;
};


USTRUCT(BlueprintType)
struct FWaterBrushEffectDisplacement
{
	GENERATED_BODY()

	FWaterBrushEffectDisplacement()
		: DisplacementHeight(0)
		, DisplacementTiling(0)
		, Texture(nullptr)
		, Midpoint(-128.0f)
		, Channel(0,0,0,1)
		, WeightmapInfluence(0.0f)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float DisplacementHeight;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float DisplacementTiling;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	TSoftObjectPtr<UTexture2D> Texture;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float Midpoint;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	FLinearColor Channel;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float WeightmapInfluence;
};


USTRUCT(BlueprintType)
struct FWaterBrushEffectSmoothBlending
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float InnerSmoothDistance = 0.01f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float OuterSmoothDistance = 0.01f;
};

USTRUCT(BlueprintType)
struct FWaterBrushEffectTerracing
{
	GENERATED_BODY()

	FWaterBrushEffectTerracing()
		: TerraceAlpha(0.0f)
		, TerraceSpacing(256.0f)
		, TerraceSmoothness(0.0f)
		, MaskLength(0.0f)
		, MaskStartOffset(0.0f)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float TerraceAlpha;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float TerraceSpacing;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float TerraceSmoothness;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float MaskLength;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	float MaskStartOffset;
};


USTRUCT(BlueprintType)
struct FWaterBrushEffects
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BrushEffects)
	FWaterBrushEffectBlurring Blurring;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FalloffSettings)
	FWaterBrushEffectCurlNoise CurlNoise;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FalloffSettings)
	FWaterBrushEffectDisplacement Displacement;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FalloffSettings)
	FWaterBrushEffectSmoothBlending SmoothBlending;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FalloffSettings)
	FWaterBrushEffectTerracing Terracing;
};
