// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterCurveSettings.generated.h"

class UCurveFloat;

USTRUCT(BlueprintType)
struct FWaterCurveSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Water)
	bool bUseCurveChannel = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Water)
	TObjectPtr<UCurveFloat> ElevationCurveAsset = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Water)
	float ChannelEdgeOffset = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Water)
	float ChannelDepth = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Water)
	float CurveRampWidth = 512.0f;
};
