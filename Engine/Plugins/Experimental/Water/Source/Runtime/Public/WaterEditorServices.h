// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

template <typename FuncType> class TFunctionRef;
class UClass;
class UPackage;
class UTexture2D;

#if WITH_EDITORONLY_DATA
class IWaterEditorServices
{
public:
	virtual ~IWaterEditorServices() = default;

	virtual void RegisterWaterActorSprite(UClass* InClass, UTexture2D* Texture) = 0;
	virtual UTexture2D* GetWaterActorSprite(UClass* InClass) const = 0;
	virtual UTexture2D* GetErrorSprite() const = 0;

	/**
	 * Mark a package as modified and needs to be dirtied manually by the user. This will be picked up by the editor module and notify the user that they need to resave the package manually.
	 * This can happen when a package has outdated derived data (such as a water body's saved WaterInfoMesh) is being rebuilt on load due to some external change that affects the derived data.
	 */
	virtual bool TryMarkPackageAsModified(UPackage* InPackage) = 0;
	virtual bool HasAnyModifiedPackages() const = 0;
	virtual void ForEachModifiedPackage(const TFunctionRef<bool(UPackage*)>& Func) const = 0;
	virtual void ClearModifiedPackages() = 0;
	virtual void DirtyAllModifiedPackages() = 0;

	virtual bool GetShouldUpdateWaterMeshDuringInteractiveChanges() const = 0;
};
#endif // WITH_EDITORONLY_DATA
