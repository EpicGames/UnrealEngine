// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterFalloffSettings.generated.h"

UENUM(BlueprintType)
enum class EWaterBrushFalloffMode : uint8
{
	Angle,
	Width,
};

USTRUCT(BlueprintType)
struct FWaterFalloffSettings
{
	GENERATED_BODY()
	FWaterFalloffSettings();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FalloffSettings)
	EWaterBrushFalloffMode FalloffMode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FalloffSettings, meta = (EditCondition = "FalloffMode == EWaterBrushFalloffMode::Angle"))
	float FalloffAngle;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FalloffSettings, meta = (ClampMin = "0.1", EditCondition = "FalloffMode == EWaterBrushFalloffMode::Width"))
	float FalloffWidth;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FalloffSettings)
	float EdgeOffset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FalloffSettings)
	float ZOffset;
};
