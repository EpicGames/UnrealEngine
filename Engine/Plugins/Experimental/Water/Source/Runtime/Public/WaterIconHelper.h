// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "HAL/Platform.h"

class UClass;

class AActor;
class UTexture2D;
class UBillboardComponent;

struct FWaterIconHelper
{
	/** Ensures a billboard component is created and added to the actor's components. */
	template <typename ActorType>
	static UBillboardComponent* EnsureSpriteComponentCreated(ActorType* Actor, const TCHAR* InIconTextureName)
	{
		return EnsureSpriteComponentCreated_Internal(Actor, ActorType::StaticClass(), InIconTextureName);
	}

	/** Updates the texture/scale/position of the actor's billboard component, if any */
	static WATER_API void UpdateSpriteComponent(AActor* Actor, UTexture2D* InTexture);

private:
	static WATER_API UBillboardComponent* EnsureSpriteComponentCreated_Internal(AActor* Actor, UClass* InClass, const TCHAR* InIconTextureName);
};

#endif
