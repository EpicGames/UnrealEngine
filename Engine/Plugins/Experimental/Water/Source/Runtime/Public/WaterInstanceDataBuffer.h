// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderingThread.h"

template <bool bWithWaterSelectionSupport>
class TWaterInstanceDataBuffers
{
public:
	static constexpr int32 NumBuffers = bWithWaterSelectionSupport ? 3 : 2;

	TWaterInstanceDataBuffers(int32 InInstanceCount)
	{
		ENQUEUE_RENDER_COMMAND(AllocateWaterInstanceDataBuffer)
		(
			[this, InInstanceCount](FRHICommandListImmediate& RHICmdList)
			{
				const int32 SizeInBytes = Align<int32>(InInstanceCount * sizeof(FVector4f), 4 * 1024);

				const FRHIBufferCreateDesc CreateDesc =
					FRHIBufferCreateDesc::CreateVertex(TEXT("WaterInstanceDataBuffers"), SizeInBytes)
					.AddUsage(EBufferUsageFlags::Dynamic)
					.DetermineInitialState();

				for (int32 i = 0; i < NumBuffers; ++i)
				{
					Buffer[i] = RHICmdList.CreateBuffer(CreateDesc);
					BufferMemory[i] = TArrayView<FVector4f>();
				}
			}
		);
	}

	~TWaterInstanceDataBuffers()
	{
		for (int32 i = 0; i < NumBuffers; ++i)
		{
			Buffer[i].SafeRelease();
		}
	}

	void Lock(FRHICommandListBase& RHICmdList, int32 InInstanceCount)
	{
		for (int32 i = 0; i < NumBuffers; ++i)
		{
			BufferMemory[i] = Lock(RHICmdList, InInstanceCount, i);
		}
	}

	void Unlock(FRHICommandListBase& RHICmdList)
	{
		for (int32 i = 0; i < NumBuffers; ++i)
		{
			Unlock(RHICmdList, i);
			BufferMemory[i] = TArrayView<FVector4f>();
		}
	}

	FBufferRHIRef GetBuffer(int32 InBufferID) const
	{
		return Buffer[InBufferID];
	}

	TArrayView<FVector4f> GetBufferMemory(int32 InBufferID) const
	{
		check(!BufferMemory[InBufferID].IsEmpty());
		return BufferMemory[InBufferID];
	}

private:
	TArrayView<FVector4f> Lock(FRHICommandListBase& RHICmdList, int32 InInstanceCount, int32 InBufferID)
	{
		uint32 SizeInBytes = InInstanceCount * sizeof(FVector4f);

		if (SizeInBytes > Buffer[InBufferID]->GetSize())
		{
			Buffer[InBufferID].SafeRelease();

			// Align size in to avoid reallocating for a few differences of instance count
			uint32 AlignedSizeInBytes = Align<uint32>(SizeInBytes, 4 * 1024);

			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex(TEXT("WaterInstanceDataBuffers"), AlignedSizeInBytes)
				.AddUsage(EBufferUsageFlags::Dynamic)
				.DetermineInitialState();

			Buffer[InBufferID] = RHICmdList.CreateBuffer(CreateDesc);
		}

		FVector4f* Data = reinterpret_cast<FVector4f*>(RHICmdList.LockBuffer(Buffer[InBufferID], 0, SizeInBytes, RLM_WriteOnly));
		return TArrayView<FVector4f>(Data, InInstanceCount);
	}

	void Unlock(FRHICommandListBase& RHICmdList, int32 InBufferID)
	{
		RHICmdList.UnlockBuffer(Buffer[InBufferID]);
	}

	FBufferRHIRef Buffer[NumBuffers];
	TArrayView<FVector4f> BufferMemory[NumBuffers];
};
