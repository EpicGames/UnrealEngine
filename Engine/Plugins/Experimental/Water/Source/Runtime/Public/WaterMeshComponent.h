// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MeshComponent.h"
#include "WaterQuadTree.h"
#include "WaterQuadTreeBuilder.h"
#include "WaterMeshComponent.generated.h"

#define UE_API WATER_API

struct FPSOPrecacheParams;

/**
 * Water Mesh Component responsible for generating and rendering a continuous water mesh on top of all the existing water body actors in the world
 * The component contains a quadtree which defines where there are water tiles. A function for traversing the quadtree and outputing a list of instance data for each tile to be rendered from a point of view is included
 */
UCLASS(MinimalAPI, ClassGroup = (Rendering, Water), hidecategories = (Object, Activation, "Components|Activation", Collision, Lighting, HLOD, Navigation, Replication, Input, MaterialParameters, TextureStreaming), editinlinenew)
class UWaterMeshComponent : public UMeshComponent
{
	GENERATED_BODY()

public:
	UE_API UWaterMeshComponent();

	//~ Begin UObject Interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UMeshComponent Interface
	virtual int32 GetNumMaterials() const override { return 0; }
	//~ End UMeshComponent Interface

	//~ Begin UPrimitiveComponent Interface
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	UE_API virtual void SetMaterial(int32 ElementIndex, UMaterialInterface* Material) override;
#if WITH_EDITOR
	UE_API virtual bool ShouldRenderSelected() const override;
#endif // WITH_EDITOR
	//~ End UPrimitiveComponent Interface

	//~ Begin INavRelevantInterface Interface
	virtual bool IsNavigationRelevant() const override { return false; }
	//~ End INavRelevantInterface Interface

	UE_API virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;

	UE_API void Update();

	/** Use this instead of GetMaterialRelevance, since this one will go over all materials from all tiles */
	UE_DEPRECATED(5.7, "Please use GetWaterMaterialRelevance with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	UE_API FMaterialRelevance GetWaterMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const;
	UE_API FMaterialRelevance GetWaterMaterialRelevance(EShaderPlatform InShaderPlatform) const;

	const FWaterQuadTreeBuilder& GetWaterQuadTreeBuilder() const { return WaterQuadTreeBuilder; }

	const TSet<TObjectPtr<UMaterialInterface>>& GetUsedMaterialsSet() const { return UsedMaterials; }

	void MarkWaterMeshGridDirty() { bNeedsRebuild = true; }

	int32 GetTessellationFactor() const { return FMath::Clamp(TessellationFactor + TessFactorBiasScalability, 1, 12); }

	float GetLODScale() const { return LODScale + LODScaleBiasScalability; }

	UE_API FIntPoint GetExtentInTiles() const;

	UE_DEPRECATED(5.5, "It is no longer possible to manually set the dynamic mesh center. This is controlled per view by the water view extension.")
	void SetDynamicWaterMeshCenter(const FVector2D& NewCenter) { }
	UE_DEPRECATED(5.5, "Dynamic water mesh center is now per-view and must be retrieved through the water view extension (water zone actor provides utilities to do this as well)")
	FVector2D GetDynamicWaterMeshCenter() const { return FVector2D::ZeroVector; }

	UE_API FVector2D GetGlobalWaterMeshCenter() const;

	UE_API bool IsLocalOnlyTessellationEnabled() const;

	UE_API void SetTileSize(float NewTileSize);
	float GetTileSize() const { return TileSize; }

	/** At above what density level a tile is allowed to force collapse even if not all leaf nodes in the subtree are present.
	 *	Collapsing will not occus if any child node in the subtree has different materials.
	 *	Setting this to -1 means no collapsing is allowed and the water mesh will always keep it's silhouette at any distance.
	 *	Setting this to 0 will allow every level to collapse
	 *	Setting this to something higher than the LODCount will have no effect
	 */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (ClampMin = "-1"))
	int32 ForceCollapseDensityLevel = -1;

	UPROPERTY(EditAnywhere, Category = "Rendering|FarDistance")
	TObjectPtr<UMaterialInterface> FarDistanceMaterial = nullptr;

	UPROPERTY(EditAnywhere, Category = "Rendering|FarDistance", meta = (ClampMin = "0"))
	float FarDistanceMeshExtent = 0.0f;

	UFUNCTION(BlueprintPure, Category = Rendering)
	bool IsEnabled() const { return bIsEnabled; }

private:
	//~ Begin USceneComponent Interface
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ Begin USceneComponent Interface

	/** Based on all water bodies in the scene, rebuild the water mesh */
	UE_API void RebuildWaterMesh(float InTileSize, const FIntPoint& InExtentInTiles);

	/** World size of the water tiles at LOD0. Multiply this with the ExtentInTiles to get the world extents of the system */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (ClampMin = "100", AllowPrivateAcces = "true"))
	float TileSize = 2400.0f;

	/** The current quad tree resolution derived from the extent of the water zone and the water mesh tile size (Extent / TileSize). */
	UPROPERTY(Transient, VisibleAnywhere, Category = Rendering)
	mutable FIntPoint QuadTreeResolution = FIntPoint::ZeroValue;

	FWaterQuadTreeBuilder WaterQuadTreeBuilder;

	/** Tiles containing water, stored in a quad tree */
	FWaterQuadTree WaterQuadTree;

	/** Unique list of materials used by this component */
	UPROPERTY(Transient, NonPIEDuplicateTransient, TextExportTransient)
	TSet<TObjectPtr<UMaterialInterface>> UsedMaterials;

	/** Maps from materials assigned to each water body to actually used MIDs. Persists across rebuilds in order to cache MIDs */
	UPROPERTY(Transient, NonPIEDuplicateTransient, TextExportTransient)
	TMap<TObjectPtr<UMaterialInterface>, TObjectPtr<UMaterialInstanceDynamic>> MaterialToMID;

	/** Forces the water mesh to always render the far mesh, regardless if there is an ocean or not.*/
	UPROPERTY(Category = "Rendering|FarDistance", EditAnywhere)
	bool bUseFarMeshWithoutOcean = false;

	/** Absolute world space height of the far mesh when there is no ocean.*/
	UPROPERTY(Category = "Rendering|FarDistance", EditAnywhere)
	double FarDistanceMeshHeightWithoutOcean = 0.0;

	/** Dirty flag which will make sure the water mesh is updated properly */
	bool bNeedsRebuild = true;

	/** If the system is enabled */
	bool bIsEnabled = false;

	/** Cached CVarWaterMeshLODCountBias to detect changes in scalability */
	int32 LODCountBiasScalability = 0;
	
	/** Cached CVarWaterMeshTessFactorBias to detect changes in scalability */
	int32 TessFactorBiasScalability = 0;

	/** Cached CVarWaterMeshLODScaleBias to detect changes in scalability */
	float LODScaleBiasScalability = 0.0f;

	/** Highest tessellation factor of a water tile. Max number of verts on the side of a tile will be (2^TessellationFactor)+1)  */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (ClampMin = "1", ClampMax = "12"))
	int32 TessellationFactor = 6;

	/** World scale of the concentric LODs */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (ClampMin = "0.5"))
	float LODScale = 1.0f;

#if WITH_EDITOR
	//~ Begin USceneComponent Interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ Begin USceneComponent Interface
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FIntPoint ExtentInTiles_DEPRECATED = FIntPoint(64, 64);
#endif // WITH_EDITORONLY_DATA
};

#undef UE_API
