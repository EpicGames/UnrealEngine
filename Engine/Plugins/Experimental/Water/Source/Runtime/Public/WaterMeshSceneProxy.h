// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialInterface.h"
#include "PrimitiveSceneProxy.h"
#include "Materials/MaterialRelevance.h"
#include "WaterQuadTree.h"
#include "WaterQuadTreeBuilder.h"
#include "WaterVertexFactory.h"
#include "RayTracingGeometry.h"
#include "RenderGraphResources.h"
#include "WaterQuadTreeGPU.h"

class FMeshElementCollector;

class UWaterMeshComponent;

using FWaterInstanceDataBuffersType = TWaterInstanceDataBuffers<WITH_WATER_SELECTION_SUPPORT>;
using FWaterMeshUserDataBuffersType = TWaterMeshUserDataBuffers<WITH_WATER_SELECTION_SUPPORT>;
using FWaterMeshUserDataType = TWaterMeshUserData<WITH_WATER_SELECTION_SUPPORT>;

/** Set of quadtree related constants that do not change over the lifetime of the FWaterMeshSceneProxy and are shared by all quadtrees owned by it. */
struct FWaterQuadTreeConstants
{
	/** Scale of the concentric LOD squares  */
	float LODScale = -1.0f;

	/** Number of quads per side of a water quad tree tile at LOD0 */
	int32 NumQuadsLOD0 = 0;

	int32 NumQuadsPerIndirectDrawTile = 0;

	/** Number of densities (same as number of grid index/vertex buffers) */
	int32 DensityCount = 0;

	int32 ForceCollapseDensityLevel = TNumericLimits<int32>::Max();
};

/** A water quadtree instance owned by FWaterMeshSceneProxy and associated with a certain view. In splitscreen, each player should get their own FViewWaterQuadTree. */
class FViewWaterQuadTree
{
public:
	struct FWaterLODParams
	{
		int32 LowestLOD;
		float HeightLODFactor;
		float WaterHeightForLOD;
	};

	struct FUserDataAndIndirectArgs
	{
		TStaticArray<FWaterMeshUserDataType*, 3> UserData = {};
		TRefCountPtr<FRDGPooledBuffer> IndirectArgs = nullptr;
	};

	// Rebuilds the quadtree at the specified position.
	void Update(const FWaterQuadTreeBuilder& Builder, const FVector2D& CenterPosition);
	
	// Traverses the GPU quadtree to build indirect draw calls and associated buffers. May also initialize the GPU quadtree if it wasn't initialized already.
	void TraverseGPUQuadTree(FRDGBuilder& GraphBuilder, bool bDepthBufferIsPopulated);
	
	// Allocates transient GPU resources to build indirect draw calls for the GPU quadtree and returns parameters needed by the water vertex factory to draw the quadtree.
	FUserDataAndIndirectArgs PrepareGPUQuadTreeForRendering(const TArray<const FSceneView*>& Views, uint32 VisibilityMap, FMeshElementCollector& Collector, const FWaterQuadTreeConstants& QuadTreeConstants, const TArrayView<const EWaterMeshRenderGroupType> & BatchRenderGroups, FRHICommandListBase& RHICmdList) const;
	
	// Evaluates the CPU quadtree at the given position and returns parameters used for CPU quadtree draw call generation.
	FWaterLODParams GetWaterLODParams(const FVector& Position, float LODScale) const;
	
	const FWaterQuadTree& GetWaterQuadTree() const { return WaterQuadTree; }
	FWaterInstanceDataBuffersType* GetWaterInstanceDataBuffers() const { return WaterInstanceDataBuffers.Get(); }
	FWaterMeshUserDataBuffersType* GetWaterMeshUserDataBuffers() const { return WaterMeshUserDataBuffers.Get(); }
	double GetMinHeight() const { return WaterQuadTreeMinHeight; }
	double GetMaxHeight() const { return WaterQuadTreeMaxHeight; }

private:

	/** Tiles containing water, stored in a quad tree. */
	FWaterQuadTree WaterQuadTree;

	/** GPU quad tree instance. Only initialized and used if WaterQuadTree.IsGPUQuadTree() is true. */
	FWaterQuadTreeGPU QuadTreeGPU;

	/** Unique Instance data buffer shared accross water batch draw calls */
	TUniquePtr<FWaterInstanceDataBuffersType> WaterInstanceDataBuffers = nullptr;

	/** Per-"water render group" user data (the number of groups might vary depending on whether we're in the editor or not) */
	TUniquePtr<FWaterMeshUserDataBuffersType> WaterMeshUserDataBuffers = nullptr;

	/** Vertical extent of the quadtree. */
	double WaterQuadTreeMinHeight = DBL_MAX;
	double WaterQuadTreeMaxHeight = -DBL_MAX;

	mutable FWaterQuadTreeGPU::FTraverseParams WaterQuadTreeGPUTraverseParams;
	mutable bool bNeedToTraverseGPUQuadTree = false;

	/** Initializes the GPU quad tree. */
	void BuildGPUQuadTree(FRDGBuilder& GraphBuilder);
};


/** Water mesh scene proxy */

class FWaterMeshSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FWaterMeshSceneProxy(UWaterMeshComponent* Component);

	virtual ~FWaterMeshSceneProxy();

	virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;

	virtual void DestroyRenderThreadResources() override;

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;

	virtual const TArray<FBoxSphereBounds>* GetOcclusionQueries(const FSceneView* View) const override;

	virtual void AcceptOcclusionResults(const FSceneView* View, TArray<bool>* Results, int32 ResultsStart, int32 NumResults) override;

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;

	virtual bool HasSubprimitiveOcclusionQueries() const override;

	virtual bool CanBeOccluded() const override
	{
		return !MaterialRelevance.bDisableDepthTest;
	}

	virtual uint32 GetMemoryFootprint() const override
	{
		return(sizeof(*this) + GetAllocatedSize());
	}

	uint32 GetAllocatedSize() const
	{
		return(FPrimitiveSceneProxy::GetAllocatedSize() + (WaterVertexFactories.GetAllocatedSize() + WaterVertexFactories.Num() * sizeof(FWaterVertexFactoryType)) + ViewQuadTrees.GetAllocatedSize());
	}

#if WITH_WATER_SELECTION_SUPPORT
	virtual HHitProxy* CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;
#endif // WITH_WATER_SELECTION_SUPPORT

	// At runtime, we only ever need one version of the vertex factory : with selection support (editor) or without : 
	using FWaterVertexFactoryType = TWaterVertexFactory<WITH_WATER_SELECTION_SUPPORT, EWaterVertexFactoryDrawMode::NonIndirect>;
	using FWaterVertexFactoryIndirectDrawType = TWaterVertexFactory<WITH_WATER_SELECTION_SUPPORT, EWaterVertexFactoryDrawMode::Indirect>;
	using FWaterVertexFactoryIndirectDrawISRType = TWaterVertexFactory<WITH_WATER_SELECTION_SUPPORT, EWaterVertexFactoryDrawMode::IndirectInstancedStereo>;

#if RHI_RAYTRACING
	virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override final;
	virtual bool HasRayTracingRepresentation() const override { return true; }
	virtual bool IsRayTracingRelevant() const override { return true; }
#endif

	// Creates and initializes a new quadtree centered around CenterPosition and associated with the given key. Returns true if the quadtree was created and false if it already exists.
	bool CreateViewWaterQuadTree(int32 Key, const FVector2D& CenterPosition);
	// Updates an existing quadtree by reconstructing it at a new CenterPosition. Returns true if the quadtree exists and was updated and false otherwise.
	bool UpdateViewWaterQuadTree(int32 Key, const FVector2D& CenterPosition);
	// Destroys the quadtree associated with Key.
	void DestroyViewWaterQuadTree(int32 Key);

	int32 FindBestQuadTreeForViewLocation(const FVector2D& ViewPosition2D) const;

private:

#if RHI_RAYTRACING
	struct FRayTracingWaterData
	{
		FRayTracingGeometry Geometry;
		FRWBuffer DynamicVertexBuffer;
	};
#endif

	struct FOcclusionCullingResults
	{
		uint32 FrameNumber;
		TArray<bool> Results;
	};

	TMap<int32, FViewWaterQuadTree> ViewQuadTrees;

	FMaterialRelevance MaterialRelevance;

	// One vertex factory per LOD. Only used for CPU driven water quadtree rendering.
	TArray<FWaterVertexFactoryType*> WaterVertexFactories;
	FWaterVertexFactoryIndirectDrawType* WaterVertexFactoryIndirectDraw = nullptr;
	FWaterVertexFactoryIndirectDrawISRType* WaterVertexFactoryIndirectDrawISR = nullptr;

	FWaterQuadTreeBuilder WaterQuadTreeBuilder;
	FWaterQuadTreeConstants WaterQuadTreeConstants;
	// If this is true, then this proxy can manage multiple local quadtrees, potentially associated with different views.
	bool bIsLocalOnlyTessellationEnabled = false;

	mutable int32 HistoricalMaxViewInstanceCount = 0;

#if RHI_RAYTRACING
	// Per density array of ray tracing geometries.
	TArray<TIndirectArray<FRayTracingWaterData>> RayTracingWaterData;	
#endif

	
	// CPU-driven occlusion culling related members
	TArray<FBoxSphereBounds> OcclusionCullingBounds;
	TArray<FBoxSphereBounds> EmptyOcclusionCullingBounds;
	TMap<uint32, FOcclusionCullingResults> OcclusionResults;
	UE::FMutex OcclusionResultsMutex;
	int32 OcclusionResultsFarMeshOffset = INT32_MAX;
	uint32 SceneProxyCreatedFrameNumberRenderThread = INDEX_NONE;

	bool HasWaterData() const;
	TArray<EWaterMeshRenderGroupType, TInlineAllocator<FWaterVertexFactoryType::NumRenderGroups>> GetBatchRenderGroups(const TArray<const FSceneView*>& Views, uint32 VisibilityMap) const;
	uint32 GetWireframeVisibilityMapAndMaterial(const TArray<const FSceneView*>& Views, uint32 VisibilityMap, FMeshElementCollector& Collector, class FColoredMaterialRenderProxy*& OutMaterialInstance) const;
	int32 FindBestQuadTreeForView(const FSceneView* View) const;
	TArray<int32, TInlineAllocator<8>> GetViewToQuadTreeMapping(const TArray<const FSceneView*>& Views, uint32 VisibilityMap) const;
#if RHI_RAYTRACING
	void SetupRayTracingInstances(FRHICommandListBase& RHICmdList, int32 NumInstances, uint32 DensityIndex);
#endif
};
