// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

class IWaterEditorServices;

//////////////////////////////////////////////////////////////////////////
// IWaterModuleInterface

WATER_API DECLARE_LOG_CATEGORY_EXTERN(LogWater, Log, All);


class IWaterModuleInterface : public IModuleInterface
{
public:
#if WITH_EDITOR
	virtual void SetWaterEditorServices(IWaterEditorServices* InWaterEditorServices) = 0;
	virtual IWaterEditorServices* GetWaterEditorServices() const = 0;
#endif // WITH_EDITOR
};
