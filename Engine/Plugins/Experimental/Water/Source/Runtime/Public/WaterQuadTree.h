// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConvexVolume.h"
#include "HitProxies.h"
#include "Math/Box.h"
#include "Math/Box2D.h"

class FMaterialRenderProxy;
class UMaterialInterface;
class HHitProxy;

/** Render data per water body */
struct FWaterBodyRenderData
{
	bool IsRiver() const { return WaterBodyType == 0; }
	bool IsLake() const { return WaterBodyType == 1; }
	bool IsOcean() const { return WaterBodyType == 2; }

	/** The standard material to be used for this water body */
	FMaterialRenderProxy* Material = nullptr;

	/** Transition materials (Only set up for river water bodies) */
	FMaterialRenderProxy* RiverToLakeMaterial = nullptr;
	FMaterialRenderProxy* RiverToOceanMaterial = nullptr;

	/** World Z position of the waterbody, this is where the tiles for this water body will be rendered*/
	double SurfaceBaseHeight = 0.0;

	/** Z bounds of the water body */
	double BoundsMinZ = DBL_MAX;
	double BoundsMaxZ = -DBL_MAX;

	/** Maximum Z displacement due to waves */
	float MaxWaveHeight = 0.0f;

	/** Render priority. If two water bodies overlap, this will decide which water body is used for a tile */
	int16 Priority = TNumericLimits<int16>::Min();

	/** Offset into the wave data buffer on GPU */
	int16 WaterBodyIndex = INDEX_NONE;

	int16 MaterialIndex = INDEX_NONE;
	int16 RiverToLakeMaterialIndex = INDEX_NONE;
	int16 RiverToOceanMaterialIndex = INDEX_NONE;

	/** Water body type. River, Lake or Ocean, defaults to an invalid type */
	int8 WaterBodyType = -1;

#if WITH_WATER_SELECTION_SUPPORT
	/** Hit proxy for this waterbody */
	TRefCountPtr<HHitProxy> HitProxy = nullptr;

	/** Whether the water body actor is selected or not */
	bool bWaterBodySelected = false;
#endif // WITH_WATER_SELECTION_SUPPORT

	bool operator==(const FWaterBodyRenderData& Other) const
	{
		return	Material				== Other.Material &&
				RiverToLakeMaterial		== Other.RiverToLakeMaterial &&
				RiverToOceanMaterial	== Other.RiverToOceanMaterial &&
				SurfaceBaseHeight		== Other.SurfaceBaseHeight &&
				Priority				== Other.Priority &&
				WaterBodyIndex			== Other.WaterBodyIndex &&
				WaterBodyType			== Other.WaterBodyType
#if WITH_WATER_SELECTION_SUPPORT
				&& HitProxy == Other.HitProxy
				&& bWaterBodySelected == Other.bWaterBodySelected
#endif // WITH_WATER_SELECTION_SUPPORT
				; 
	}
};

struct FWaterBodyQuadTreeRasterInfo
{
	FTransform LocalToWorld = FTransform::Identity;
	const class FStaticMeshRenderData* RenderData = nullptr;
	uint32 WaterBodyRenderDataIndex = INDEX_NONE;
	uint8 Priority = 0;
	bool bIsRiver = false;
};

struct FWaterQuadTree 
{
	enum { INVALID_PARENT = 0xFFFFFFF };

	static constexpr int32 NumStreams = WITH_WATER_SELECTION_SUPPORT ? 3 : 2;

	struct FStagingInstanceData
	{
		int32 BucketIndex;
		FVector4f Data[NumStreams];
	};

	/** Output of the quadtree when asking to traverse it for visible water tiles */
	struct FTraversalOutput
	{
		TArray<int32> BucketInstanceCounts;

		/**
		 *	This is the raw data that will be bound for the draw call through a buffer. Stored in buckets sorted by material and density level
		 *	Each instance contains:
		 *	[0] (xyz: translate, w: wave param index)
		 *	[1] (x: (bit 0-7)lod level, (bit 8)bShouldMorph, y: HeightMorph zw: scale)
		 *  [2] (editor only, HitProxy ID of the associated WaterBody actor)
		 */
		TArray<FStagingInstanceData> StagingInstanceData;

		/** Number of added instances */
		int32 InstanceCount = 0;
	};

	/** Output of the quadtree when asking to traverse it for visible water tiles */
	struct FTraversalDesc
	{
		const TArray<bool>* OcclusionCullingResults = nullptr;
		int32 OcclusionCullingFarMeshOffset = 0;
		int32 LowestLOD = 0;
		int32 LODCount = 0;
		int32 DensityCount = 0;
		float HeightMorph = 0.0f;
		int32 ForceCollapseDensityLevel = TNumericLimits<int32>::Max();
		float LODScale = 1.0;
		FVector ObserverPosition = FVector::ZeroVector;
		FVector PreViewTranslation = FVector::ZeroVector;
		FConvexVolume Frustum;
		FBox2D WaterInfoBounds = FBox2D(ForceInitToZero);
		bool bLODMorphingEnabled = true;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// Debug
		int32 DebugShowTile = 0;
		class FPrimitiveDrawInterface* DebugPDI = nullptr;
		bool bDebugDrawIntoForeground = false;
#endif
	};

	/** 
	 *	Initialize the tree. This will unlock the tree for node insertion using AddWaterTilesInsideBounds(...). 
	 *	Tree must be locked before traversal, see Lock(). 
	 */
	void InitTree(const FBox2D& InBounds, float InTileSize, FIntPoint InExtentInTiles, bool bInIsGPUQuadTree);

	/** Unlock to make it read-only. This will optionally prune the node array to remove redundant nodes, nodes that can be implicitly traversed */
	void Unlock(bool bPruneRedundantNodes);

	/** Add tiles that intersect InBounds recursively from the root node. Tree must be unlocked. Typically called on Game Thread */
	void AddWaterTilesInsideBounds(const FBox& InBounds, uint32 InWaterBodyIndex);

	/** Add Ocean by giving a closed spline that represents the land mass for this ocean. */
	void AddOcean(const TArray<FVector2D>& InPoly, const FBox& InOceanBounds, uint32 InWaterBodyIndex);

	/** Add Lake by giving a closed spline that represents the lake */
	void AddLake(const TArray<FVector2D>& InPoly, const FBox& InLakeBounds, uint32 InWaterBodyIndex);

	/** Add an automatically generated mesh (8 quads) skirt around InInnerRegion which extends out InFarDistanceMeshExtent, is placed at Z value InFarDistanceMeshHeight and is rendered using InFarMeshMaterial */
	void AddFarMesh(FMaterialRenderProxy* InFarMeshMaterial, const FBox2D& InInnerRegion, double InFarDistanceMeshExtent, double InFarDistanceMeshHeight);

	/** Assign an index to each material */
	void BuildMaterialIndices();

	/** Add water body render data to this tree. Returns the index in the array. Use this index to add tiles with this water body to the tree, see AddWaterTilesInsideBounds(..) */
	uint32 AddWaterBodyRenderData(const FWaterBodyRenderData& InWaterBodyRenderData) { return NodeData.WaterBodyRenderData.Add(InWaterBodyRenderData); }

	void AddWaterBodyRasterInfo(const FWaterBodyQuadTreeRasterInfo& InWaterBodyRasterInfo) { WaterBodyRasterInfos.Add(InWaterBodyRasterInfo); }

	/** Get bounds of the root node if there is one, otherwise some default box */
	FBox GetBounds() const { return NodeData.Nodes.Num() > 0 ? NodeData.Nodes[0].Bounds : FBox(-FVector::OneVector, FVector::OneVector); }

	/** Get bounds of the root node if there is one (including far mesh), otherwise some default box */
	FBox GetBoundsIncludingFarMesh() const { return GetBounds() + FarMeshData.FarMeshBounds; }
	
	/** Return the 2D region containing water tiles. Tiles can not be generated outside of this region */
	FBox2D GetTileRegion() const { return TileRegion; }

	/** Build the instance data needed to render the water tiles from a given point of view. Typically called on Render Thread for rendering */
	void BuildWaterTileInstanceData(const FTraversalDesc& InTraversalDesc, FTraversalOutput& Output) const;

	/** Bilinear interpolation between four neighboring base height samples around InWorldLocationXY. The samples are done on the leaf node grid resolution. Returns true if all 4 samples were taken in valid nodes */
	bool QueryInterpolatedTileBaseHeightAtLocation(const FVector2D& InWorldLocationXY, float& OutHeight) const;

	/** Walks down the tree and returns the tile height at InWorldLocationXY in OutWorldHeight. Returns true if the query hits an exact solution (either leaf tile or a complete subtree parent), otherwise false. */
	bool QueryTileBaseHeightAtLocation(const FVector2D& InWorldLocationXY, float& OutWorldHeight) const;

	/** Walks down the tree and returns the tile bounds at InWorldLocationXY in OutWorldBounds. Returns true if the query finds a leaf tile to return, otherwise false. */
	bool QueryTileBoundsAtLocation(const FVector2D& InWorldLocationXY, FBox& OutWorldBounds) const;

	/** Total node count in the tree, including inner nodes, root node and leaf nodes */
	int32 GetNodeCount() const { return NodeData.Nodes.Num(); }

	/** Get cached leaf world size of one side of the tile (same applies for X and Y) */
	float GetLeafSize() const { return LeafSize; }

	/** Number of maximum leaf nodes on one side, same applies for X and Y. (Maximum number of total leaf nodes in this tree is LeafSideCount*LeafSideCount) */
	int32 GetMaxLeafCount() const { return MaxLeafCount; }

	/** Max depth of the tree */
	int32 GetTreeDepth() const { return TreeDepth; }

	const TArray<FMaterialRenderProxy*>& GetWaterMaterials() const { return WaterMaterials; }

	const TArray<FWaterBodyQuadTreeRasterInfo>& GetWaterBodyRasterInfos() const { return WaterBodyRasterInfos; }

	const TArray<FWaterBodyRenderData>& GetWaterBodyRenderData() const { return NodeData.WaterBodyRenderData; }

	FIntPoint GetResolution() const { return ExtentInTiles * 2; }

	bool IsGPUQuadTree() const { return bIsGPUQuadTree; }

	/** Calculate the world distance to a LOD */
	static float GetLODDistance(int32 InLODLevel, float InLODScale) { return FMath::Pow(2.0f, (float)(InLODLevel + 1)) * InLODScale; }

	/** Total memory dynamically allocated by this object */
	uint32 GetAllocatedSize() const { return NodeData.GetAllocatedSize() + WaterMaterials.GetAllocatedSize() + WaterBodyRasterInfos.GetAllocatedSize() + BreadthFirstOrder.GetAllocatedSize() + FarMeshData.GetAllocatedSize(); }

#if WITH_WATER_SELECTION_SUPPORT
	/** Obtain all possible hit proxies (proxies of all the water bodies) */
	void GatherHitProxies(TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) const;
#endif // WITH_WATER_SELECTION_SUPPORT

	TArray<FBoxSphereBounds> ComputeNodeBounds(int32 MaxNumBounds, float OcclusionCullExpandBoundsAmountXY, bool bIncludeFarMeshTiles, int32* OutFarMeshOffset) const;

private:
	struct FNodeData;

	/** Private recursion function, see AddOcean(...) */
	void AddOceanRecursive(const TArray<FVector2D>& InPoly, const FBox2D& InBox, const FVector2D& InZBounds, bool HSplit, int32 InDepth, uint32 InWaterBodyIndex);

	/** Private recursion function, see AddLake(...) */
	void AddLakeRecursive(const TArray<FVector2D>& InPoly, const FBox2D& InBox, const FVector2D& InZBounds, bool HSplit, int32 InDepth, uint32 InWaterBodyIndex);

	struct FNode
	{

		FNode() : WaterBodyIndex(0), TransitionWaterBodyIndex(0), ParentIndex(INVALID_PARENT), HasCompleteSubtree(1), IsSubtreeSameWaterBody(1), HasMaterial(0) {}

		/** If this node is allowed to be rendered, it means it can be rendered in place of all leaf nodes in its subtree. */
		bool CanRender(int32 InDensityLevel, int32 InForceCollapseDensityLevel, const FWaterBodyRenderData& InWaterBodyRenderData) const;

		/** Add instance for rendering this node*/
		void AddNodeForRender(const FNodeData& InNodeData, const FWaterBodyRenderData& InWaterBodyRenderData, int32 InDensityLevel, int32 InLODLevel, const FTraversalDesc& InTraversalDesc, FTraversalOutput& Output) const;

		/** Recursive function to traverse down to the appropriate density level. The LODLevel is constant here since this function is only called on tiles that are fully inside a LOD range */
		void SelectLODRefinement(const FNodeData& InNodeData, int32 InDensityLevel, int32 InLODLevel, const FTraversalDesc& InTraversalDesc, FTraversalOutput& Output) const;

		/** Recursive function to select nodes visible from the current point of view */
		void SelectLOD(const FNodeData& InNodeData, int32 InLODLevel, const FTraversalDesc& InTraversalDesc, FTraversalOutput& Output) const;

		/** Recursive function to select nodes visible from the current point of view within an active bounding box */
		void SelectLODWithinBounds(const FNodeData& InNodeData, int32 InLODLevel, const FTraversalDesc& InTraversalDesc, FTraversalOutput& Output) const;

		/** Recursive function to query the height(prior to any displacement) at a given location, return false if no height could be found */
		bool QueryBaseHeightAtLocation(const FNodeData& InNodeData, const FVector2D& InWorldLocationXY, float& OutHeight) const;

		/** Recursive function to query the bounds of a tile at a given location, return false if no leaf node could be found */
		bool QueryBoundsAtLocation(const FNodeData& InNodeData, const FVector2D& InWorldLocationXY, FBox& OutHeight) const;

		/** Add nodes that intersect InMeshBounds. LODLevel is the current level. This is the only method used to generate the tree */
		void AddNodes(FNodeData& InNodeData, const FBox& InMeshBounds, const FBox& InWaterBodyBounds, uint32 InWaterBodyIndex, int32 InLODLevel, uint32 InParentIndex);
		
		/** Check if all conditions are met to potentially allow this and another node to render as one */
		bool CanMerge(const FNode& Other) const { return Other.WaterBodyIndex == WaterBodyIndex && Other.TransitionWaterBodyIndex == TransitionWaterBodyIndex; }

		/** World bounds */
		FBox Bounds = FBox(-FVector::OneVector, FVector::OneVector);

		/** Index into the water body render data array on the tree. If this is not a leaf node, this will represent the waterbody */
		uint32 WaterBodyIndex : 16;

		/** Index to the water body that this tile possibly transitions to */
		uint32 TransitionWaterBodyIndex : 16;

		/** Index to parent */
		uint32 ParentIndex : 28;

		/** If all 4 child nodes have a full set of leaf nodes (each descentant has 4 children all the way down) */
		uint32 HasCompleteSubtree : 1;

		/** If all descendant nodes are from the same waterbody. We can safely collapse this even if HasCompleteSubtree is false */
		uint32 IsSubtreeSameWaterBody : 1;

		/** Cached value to avoid having to visit this node's FWaterBodyRenderData */
		uint32 HasMaterial : 1;

		// 1 spare bits here in the bit field with ParentIndex

		/** Children, 0 means invalid */
		uint32 Children[4] = { 0, 0, 0, 0 };
		uint32 BreadthFirstIndex = INDEX_NONE;
	};

	int32 TreeDepth = 0;

	float LeafSize = 0.0f;
	int32 MaxLeafCount = 0;
	FIntPoint ExtentInTiles = FIntPoint::ZeroValue;

	/** 
	 * This is the bounding box that describes where the water tiles can be generated. This will be same or smaller than the root node bounds of the tree and might not be square.
	 * Shares the same Min position with the root node bounds, so this region is a subset of the root node bounds.
	 */
	FBox2D TileRegion;

	/** Contains all the node data of the quad tree. Root node is Nodes[0] */
	struct FNodeData
	{
		/** Storage for all nodes in the tree. Each node has 4 indices into this array to locate its children */
		TArray<FNode> Nodes;

		/** Render data for all water bodies in this tree, indexed by the nodes */
		TArray<FWaterBodyRenderData> WaterBodyRenderData;

		/** Total memory dynamically allocated by this object */
		uint32 GetAllocatedSize() const { return Nodes.GetAllocatedSize() + WaterBodyRenderData.GetAllocatedSize(); }
	} NodeData;

	TArray<FMaterialRenderProxy*> WaterMaterials;
	TArray<FWaterBodyQuadTreeRasterInfo> WaterBodyRasterInfos;
	
/** Node indices ordered by breadth first ordering */
	TArray<int32> BreadthFirstOrder;

	/** Contains everything needed to render the far mesh. This data lives outside the quadtree structure itself */
	struct FFarMeshData
	{
		struct FFarMeshInstanceData
		{
			FVector WorldPosition = FVector::ZeroVector;
			FVector2f Scale = FVector2f(1.0f, 1.0f);
		};

		/** Stored data for rendering all far mesh instances (8 or them if used). This is built once when the quadtree is built and then read and transformed each time the quadtree is traversed */
		TArray<FFarMeshInstanceData> InstanceData;

		/** Material for the Far Distance Mesh, its material render proxy will be cached in WaterMaterials when BuildMaterialIndices is called */
		FMaterialRenderProxy* Material = nullptr;

		void Clear()
		{
			InstanceData.Empty();
			Material = nullptr;
			MaterialIndex = INDEX_NONE;
			FarMeshBounds.Init();
		}

		/** Total memory dynamically allocated by this object */
		uint32 GetAllocatedSize() const { return InstanceData.GetAllocatedSize(); }

		/** Cached material index */
		int16 MaterialIndex = INDEX_NONE;

		/** FarMesh bounds */
		FBox FarMeshBounds;

	} FarMeshData;

	/** If true, the tree may not change */
	bool bIsReadOnly = true;

	bool bIsGPUQuadTree = false;
};
