// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

class HHitProxy;
class FStaticMeshRenderData;
class UMaterialInterface;
struct FWaterQuadTree;
enum class EWaterBodyType : uint8;

class FWaterQuadTreeBuilder
{
public:

	struct FWaterBody
	{
		const UMaterialInterface* Material = nullptr;
		const UMaterialInterface* RiverToLakeMaterial = nullptr;
		const UMaterialInterface* RiverToOceanMaterial = nullptr;
		FStaticMeshRenderData* StaticMeshRenderData = nullptr;
		FTransform LocalToWorld;
		FBoxSphereBounds Bounds = FBoxSphereBounds();
		int32 OverlapMaterialPriority = 0;
		EWaterBodyType Type = EWaterBodyType(0);
		int32 WaterBodyIndex = INDEX_NONE;
		float SurfaceBaseHeight = 0.0f;
		float MaxWaveHeight = 0.0f;
		FBox PolygonBounds = FBox();
		TArray<TArray<FVector2D>> PolygonBatches;
		TArray<FBox> RiverBoxes;
#if WITH_WATER_SELECTION_SUPPORT
		TRefCountPtr<HHitProxy> HitProxy = nullptr;
		bool bWaterBodySelected = false;
#endif // WITH_WATER_SELECTION_SUPPORT
	};

	void Init(const FBox2D& InWaterZoneBounds2D, const FIntPoint& InExtentInTiles, float InTileSize, const UMaterialInterface* InFarDistanceMaterial, float InFarDistanceMeshExtent, double InDefaultFarDistanceMeshHeight, bool bInUseFarMeshWithoutOcean, bool bInIsGPUQuadTree);
	void AddWaterBody(const FWaterBody& WaterBody);
	bool BuildWaterQuadTree(FWaterQuadTree& WaterQuadTree, const FVector2D& GridPosition) const;
#if WITH_WATER_SELECTION_SUPPORT
	void GatherHitProxies(TArray<TRefCountPtr<HHitProxy>>& OutHitProxies) const;
#endif // WITH_WATER_SELECTION_SUPPORT
	bool IsGPUQuadTree() const;
	float GetLeafSize() const;
	int32 GetMaxLeafCount() const;
	int32 GetTreeDepth() const;
	FIntPoint GetResolution() const;

private:
	TArray<FWaterBody> WaterBodies;
	FBox2D WaterZoneBounds2D = FBox2D();
	FIntPoint ExtentInTiles = FIntPoint::ZeroValue;
	float TileSize = 0.0f;
	int32 TreeDepth = 0;
	const UMaterialInterface* FarDistanceMaterial = nullptr;
	float FarDistanceMeshExtent = 0.0f;
	double DefaultFarDistanceMeshHeight = 0.0;
	bool bUseFarMeshWithoutOcean = false;
	bool bIsGPUQuadTree = false;
};
