// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Templates/SubclassOf.h"
#include "WaterRuntimeSettings.generated.h"

#define UE_API WATER_API

enum ECollisionChannel : int;

class UMaterialInterface;
class UMaterialParameterCollection;
class UWaterBodyComponent;
class UWaterBodyRiverComponent;
class UWaterBodyLakeComponent;
class UWaterBodyOceanComponent;
class UWaterBodyCustomComponent;

/**
 * Implements the runtime settings for the Water plugin.
 */
UCLASS(MinimalAPI, config = Engine, defaultconfig, meta=(DisplayName="Water"))
class UWaterRuntimeSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UWaterRuntimeSettings();

	UE_API virtual FName GetCategoryName() const;

	FName GetDefaultWaterCollisionProfileName() const { return DefaultWaterCollisionProfileName; }

	UE_API UMaterialInterface* GetDefaultWaterInfoMaterial() const;

	UE_DEPRECATED(5.4, "Seting moved to WaterBodyActor")
	UE_API TSubclassOf<UWaterBodyRiverComponent> GetWaterBodyRiverComponentClass() const;

	UE_DEPRECATED(5.4, "Seting moved to WaterBodyActor")
	UE_API TSubclassOf<UWaterBodyLakeComponent> GetWaterBodyLakeComponentClass() const;

	UE_DEPRECATED(5.4, "Seting moved to WaterBodyActor")
	UE_API TSubclassOf<UWaterBodyOceanComponent> GetWaterBodyOceanComponentClass() const;

	UE_DEPRECATED(5.4, "Seting moved to WaterBodyActor")
	UE_API TSubclassOf<UWaterBodyCustomComponent> GetWaterBodyCustomComponentClass() const;

	//#todo_water: how can we put these settins on the editor settings but still access them from the WaterModule?
	bool ShouldWarnOnMismatchOceanExtent() const { return bWarnOnMismatchOceanExtent; }
	void SetShouldWarnOnMismatchOceanExtent(bool bEnable) { Modify(); bWarnOnMismatchOceanExtent = bEnable; }

	UE_API virtual void PostInitProperties() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	/** Collision channel to use for tracing and blocking water bodies */
	UPROPERTY(EditAnywhere, config, Category = Collision)
	TEnumAsByte<ECollisionChannel> CollisionChannelForWaterTraces;

	/** Material Parameter Collection for everything water-related */
	UPROPERTY(EditAnywhere, config, Category = Rendering)
	TSoftObjectPtr<UMaterialParameterCollection> MaterialParameterCollection;

	/** Offset in Z for the water body icon in world-space. */
	UPROPERTY(EditAnywhere, config, Category = Rendering)
	float WaterBodyIconWorldZOffset = 75.0f;

#if WITH_EDITORONLY_DATA
	// Delegate called whenever the curve data is updated
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnUpdateSettings, const UWaterRuntimeSettings* /*Settings*/, EPropertyChangeType::Type /*ChangeType*/);
	static UE_API FOnUpdateSettings OnSettingsChange;
#endif

private:
	/** Default collision profile name of water bodies */
	UPROPERTY(VisibleAnywhere, config, Category = Collision)
	FName DefaultWaterCollisionProfileName;

	UPROPERTY(EditAnywhere, config, Category = Water)
	TSoftObjectPtr<UMaterialInterface> DefaultWaterInfoMaterial;
	
	UPROPERTY(Config, meta=(DeprecatedProperty, DeprecationMessage = "Moved to WaterBodyActor", MetaClass = "/Script/Water.WaterBodyRiverComponent"))
	TSubclassOf<UWaterBodyRiverComponent> WaterBodyRiverComponentClass_DEPRECATED;

	UPROPERTY(Config, meta=(DeprecatedProperty, DeprecationMessage = "Moved to WaterBodyActor", MetaClass = "/Script/Water.WaterBodyLakeComponent"))
	TSubclassOf<UWaterBodyLakeComponent> WaterBodyLakeComponentClass_DEPRECATED;

	UPROPERTY(Config, meta=(DeprecatedProperty, DeprecationMessage = "Moved to WaterBodyActor", MetaClass = "/Script/Water.WaterBodyOceanComponent"))
	TSubclassOf<UWaterBodyOceanComponent> WaterBodyOceanComponentClass_DEPRECATED;

	UPROPERTY(Config, meta=(DeprecatedProperty, DeprecationMessage = "Moved to WaterBodyActor", MetaClass = "/Script/Water.WaterBodyCustomComponent"))
	TSubclassOf<UWaterBodyCustomComponent> WaterBodyCustomComponentClass_DEPRECATED;

	// #todo_water: move this
	/** If enabled, MapCheck will notify users that their ocean does not completely fill the water zone. This can be desirable in some cases but the default should be to fill completely. */
	UPROPERTY(EditAnywhere, config, Category = Water)
	bool bWarnOnMismatchOceanExtent = true;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float WaterBodyIconWorldSize_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
};

#undef UE_API
