// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterSplineMetadata.h"
#include "WaterSplineComponent.generated.h"

#define UE_API WATER_API

#if WITH_EDITOR
struct FOnWaterSplineDataChangedParams
{
	FOnWaterSplineDataChangedParams(const FPropertyChangedEvent& InPropertyChangedEvent = FPropertyChangedEvent(/*InProperty = */nullptr)) 
		: PropertyChangedEvent(InPropertyChangedEvent)
	{}

	/** Provides some additional context about how the water brush actor data has changed (property, type of change...) */
	FPropertyChangedEvent PropertyChangedEvent;

	/** Indicates user initiated change*/
	bool bUserTriggered = false;
};
DECLARE_MULTICAST_DELEGATE_OneParam(FOnWaterSplineDataChanged, const FOnWaterSplineDataChangedParams&);
#endif // WITH_EDITOR

UCLASS(MinimalAPI, ClassGroup = (Custom), meta = (BlueprintSpawnableComponent))
class UWaterSplineComponent : public USplineComponent
{
	GENERATED_UCLASS_BODY()
public:
	/**
	 * Defaults which are used to propagate values to spline points on instances of this in the world
	 */
	UPROPERTY(Category = Water, EditDefaultsOnly)
	FWaterSplineCurveDefaults WaterSplineDefaults;

	/** 
	 * This stores the last defaults propagated to spline points on an instance of this component 
	 *  Used to determine if spline points were modifed by users or if they exist at a current default value
	 */
	UPROPERTY()
	FWaterSplineCurveDefaults PreviousWaterSplineDefaults;
public:
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPie) override;

	/** Spline component interface */
	UE_API virtual USplineMetadata* GetSplinePointsMetadata() override;
	UE_API virtual const USplineMetadata* GetSplinePointsMetadata() const override;

	UE_API virtual TArray<ESplinePointType::Type> GetEnabledSplinePointTypes() const override;

	virtual bool AllowsSplinePointScaleEditing() const override { return false; }

	/*
	 * Call to update water spline
	 * Necessary if using USplineComponent::AddPoint(s) instead of editing the spline in editor
	 */
	UFUNCTION(BlueprintCallable, Category = Water, DisplayName="Synchronize And Broadcast Data Change")
	UE_API void K2_SynchronizeAndBroadcastDataChange();

#if WITH_EDITOR
	FOnWaterSplineDataChanged& OnWaterSplineDataChanged() { return WaterSplineDataChangedEvent; }

	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditImport() override;
	
	UE_API void ResetSpline(const TArray<FVector>& Points);
	UE_API bool SynchronizeWaterProperties();
#endif // WITH_EDITOR

	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	UE_API virtual void Serialize(FArchive& Ar) override;

private:
#if WITH_EDITOR
	FOnWaterSplineDataChanged WaterSplineDataChangedEvent;
#endif // WITH_EDITOR
};

#undef UE_API
