// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SplineComponent.h"

#include "WaterSplineMetadata.generated.h"

#define UE_API WATER_API


#if WITH_EDITOR

class UWaterSplineMetadata;

struct FOnWaterSplineMetadataChangedParams
{
	FOnWaterSplineMetadataChangedParams(const FPropertyChangedEvent& InPropertyChangedEvent = FPropertyChangedEvent(/*InProperty = */nullptr)) 
		: PropertyChangedEvent(InPropertyChangedEvent)
	{}

	/** Provides some additional context about how the water brush actor data has changed (property, type of change...) */
	FPropertyChangedEvent PropertyChangedEvent;

	UWaterSplineMetadata* WaterSplineMetadata = nullptr;
		
	/** Indicates user initiated change*/
	bool bUserTriggered = false;
};
#endif // WITH_EDITOR

USTRUCT()
struct FWaterSplineCurveDefaults
{
	GENERATED_BODY()

	FWaterSplineCurveDefaults()
		: DefaultDepth(150.f)
		, DefaultWidth(2048.f)
		, DefaultVelocity(128.f)
		, DefaultAudioIntensity(1.f)
	{}

	UPROPERTY(EditDefaultsOnly, Category=WaterCurveDefaults)
	float DefaultDepth;

	UPROPERTY(EditDefaultsOnly, Category = WaterCurveDefaults)
	float DefaultWidth;
	
	UPROPERTY(EditDefaultsOnly, Category = WaterCurveDefaults)
	float DefaultVelocity;

	UPROPERTY(EditDefaultsOnly, Category = WaterCurveDefaults)
	float DefaultAudioIntensity;

};
UCLASS(MinimalAPI)
class UWaterSplineMetadata : public USplineMetadata
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	UE_API bool CanEditRiverWidth() const;
	UE_API bool CanEditDepth() const;
	UE_API bool CanEditVelocity() const;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	/** Insert point before index, lerping metadata between previous and next key values */
	UE_API virtual void InsertPoint(int32 Index, float t, bool bClosedLoop) override;
	/** Update point at index by lerping metadata between previous and next key values */
	UE_API virtual void UpdatePoint(int32 Index, float t, bool bClosedLoop) override;
	UE_API virtual void AddPoint(float InputKey) override;
	UE_API virtual void RemovePoint(int32 Index) override;
	UE_API virtual void DuplicatePoint(int32 Index) override;
	UE_API virtual void CopyPoint(const USplineMetadata* FromSplineMetadata, int32 FromIndex, int32 ToIndex) override;
	UE_API virtual void Reset(int32 NumPoints) override;
	UE_API virtual void Fixup(int32 NumPoints, USplineComponent* SplineComp) override;

	UE_API bool PropagateDefaultValue(int32 PointIndex, const FWaterSplineCurveDefaults& CurrentDefaults, const FWaterSplineCurveDefaults& NewDefaults);

	UPROPERTY(EditAnywhere, Category="Water")
	FInterpCurveFloat Depth;

	/** The Current of the water at this vertex.  Magnitude and direction */
	UPROPERTY(EditAnywhere, Category = "Water")
	FInterpCurveFloat WaterVelocityScalar;

	/** Rivers Only: The width of the river (from center) in each direction  */
	UPROPERTY(EditAnywhere, Category = "Water")
	FInterpCurveFloat RiverWidth;

	/** A scalar used to define intensity of the water audio along the spline */
	UPROPERTY(EditAnywhere, Category = "Water|Audio")
	FInterpCurveFloat AudioIntensity;

#if WITH_EDITORONLY_DATA
	/** Whether water velocity visualization should be displayed */
	UPROPERTY(EditAnywhere, Category = "Water", meta = (InlineEditConditionToggle = true))
	bool bShouldVisualizeWaterVelocity;

	/** Whether river width visualization should be displayed */
	UPROPERTY(EditAnywhere, Category = "Water", meta = (InlineEditConditionToggle = true))
	bool bShouldVisualizeRiverWidth;

	/** Whether depth visualization should be displayed */
	UPROPERTY(EditAnywhere, Category = "Water", meta = (InlineEditConditionToggle = true))
	bool bShouldVisualizeDepth;

	// Delegate called whenever the metadata is updated
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWaterSplineMetadataChanged, const FOnWaterSplineMetadataChangedParams&);
	FOnWaterSplineMetadataChanged OnChangeMetadata;
	
#endif // WITH_EDITORONLY_DATA

private:

	UPROPERTY()
	FInterpCurveVector WaterVelocity_DEPRECATED;
};

#undef UE_API
