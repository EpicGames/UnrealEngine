// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "MaterialShared.h"
#include "Subsystems/WorldSubsystem.h"
#include "Interfaces/Interface_PostProcessVolume.h"
#include "WaterBodyManager.h"
#include "WaterTerrainComponent.h"
#include "WaterZoneActor.h"
#include "WaterSubsystem.generated.h"

#define UE_API WATER_API

class UWaterTerrainComponent;
class UStaticMesh;

DECLARE_STATS_GROUP(TEXT("Water"), STATGROUP_Water, STATCAT_Advanced);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnCameraUnderwaterStateChanged, bool, bIsUnderWater, float, DepthUnderwater);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnWaterScalabilityChanged);

class UWaterBodyComponent;
class UMaterialParameterCollection;
class UWaterRuntimeSettings;
class FSceneView;
class UTexture2D;
struct FUnderwaterPostProcessDebugInfo;
enum class EWaterBodyQueryFlags;
class ABuoyancyManager;
class FWaterViewExtension;

namespace UE::WaterInfo { struct FRenderingContext; }

bool IsWaterEnabled(bool bIsRenderThread);

UCLASS(Transient, MinimalAPI)
class UUnderwaterPostProcessVolume : public UObject, public IInterface_PostProcessVolume
{
	GENERATED_BODY()

public:
	UUnderwaterPostProcessVolume()
		: PostProcessProperties()
	{
		PostProcessProperties.VolumeGuid = FGuid(0x857a3d0e, 0x222e4c40, 0x833b5e81, 0x499b8d95);
	}

	virtual bool EncompassesPoint(FVector Point, float SphereRadius/*=0.f*/, float* OutDistanceToPoint) override
	{
		// For underwater, the distance to point is 0 for now because underwater doesn't look correct if it is blended with other post process due to the wave masking
		if (OutDistanceToPoint)
		{
			*OutDistanceToPoint = 0;
		}

		// If post process properties are enabled and valid return true.  We already computed if it encompasses the water volume earlier
		return PostProcessProperties.bIsEnabled && PostProcessProperties.Settings;
	}

	virtual FPostProcessVolumeProperties GetProperties() const override
	{
		return PostProcessProperties;
	}

#if DEBUG_POST_PROCESS_VOLUME_ENABLE
	virtual FString GetDebugName() const override
	{
		return FString("UnderwaterPostProcessVolume");
	}
#endif

	FPostProcessVolumeProperties PostProcessProperties;
};

/**
 * This is the API used to get information about water at runtime
 */
UCLASS(MinimalAPI, BlueprintType, Transient)
class UWaterSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()

public:
	UE_API UWaterSubsystem();

	// FTickableGameObject implementation Begin
	virtual bool IsTickable() const override { return true; }
	virtual bool IsTickableInEditor() const override { return true; }
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual TStatId GetStatId() const override;
	// FTickableGameObject implementation End

	// UWorldSubsystem implementation Begin
	/** Override to support water subsystems in editor preview worlds */
	UE_API virtual bool DoesSupportWorldType(EWorldType::Type WorldType) const override;
	// UWorldSubsystem implementation End

	// USubsystem implementation Begin
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void PostInitialize() override;
	UE_API virtual void Deinitialize() override;
	// USubsystem implementation End

	/** Static helper function to get a water subsystem from a world, returns nullptr if world or subsystem don't exist */
	static UE_API UWaterSubsystem* GetWaterSubsystem(const UWorld* InWorld);

	/** Static helper function to get a waterbody manager from a world, returns nullptr if world or manager don't exist */
	static UE_API FWaterBodyManager* GetWaterBodyManager(const UWorld* InWorld);

	/** Static helper function to get a weak ptr to the water scene view extension for a given world. */
	static UE_API FWaterViewExtension* GetWaterViewExtension(const UWorld* InWorld);

	/** Static helper function to get a TWeakPtr to the water scene view extension for a given world. */
	static TWeakPtr<FWaterViewExtension, ESPMode::ThreadSafe> GetWaterViewExtensionWeakPtr(const UWorld* InWorld);

	ABuoyancyManager* GetBuoyancyManager() const { return BuoyancyManager; }

	TWeakObjectPtr<UWaterBodyComponent> GetOceanBodyComponent() { return OceanBodyComponent; }
	void SetOceanBodyComponent(TWeakObjectPtr<UWaterBodyComponent> InOceanBodyComponent) { OceanBodyComponent = InOceanBodyComponent; }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category=Water)
	UE_API bool IsShallowWaterSimulationEnabled() const;

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = Water)
	UE_API bool IsUnderwaterPostProcessEnabled() const;

	UFUNCTION(BlueprintCallable, Category = Water)
	static UE_API float GetUnderwaterCollisionTraceDistance();

	UFUNCTION(BlueprintCallable, Category = Water)
	static UE_API float GetUnderwaterPreciseTraceDistance();

	UFUNCTION(BlueprintCallable, Category=Water)
	static UE_API int32 GetShallowWaterMaxDynamicForces();

	UFUNCTION(BlueprintCallable, Category = Water)
	static UE_API int32 GetShallowWaterMaxImpulseForces();

	UFUNCTION(BlueprintCallable, Category = Water)
	static UE_API int32 GetShallowWaterSimulationRenderTargetSize();

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = Water)
	UE_API bool IsWaterRenderingEnabled() const;

	UFUNCTION(BlueprintCallable, Category = Water)
	UE_API float GetWaterTimeSeconds() const;

	UFUNCTION(BlueprintCallable, Category = Water)
	UE_API float GetSmoothedWorldTimeSeconds() const;

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = Water)
	float GetCameraUnderwaterDepth() const { return CachedDepthUnderwater; }

	UFUNCTION(BlueprintCallable, Category = Water)
	UE_API void PrintToWaterLog(const FString& Message, bool bWarning);

	/** Returns the base height of the ocean. This should correspond to its world Z position */
	UFUNCTION(BlueprintCallable, Category = Water)
	UE_API float GetOceanBaseHeight() const;

	/** Returns the relative flood height */
	UFUNCTION(BlueprintCallable, Category = Water)
	float GetOceanFloodHeight() const { return FloodHeight; }

	/** Returns the total height of the ocean. This should correspond to the base height plus any additional height, like flood for example */
	UFUNCTION(BlueprintCallable, Category = Water)
	float GetOceanTotalHeight() const { return GetOceanBaseHeight() + GetOceanFloodHeight(); }

	UFUNCTION(BlueprintCallable, Category = Water)
	UE_API void SetOceanFloodHeight(float InFloodHeight);

	UE_API void SetSmoothedWorldTimeSeconds(float InTime);
	
	UE_API void SetOverrideSmoothedWorldTimeSeconds(float InTime);
	float GetOverrideSmoothedWorldTimeSeconds() const { return OverrideWorldTimeSeconds; }
	
	UE_API void SetShouldOverrideSmoothedWorldTimeSeconds(bool bOverride);
	bool GetShouldOverrideSmoothedWorldTimeSeconds() const { return bUsingOverrideWorldTimeSeconds; }

	UE_API void SetShouldPauseWaveTime(bool bInPauseWaveTime);

	UMaterialParameterCollection* GetMaterialParameterCollection() const {	return MaterialParameterCollection; }
	
	UE_API void MarkAllWaterZonesForRebuild(EWaterZoneRebuildFlags RebuildFlags = EWaterZoneRebuildFlags::All, const UObject* DebugRequestingObject = nullptr);
	UE_API void MarkWaterZonesInRegionForRebuild(const FBox2D& InUpdateRegion, EWaterZoneRebuildFlags InRebuildFlags, const UObject* DebugRequestingObject = nullptr);

	/** Returns the water with the highest priority within the bounds provided. */
	static UE_API TSoftObjectPtr<AWaterZone> FindWaterZone(const UWorld* World, const FBox2D& Bounds, const TSoftObjectPtr<const ULevel> PreferredLevel = {});
	UE_API TSoftObjectPtr<AWaterZone> FindWaterZone(const FBox2D& Bounds, const TSoftObjectPtr<const ULevel> PreferredLevel = {}) const;

	UE_API void RegisterWaterTerrainComponent(UWaterTerrainComponent* WaterTerrainComponent);
	UE_API void UnregisterWaterTerrainComponent(UWaterTerrainComponent* WaterTerrainComponent);

	/** Returns a list of all water terrain components registered to the water subsystem. Can be used instead of searching all actors to find them. */
	UE_API void GetWaterTerrainComponents(TArray<UWaterTerrainComponent*>& OutWaterTerrainComponents) const;

#if WITH_EDITOR
	UE_API void OnActorMoved(AActor* MovedActor);

	/** Little scope object to temporarily change the value of bAllowWaterSubsystemOnPreviewWorld */
	struct FScopedAllowWaterSubsystemOnPreviewWorld
	{
		UE_API FScopedAllowWaterSubsystemOnPreviewWorld(bool bNewValue);
		UE_API ~FScopedAllowWaterSubsystemOnPreviewWorld();

		// Non-copyable
	private:
		FScopedAllowWaterSubsystemOnPreviewWorld() = delete;
		FScopedAllowWaterSubsystemOnPreviewWorld& operator=(const FScopedAllowWaterSubsystemOnPreviewWorld&) = delete;
		FScopedAllowWaterSubsystemOnPreviewWorld(const FScopedAllowWaterSubsystemOnPreviewWorld&) = delete;

		bool bPreviousValue = false;
	};
	static void SetAllowWaterSubsystemOnPreviewWorld(bool bInValue) { bAllowWaterSubsystemOnPreviewWorld = bInValue; }
	static bool GetAllowWaterSubsystemOnPreviewWorld() { return bAllowWaterSubsystemOnPreviewWorld; }
#endif // WITH_EDITOR

private:
	void NotifyWaterScalabilityChangedInternal(IConsoleVariable* CVar);
	void NotifyWaterVisibilityChangedInternal(IConsoleVariable* CVar);
	void ComputeUnderwaterPostProcess(FVector ViewLocation, FSceneView* SceneView);
	void SetMPCTime(float Time, float PrevTime);
	void AdjustUnderwaterWaterInfoQueryFlags(EWaterBodyQueryFlags& InOutFlags);
	void ApplyRuntimeSettings(const UWaterRuntimeSettings* Settings, EPropertyChangeType::Type ChangeType);

	void OnMarkRenderStateDirty(UActorComponent& Component);

	void OnWaterTerrainActorChanged(const AActor* TerrainActor);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	void ShowOnScreenDebugInfo(const FVector& InViewLocation, const FUnderwaterPostProcessDebugInfo& InDebugInfo);
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

public:
	UPROPERTY(Transient)
	TObjectPtr<ABuoyancyManager> BuoyancyManager;

	DECLARE_EVENT_OneParam(UWaterSubsystem, FOnWaterSubsystemInitialized, UWaterSubsystem*)
	static UE_API FOnWaterSubsystemInitialized OnWaterSubsystemInitialized;

	UPROPERTY(BlueprintAssignable, Category = Water)
	FOnCameraUnderwaterStateChanged OnCameraUnderwaterStateChanged;

	UPROPERTY(BlueprintAssignable, Category = Water)
	FOnWaterScalabilityChanged OnWaterScalabilityChanged;

	UPROPERTY()
	TObjectPtr<UStaticMesh> DefaultRiverMesh;

	UPROPERTY()
	TObjectPtr<UStaticMesh> DefaultLakeMesh;

private:

	TWeakObjectPtr<UWaterBodyComponent> OceanBodyComponent;

	ECollisionChannel UnderwaterTraceChannel;

	float CachedDepthUnderwater;
	float SmoothedWorldTimeSeconds;
	float NonSmoothedWorldTimeSeconds;
	float PrevWorldTimeSeconds;
	float OverrideWorldTimeSeconds;
	float FloodHeight = 0.0f;
	bool bUsingSmoothedTime;
	bool bUsingOverrideWorldTimeSeconds;
	bool bUnderWaterForAudio;
	bool bPauseWaveTime;

	/** The parameter collection asset that holds the global parameters that are updated by this actor */
	UPROPERTY()
	TObjectPtr<UMaterialParameterCollection> MaterialParameterCollection;

	UPROPERTY()
	TObjectPtr<UUnderwaterPostProcessVolume> UnderwaterPostProcessVolume;

	FWaterBodyManager WaterBodyManager;

	/**
	 * Keeps track of all actors that have WaterTerrainComponents so we can avoid extra work iterating
	 * every actors components to find one when global events are triggered.
	 */
	TMultiMap<const AActor*, TWeakObjectPtr<UWaterTerrainComponent>> WaterTerrainActors;
	FDelegateHandle OnMarkRenderStateDirtyHandle{};

#if WITH_EDITOR
	/** By default, there is no water subsystem allowed on preview worlds except when explicitly requested : */
	static bool bAllowWaterSubsystemOnPreviewWorld;
#endif // WITH_EDITOR
};

#undef UE_API
