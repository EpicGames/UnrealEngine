// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "WaterTerrainComponent.generated.h"

#define UE_API WATER_API

class AWaterZone;

/**
 * Water Terrain Component can be attached to any actor with primitive components to allow them to render into a Water Info Texture as the terrain.
 */
UCLASS(MinimalAPI, meta=(BlueprintSpawnableComponent))
class UWaterTerrainComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	/** Returns a list of all Primitive Components that should render as the terrain */
	UE_API virtual TArray<UPrimitiveComponent*> GetTerrainPrimitives() const;

	UE_API virtual FBox2D GetTerrainBounds() const;

	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;

	UE_API virtual bool AffectsWaterZone(AWaterZone* WaterZone) const;
protected:

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
protected:
	/** By default, the terrain component will be rendering into any overlapping water zone.
	 * If the override is set, it will only render to that specific water zone. */
	UPROPERTY(EditAnywhere, Category = Water)
	TSoftObjectPtr<AWaterZone> WaterZoneOverride;
};

#undef UE_API
