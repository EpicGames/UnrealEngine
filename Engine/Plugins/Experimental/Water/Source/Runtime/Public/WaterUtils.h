// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

enum ETextureRenderTargetFormat : int;

class UMaterialInstanceDynamic;
class UMaterialInterface;
class UTextureRenderTarget2D;
class UTextureRenderTarget2DArray;

struct FWaterUtils
{
	/** 
	 *  Creates a transient UMaterialInstanceDynamic out of the material interface in input or returns the existing one if it's compatible (same parent material)
	 * @param InMID is the current MID currently used. It will be the one returned if it's already compatible with InMaterialInterface
	 * @param InMIDName is the UObject's name in case we need to create a MID
	 * @param InMaterialInterface is the parent material interface of the desired MID: can be a UMaterial or UMaterialInstanceConstant, in which case 
	    a MID can be created, or a UMaterialInstanceDynamic, which will just be returned as-is (as we can't create MID out of another). In the latter case, it is expected to be a transient MID.
		InMID is read-only so it's up to the caller to decide to update or not the MID. Usually, the calling code will be : MID = FWaterUtils::GetOrCreateTransientMID(MID, ...)
	* @param InAdditionalObjectFlags is the additional flags that should be set on the newly-created object, if any (it will come out with at least RF_Transient as the name of the function implies)
	 * @return a compatible transient MID if InMaterialInterface is valid, nullptr otherwise
	 */
	static WATER_API UMaterialInstanceDynamic* GetOrCreateTransientMID(UMaterialInstanceDynamic* InMID, FName InMIDName, UMaterialInterface* InMaterialInterface, EObjectFlags InAdditionalObjectFlags = RF_NoFlags);

	/** 
	 *  Creates a transient render target of the proper size/format/... or returns the existing one if it's compatible.
	     InRenderTarget is read-only so it's up to the caller to decide to update or not the RT. Usually, the calling code will be : RT = FWaterUtils::GetOrCreateTransientRenderTarget2D(RT, ...)
	 * @param InRenderTarget is the current render target currently used. It will be the one returned if it's already compatible with the size/format/...
	 * @param InRenderTargetName is the UObject's name in case we need to create a MID
	 * @param InSize is the render target's size
	 * @param InFormat is the render target's format
	 * @param InClearColor is the render target's default clear color
	 * @param bInAutoGenerateMipMaps is for generating mipmaps automatically after rendering to the render target
	 * @return a compatible transient render target if InMaterialInterface is size/format/... is valid, nullptr otherwise
	 */
	static WATER_API UTextureRenderTarget2D* GetOrCreateTransientRenderTarget2D(UTextureRenderTarget2D* InRenderTarget, FName InRenderTargetName, const FIntPoint& InSize, ETextureRenderTargetFormat InFormat, 
		const FLinearColor& InClearColor = FLinearColor::Black, bool bInAutoGenerateMipMaps = false);

	/**
	 *  Creates a transient render target of the proper size/format/... or returns the existing one if it's compatible.
		 InRenderTarget is read-only so it's up to the caller to decide to update or not the RT. Usually, the calling code will be : RT = FWaterUtils::GetOrCreateTransientRenderTarget2D(RT, ...)
	 * @param InRenderTarget is the current render target currently used. It will be the one returned if it's already compatible with the size/format/...
	 * @param InRenderTargetName is the UObject's name in case we need to create a MID
	 * @param InSize is the render target's size
	 * @param InSlices is the number of slices of the render target
	 * @param InFormat is the render target's format
	 * @param InClearColor is the render target's default clear color
	 * @param bInAutoGenerateMipMaps is for generating mipmaps automatically after rendering to the render target
	 * @return a compatible transient render target if InMaterialInterface is size/format/... is valid, nullptr otherwise
	 */
	static WATER_API UTextureRenderTarget2DArray* GetOrCreateTransientRenderTarget2DArray(UTextureRenderTarget2DArray* InRenderTarget, FName InRenderTargetName, const FIntPoint& InSize, int32 InSlices, ETextureRenderTargetFormat InFormat,
		const FLinearColor& InClearColor = FLinearColor::Black, bool bInAutoGenerateMipMaps = false);

	static FGuid StringToGuid(const FString& InStr);

	static WATER_API bool IsWaterEnabled(bool bIsRenderThread);
	static WATER_API bool IsWaterMeshEnabled(bool bIsRenderThread);
	static WATER_API bool IsWaterMeshRenderingEnabled(bool bIsRenderThread);
	static WATER_API float GetWaterMaxFlowVelocity(bool bIsRenderThread);

	static FVector4f PackFlowData(float VelocityMagnitude, float DirectionAngle);
};
