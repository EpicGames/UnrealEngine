// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API WATER_API


// Custom serialization version for Water plugin
struct FWaterCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,
		// Refactor of AWaterBody into sub-classes, waves refactor, etc.
		WaterBodyRefactor,
		// Transfer of TerrainCarvingSettings from landmass to water
		MoveTerrainCarvingSettingsToWater,
		// WaterBrushManager now can specify its own brush materials instead of using those from the default water editor settings : 
		MoveBrushMaterialsToWaterBrushManager,
		// Deprecate pontoons data on UBuoyancyComponent
		UpdateBuoyancyComponentPontoonsData,
		// Move JumpFlood materials into AWaterBrushManager
		MoveJumpFloodMaterialsToWaterBrushManager,
		// Fixup Gerstner waves that were not recomputed at the right moment
		FixupUnserializedGerstnerWaves,
		// Move responsability of updating the Water MPC params to WaterMesh
		MoveWaterMPCParamsToWaterMesh,
		// Moved where the default water info material is assigned to prevent blocking async load thread
		DefaultWaterInfoMaterialAssignmentMoved,
		// Rebuild water render data to ensure no meshes from the broken tset change were serialized
		RebuildWaterMeshDataTSetChange,
		// Moved baked simulation data over to UObject for better memory usage and asset management
		MigrateShallowWaterSimulationToUObject,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;

private:
	FWaterCustomVersion() {}
};

#undef UE_API
