// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ShaderParameterMacros.h"
#include "VertexFactory.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "WaterInstanceDataBuffer.h"
#include "RHIResourceUtils.h"

class FShaderParameterMap;
struct FShaderCompilerEnvironment;

/**
 * Uniform buffer to hold parameters specific to this vertex factory. Only set up once
 */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FWaterVertexFactoryParameters, )
	SHADER_PARAMETER(float, LODScale)
	SHADER_PARAMETER(float, LeafSize)
	SHADER_PARAMETER(int32, NumQuadsPerTileSide)
	SHADER_PARAMETER(int32, NumQuadsLOD0)
	SHADER_PARAMETER(int32, NumDensities)
	SHADER_PARAMETER(int32, bRenderSelected)
	SHADER_PARAMETER(int32, bRenderUnselected)
	SHADER_PARAMETER(int32, bLODMorphingEnabled)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef<FWaterVertexFactoryParameters> FWaterVertexFactoryBufferRef;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FWaterVertexFactoryRaytracingParameters, )
	SHADER_PARAMETER_SRV(Buffer<float>, VertexBuffer)
	SHADER_PARAMETER(FVector4f, InstanceData0)
	SHADER_PARAMETER(FVector4f, InstanceData1)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef<FWaterVertexFactoryRaytracingParameters> FWaterVertexFactoryRaytracingParametersRef;

class FWaterMeshIndexBuffer : public FIndexBuffer
{
public:

	FWaterMeshIndexBuffer(int32 InNumQuadsPerSide) : NumQuadsPerSide(InNumQuadsPerSide) {}

	void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		// This is an optimized index buffer path for water tiles containing less than uint16 max vertices
		if (NumQuadsPerSide < 256)
		{
			IndexBufferRHI = CreateIndexBuffer<uint16>(RHICmdList);
		}
		else
		{
			IndexBufferRHI = CreateIndexBuffer<uint32>(RHICmdList);
		}
	}

	int32 GetIndexCount() const { return NumIndices; };

private:
	template <typename IndexType>
	FBufferRHIRef CreateIndexBuffer(FRHICommandListBase& RHICmdList)
	{
		TArray<IndexType> Indices;

		// Allocate room for indices
		Indices.Reserve(NumQuadsPerSide * NumQuadsPerSide * 6);

		// Build index buffer in morton order for better vertex reuse. This amounts to roughly 75% reuse rate vs 66% of naive scanline approach
		for (int32 Morton = 0; Morton < NumQuadsPerSide * NumQuadsPerSide; Morton++)
		{
			int32 SquareX = FMath::ReverseMortonCode2(Morton);
			int32 SquareY = FMath::ReverseMortonCode2(Morton >> 1);

			bool ForwardDiagonal = false;

			if (SquareX % 2)
			{
				ForwardDiagonal = !ForwardDiagonal;
			}
			if (SquareY % 2)
			{
				ForwardDiagonal = !ForwardDiagonal;
			}

			int32 Index0 = SquareX + SquareY * (NumQuadsPerSide + 1);
			int32 Index1 = Index0 + 1;
			int32 Index2 = Index0 + (NumQuadsPerSide + 1);
			int32 Index3 = Index2 + 1;

			Indices.Add(Index3);
			Indices.Add(Index1);
			Indices.Add(ForwardDiagonal ? Index2 : Index0);
			Indices.Add(Index0);
			Indices.Add(Index2);
			Indices.Add(ForwardDiagonal ? Index1 : Index3);
		}

		NumIndices = Indices.Num();

		// Create index buffer. Fill buffer with initial data upon creation
		return UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FWaterMeshIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Indices));
	}

	int32 NumIndices = 0;
	const int32 NumQuadsPerSide = 0;
};


class FWaterMeshVertexBuffer : public FVertexBuffer
{
public:

	FWaterMeshVertexBuffer(int32 InNumQuadsPerSide) : NumQuadsPerSide(InNumQuadsPerSide) {}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		ensureAlways(NumQuadsPerSide > 0);
		const uint32 NumVertsPerSide = NumQuadsPerSide + 1;

		NumVerts = NumVertsPerSide * NumVertsPerSide;

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<FVector4f>(TEXT("FWaterMeshVertexBuffer"), NumVerts)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask)
			.SetInitActionInitializer();

		TRHIBufferInitializer<FVector4f> DummyContents = RHICmdList.CreateBufferInitializer(CreateDesc);

		for (uint32 VertY = 0; VertY < NumVertsPerSide; VertY++)
		{
			FVector4f VertPos;
			VertPos.Y = (float)VertY / NumQuadsPerSide - 0.5f;

			for (uint32 VertX = 0; VertX < NumVertsPerSide; VertX++)
			{
				VertPos.X = (float)VertX / NumQuadsPerSide - 0.5f;

				DummyContents[NumVertsPerSide * VertY + VertX] = VertPos;
			}
		}

		VertexBufferRHI = DummyContents.Finalize();
		SRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_FLOAT));
	}

	virtual void ReleaseRHI() override
	{
		SRV.SafeRelease();
		FVertexBuffer::ReleaseRHI();
	}

	int32 GetVertexCount() const { return NumVerts; }
	FRHIShaderResourceView* GetSRV() { return SRV; }

private:
	int32 NumVerts = 0;
	const int32 NumQuadsPerSide = 0;

	FShaderResourceViewRHIRef SRV;
};

enum class EWaterMeshRenderGroupType : uint8
{
	RG_RenderWaterTiles = 0,				// Render all water bodies

#if WITH_WATER_SELECTION_SUPPORT
	RG_RenderSelectedWaterTilesOnly,		// Render only selected water bodies
	RG_RenderUnselectedWaterTilesOnly,		// Render only unselected water bodies
#endif // WITH_WATER_SELECTION_SUPPORT
};

// Water supports both a CPU-driven (non-indirect draws) and a GPU-driven (indirect draws) path.
// The regular GPU-driven path does not support ISR out of the box, so there is a special path for indirect draws supporting ISR.
enum class EWaterVertexFactoryDrawMode
{
	// Non-indirect draw calls for CPU-driven water rendering
	NonIndirect,
	
	// Indirect draw calls for GPU-driven water rendering using the GPU water quadtree to generate indirect draw calls.
	// Uses vertex streams (fixed function) to push instance data into the vertex shader.
	Indirect,

	// Indirect draw calls with support for ISR. Uses manual fetching of instance data in the vertex shader/factory.
	IndirectInstancedStereo,
};

template <bool bWithWaterSelectionSupport, EWaterVertexFactoryDrawMode DrawMode>
class TWaterVertexFactory : public FVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE(FWaterVertexFactoryType);

public:
	using Super = FVertexFactory;
	using FWaterVertexFactoryType = TWaterVertexFactory<bWithWaterSelectionSupport, DrawMode>;

	static constexpr int32 NumRenderGroups = bWithWaterSelectionSupport ? 3 : 1; // Must match EWaterMeshRenderGroupType
	static constexpr int32 NumAdditionalVertexStreams = TWaterInstanceDataBuffers<bWithWaterSelectionSupport>::NumBuffers;

	TWaterVertexFactory(ERHIFeatureLevel::Type InFeatureLevel, int32 InNumQuadsPerSide, int32 InNumQuadsLOD0, int32 InNumDensities, float InLeafSize, float InLODScale);
	~TWaterVertexFactory();

	/**
	* Constructs render resources for this vertex factory.
	*/
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/**
	* Release render resources for this vertex factory.
	*/
	virtual void ReleaseRHI() override;

	/**
	 * Should we cache the material's shader type on this platform with this vertex factory?
	 */
	static bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	static void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	static void ValidateCompiledResult(const FVertexFactoryType* Type, EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors);

	static void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);

	inline const FUniformBufferRHIRef GetWaterVertexFactoryUniformBuffer(EWaterMeshRenderGroupType InRenderGroupType) const { return UniformBuffers[(int32)InRenderGroupType]; }

	static constexpr bool UsesIndirectDraws() { return DrawMode == EWaterVertexFactoryDrawMode::Indirect || DrawMode == EWaterVertexFactoryDrawMode::IndirectInstancedStereo; }
	static constexpr bool UsesInstancedStereo() { return DrawMode == EWaterVertexFactoryDrawMode::IndirectInstancedStereo; }

private:
	void SetupUniformDataForGroup(EWaterMeshRenderGroupType InRenderGroupType);

public:
	FWaterMeshVertexBuffer* VertexBuffer = nullptr;
	FWaterMeshIndexBuffer* IndexBuffer = nullptr;

private:
	TStaticArray<FWaterVertexFactoryBufferRef, NumRenderGroups> UniformBuffers;

	const int32 NumQuadsPerSide = 0;
	const int32 NumQuadsLOD0 = 0;
	const int32 NumDensities = 0;
	const float LeafSize = 0.0f;
	const float LODScale = 0.0f;
};

extern const FVertexFactoryType* GetWaterVertexFactoryType(bool bWithWaterSelectionSupport, EWaterVertexFactoryDrawMode DrawMode);


/**
 * Water user data provided to FMeshBatchElement(s)
 */
template <bool bWithWaterSelectionSupport>
struct TWaterMeshUserData
{
	TWaterMeshUserData() = default;

	TWaterMeshUserData(EWaterMeshRenderGroupType InRenderGroupType, const TWaterInstanceDataBuffers<bWithWaterSelectionSupport>* InInstanceDataBuffers)
		: RenderGroupType(InRenderGroupType)
		, InstanceDataBuffers(InInstanceDataBuffers)
	{
	}

	EWaterMeshRenderGroupType RenderGroupType = EWaterMeshRenderGroupType::RG_RenderWaterTiles;
	const TWaterInstanceDataBuffers<bWithWaterSelectionSupport>* InstanceDataBuffers = nullptr;

#if RHI_RAYTRACING	
	FUniformBufferRHIRef WaterVertexFactoryRaytracingVFUniformBuffer = nullptr;
#endif

	FVector QuadTreePosition = FVector::ZeroVector;
	float CaptureDepthRange = 0.0f;
	FRHIBuffer* IndirectInstanceData0 = nullptr;
	FRHIBuffer* IndirectInstanceData1 = nullptr;
	FRHIBuffer* IndirectInstanceData2 = nullptr;
	FRHIBuffer* IndirectInstanceData3 = nullptr;
	FRHIShaderResourceView* IndirectInstanceDataOffsetsSRV = nullptr;
	FRHIShaderResourceView* IndirectInstanceData0SRV = nullptr;
	FRHIShaderResourceView* IndirectInstanceData1SRV = nullptr;
	FRHIShaderResourceView* IndirectInstanceData2SRV = nullptr;
	FRHIShaderResourceView* IndirectInstanceData3SRV = nullptr;
};

/**
 * List of per-"water render group" user data buffers : 
 */
template <bool bWithWaterSelectionSupport>
struct TWaterMeshUserDataBuffers
{
	using WaterMeshUserDataType = TWaterMeshUserData<bWithWaterSelectionSupport>;

	TWaterMeshUserDataBuffers(const TWaterInstanceDataBuffers<bWithWaterSelectionSupport>* InInstanceDataBuffers)
	{
		int32 Index = 0;
		UserData[Index++] = MakeUnique<WaterMeshUserDataType>(EWaterMeshRenderGroupType::RG_RenderWaterTiles, InInstanceDataBuffers);

#if WITH_WATER_SELECTION_SUPPORT
		if (bWithWaterSelectionSupport)
		{
			UserData[Index++] = MakeUnique<WaterMeshUserDataType>(EWaterMeshRenderGroupType::RG_RenderSelectedWaterTilesOnly, InInstanceDataBuffers);
			UserData[Index++] = MakeUnique<WaterMeshUserDataType>(EWaterMeshRenderGroupType::RG_RenderUnselectedWaterTilesOnly, InInstanceDataBuffers);
		}
#endif // WITH_WATER_SELECTION_SUPPORT
	}

	const WaterMeshUserDataType* GetUserData(EWaterMeshRenderGroupType InRenderGroupType)
	{
		return UserData[(int32)InRenderGroupType].Get();
	}

	TStaticArray<TUniquePtr<WaterMeshUserDataType>, TWaterVertexFactory<bWithWaterSelectionSupport, EWaterVertexFactoryDrawMode::NonIndirect>::NumRenderGroups> UserData;
};

#include "WaterVertexFactory.inl"
