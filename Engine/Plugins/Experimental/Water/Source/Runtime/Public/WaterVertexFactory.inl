// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderParameterUtils.h"
#include "RHIStaticStates.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "MeshMaterialShader.h"
#include "WaterInstanceDataBuffer.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "StereoRenderUtils.h"
#include "ShaderCompilerCore.h"

// ----------------------------------------------------------------------------------

template <bool bWithWaterSelectionSupport, EWaterVertexFactoryDrawMode DrawMode>
TWaterVertexFactory<bWithWaterSelectionSupport, DrawMode>::TWaterVertexFactory(ERHIFeatureLevel::Type InFeatureLevel, int32 InNumQuadsPerSide, int32 InNumQuadsLOD0, int32 InNumDensities, float InLeafSize, float InLODScale)
	: FVertexFactory(InFeatureLevel)
	, NumQuadsPerSide(InNumQuadsPerSide)
	, NumQuadsLOD0(InNumQuadsLOD0)
	, NumDensities(InNumDensities)
	, LeafSize(InLeafSize)
	, LODScale(InLODScale)
{
	VertexBuffer = new FWaterMeshVertexBuffer(NumQuadsPerSide);
	IndexBuffer = new FWaterMeshIndexBuffer(NumQuadsPerSide);
}

template <bool bWithWaterSelectionSupport, EWaterVertexFactoryDrawMode DrawMode>
TWaterVertexFactory<bWithWaterSelectionSupport, DrawMode>::~TWaterVertexFactory()
{
	delete VertexBuffer;
	delete IndexBuffer;
}

template <bool bWithWaterSelectionSupport, EWaterVertexFactoryDrawMode DrawMode>
void TWaterVertexFactory<bWithWaterSelectionSupport, DrawMode>::InitRHI(FRHICommandListBase& RHICmdList)
{
	Super::InitRHI(RHICmdList);

	// Setup the uniform data:
	SetupUniformDataForGroup(EWaterMeshRenderGroupType::RG_RenderWaterTiles);

#if WITH_WATER_SELECTION_SUPPORT
	SetupUniformDataForGroup(EWaterMeshRenderGroupType::RG_RenderSelectedWaterTilesOnly);
	SetupUniformDataForGroup(EWaterMeshRenderGroupType::RG_RenderUnselectedWaterTilesOnly);
#endif // WITH_WATER_SELECTION_SUPPORT

	VertexBuffer->InitResource(RHICmdList);
	IndexBuffer->InitResource(RHICmdList);

	// No streams should currently exist.
	check(Streams.Num() == 0);

	// Position stream for per vertex local position 
	FVertexStream PositionVertexStream;
	PositionVertexStream.VertexBuffer = VertexBuffer;
	PositionVertexStream.Stride = sizeof(FVector4f);
	PositionVertexStream.Offset = 0;
	PositionVertexStream.VertexStreamUsage = EVertexStreamUsage::Default;

	FVertexElement VertexPositionElement(Streams.Add(PositionVertexStream), 0, VET_Float4, 0, PositionVertexStream.Stride, false);

	// Vertex declaration
	FVertexDeclarationElementList Elements;
	Elements.Add(VertexPositionElement);

	// Adds all streams
	if constexpr (UsesIndirectDraws())
	{
		// Instanced stereo manually fetches instance data from buffers instead
		if (!UsesInstancedStereo())
		{
			FVertexStream InstanceDataVertexStream;
			InstanceDataVertexStream.VertexBuffer = nullptr;
			InstanceDataVertexStream.Stride = sizeof(uint32);
			InstanceDataVertexStream.Offset = 0;
			InstanceDataVertexStream.VertexStreamUsage = EVertexStreamUsage::Instancing;

			constexpr int NumBuffers = bWithWaterSelectionSupport ? 4 : 3;

			for (int i = 0; i < NumBuffers; ++i)
			{
				Elements.Add(FVertexElement(Streams.Add(InstanceDataVertexStream), 0, VET_UInt, 8 + i, InstanceDataVertexStream.Stride, true));
			}
		}
	}
	else if constexpr (NumAdditionalVertexStreams > 0)
	{
		// Simple instancing vertex stream with nullptr vertex buffer to be set at binding time
		FVertexStream InstanceDataVertexStream;
		InstanceDataVertexStream.VertexBuffer = nullptr;
		InstanceDataVertexStream.Stride = sizeof(FVector4f);
		InstanceDataVertexStream.Offset = 0;
		InstanceDataVertexStream.VertexStreamUsage = EVertexStreamUsage::Instancing;

		for (int32 StreamIdx = 0; StreamIdx < NumAdditionalVertexStreams; ++StreamIdx)
		{
			FVertexElement InstanceElement(Streams.Add(InstanceDataVertexStream), 0, VET_Float4, 8 + StreamIdx, InstanceDataVertexStream.Stride, true);
			Elements.Add(InstanceElement);
		}
	}

#if 0
	const bool bCanUseGPUScene = UseGPUScene(GMaxRHIShaderPlatform, GMaxRHIFeatureLevel);

	// Add support for primitiveID
	const uint8 Index = static_cast<uint8>(EVertexInputStreamType::Default);
	PrimitiveIdStreamIndex[Index] = -1;
	if (GetType()->SupportsPrimitiveIdStream() && bCanUseGPUScene)
	{
		// When the VF is used for rendering in normal mesh passes, this vertex buffer and offset will be overridden
		Elements.Add(AccessStreamComponent(FVertexStreamComponent(&GPrimitiveIdDummy, 0, 0, sizeof(uint32), VET_UInt, EVertexStreamUsage::Instancing), 13));
		PrimitiveIdStreamIndex[Index] = Elements.Last().StreamIndex;
	}
#endif

	InitDeclaration(Elements);
}

template <bool bWithWaterSelectionSupport, EWaterVertexFactoryDrawMode DrawMode>
void TWaterVertexFactory<bWithWaterSelectionSupport, DrawMode>::ReleaseRHI()
{
	for (auto& UniformBuffer : UniformBuffers)
	{
		UniformBuffer.SafeRelease();
	}

	if (VertexBuffer)
	{
		VertexBuffer->ReleaseResource();
	}

	if (IndexBuffer)
	{
		IndexBuffer->ReleaseResource();
	}

	Super::ReleaseRHI();
}

template <bool bWithWaterSelectionSupport, EWaterVertexFactoryDrawMode DrawMode>
void TWaterVertexFactory<bWithWaterSelectionSupport, DrawMode>::SetupUniformDataForGroup(EWaterMeshRenderGroupType InRenderGroupType)
{
	FWaterVertexFactoryParameters UniformParams;
	UniformParams.NumQuadsPerTileSide = NumQuadsPerSide;
	UniformParams.NumQuadsLOD0 = NumQuadsLOD0;
	UniformParams.NumDensities = NumDensities;
	UniformParams.LODScale = LODScale;
	UniformParams.LeafSize = LeafSize;
	UniformParams.bRenderSelected = true;
	UniformParams.bRenderUnselected = true;

#if WITH_WATER_SELECTION_SUPPORT
	UniformParams.bRenderSelected = (InRenderGroupType != EWaterMeshRenderGroupType::RG_RenderUnselectedWaterTilesOnly);
	UniformParams.bRenderUnselected = (InRenderGroupType != EWaterMeshRenderGroupType::RG_RenderSelectedWaterTilesOnly);
#endif // WITH_WATER_SELECTION_SUPPORT

	UniformBuffers[(int32)InRenderGroupType] = FWaterVertexFactoryBufferRef::CreateUniformBufferImmediate(UniformParams, UniformBuffer_MultiFrame);
}

template <bool bWithWaterSelectionSupport, EWaterVertexFactoryDrawMode DrawMode>
bool TWaterVertexFactory<bWithWaterSelectionSupport, DrawMode>::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	const bool bIsCompatibleWithWater = ((Parameters.MaterialParameters.MaterialDomain == MD_Surface) && Parameters.MaterialParameters.bIsUsedWithWater) || Parameters.MaterialParameters.bIsSpecialEngineMaterial;
	if (bIsCompatibleWithWater)
	{
		// Only compile the ISR/non-ISR indirect draw version if ISR is enabled/disabled
		if (UsesIndirectDraws())
		{
			const UE::StereoRenderUtils::FStereoShaderAspects Aspects(Parameters.Platform);
			const bool bMatchingISRConfig = UsesInstancedStereo() == Aspects.IsInstancedStereoEnabled();
			if (!bMatchingISRConfig)
			{
				return false;
			}
		}

		// Only let the editor platforms compile the permutations supporting selection
		const bool bSupportEditor = EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
		const bool bSupportCookedEditor = IsPCPlatform(Parameters.Platform) && !IsMobilePlatform(Parameters.Platform);

		return !bWithWaterSelectionSupport || bSupportEditor || bSupportCookedEditor;
	}
	return false;
}

template <bool bWithWaterSelectionSupport, EWaterVertexFactoryDrawMode DrawMode>
void TWaterVertexFactory<bWithWaterSelectionSupport, DrawMode>::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("WATER_MESH_FACTORY"), 1);
#if 0
	const bool bUseGPUSceneAndPrimitiveIdStream = 
		Type->SupportsPrimitiveIdStream() 
		&& UseGPUScene(Parameters.Platform, GetMaxSupportedFeatureLevel(Parameters.Platform))
		// TODO: support GPUScene on mobile
		&& !IsMobilePlatform(Parameters.Platform);

	OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), bUseGPUSceneAndPrimitiveIdStream);
#endif

	if (bWithWaterSelectionSupport)
	{
		OutEnvironment.SetDefine(TEXT("USE_VERTEXFACTORY_HITPROXY_ID"), TEXT("1"));
		OutEnvironment.SetDefine(TEXT("WITH_WATER_SELECTION_SUPPORT_VF"), TEXT("1"));
	}
	if (UsesIndirectDraws())
	{
		OutEnvironment.SetDefine(TEXT("WATER_MESH_DRAW_INDIRECT"), TEXT("1"));
		OutEnvironment.CompilerFlags.Add(CFLAG_IndirectDraw);
	}

	OutEnvironment.SetDefine(TEXT("RAY_TRACING_DYNAMIC_MESH_IN_WORLD_SPACE"), TEXT("1"));
}

template <bool bWithWaterSelectionSupport, EWaterVertexFactoryDrawMode DrawMode>
void TWaterVertexFactory<bWithWaterSelectionSupport, DrawMode>::ValidateCompiledResult(const FVertexFactoryType* Type, EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors)
{
#if 0
	if (Type->SupportsPrimitiveIdStream()
		&& UseGPUScene(Platform, GetMaxSupportedFeatureLevel(Platform))
		&& ParameterMap.ContainsParameterAllocation(FPrimitiveUniformShaderParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
	{
		OutErrors.AddUnique(*FString::Printf(TEXT("Shader attempted to bind the Primitive uniform buffer even though Vertex Factory %s computes a PrimitiveId per-instance.  This will break auto-instancing.  Shaders should use GetPrimitiveData(Parameters).Member instead of Primitive.Member."), Type->GetName()));
	}
#endif
}

template <bool bWithWaterSelectionSupport, EWaterVertexFactoryDrawMode DrawMode>
void TWaterVertexFactory<bWithWaterSelectionSupport, DrawMode>::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	// Add position stream
	Elements.Add(FVertexElement(0, 0, VET_Float4, 0, sizeof(FVector4f), false));

	// Add all the additional streams
	if constexpr (UsesIndirectDraws())
	{
		// Instanced stereo manually fetches instance data from buffers instead
		if (!UsesInstancedStereo())
		{
			Elements.Add(FVertexElement(1, 0, VET_UInt, 8, sizeof(uint32), true));
			Elements.Add(FVertexElement(2, 0, VET_UInt, 9, sizeof(uint32), true));
			Elements.Add(FVertexElement(3, 0, VET_UInt, 10, sizeof(uint32), true));
			if (bWithWaterSelectionSupport)
			{
				Elements.Add(FVertexElement(4, 0, VET_UInt, 11, sizeof(uint32), true));
			}
		}
	}
	else if constexpr (NumAdditionalVertexStreams > 0)
	{
		for (int32 StreamIdx = 0; StreamIdx < NumAdditionalVertexStreams; ++StreamIdx)
		{
			Elements.Add(FVertexElement(1 + StreamIdx, 0, VET_Float4, 8 + StreamIdx, sizeof(FVector4f), true));
		}
	}
}
