// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneViewExtension.h"
#include "WaterInfoRendering.h"
#include "Misc/ScopeLock.h"


class AWaterZone;
class FWaterMeshSceneProxy;

class FWaterViewExtension : public FWorldSceneViewExtension
{
public:
	FWaterViewExtension(const FAutoRegister& AutoReg, UWorld* InWorld);
	~FWaterViewExtension();

	void Initialize();
	void Deinitialize();

	// FSceneViewExtensionBase implementation : 
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;
	virtual void SetupViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual void PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;
	virtual void PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) override;
	virtual void PreRenderBasePass_RenderThread(FRDGBuilder& GraphBuilder, bool bDepthBufferIsPopulated) override;
	// End FSceneViewExtensionBase implementation

	WATER_API void MarkWaterInfoTextureForRebuild(const UE::WaterInfo::FRenderingContext& RenderContext);

	WATER_API void MarkGPUDataDirty();

	WATER_API void AddWaterZone(AWaterZone* InWaterZone);
	WATER_API void RemoveWaterZone(AWaterZone* InWaterZone);

	WATER_API bool GetZoneLocation(const AWaterZone* InWaterZone, int32 PlayerIndex, FVector& OutLocation) const;

	void CreateSceneProxyQuadtrees(FWaterMeshSceneProxy* SceneProxy);

	struct FWaterZoneInfo
	{
		UE::WaterInfo::FRenderingContext RenderContext;

		/**
		 * For each water zone, per view: store the bounds of the tile from which the water zone was last rendered.
		 * When the view location crosses the bounds, submit a new update to reflect the new active area
		 */
		struct FWaterZoneViewInfo
		{
			TOptional<FBox2D> UpdateBounds = FBox2D(ForceInit);
			FVector Center = FVector(ForceInit);
			FWaterMeshSceneProxy* OldSceneProxy = nullptr;
			// bIsDirty is used to force water info texture update for this WaterZone
			bool bIsDirty = true;
			// bShouldUpdateQuadtree is set to true when the water info texture is update, 
			// to make sure the quadtree for this WaterZone is also updated
			bool bShouldUpdateQuadtree = false;
		};
		TArray<FWaterZoneViewInfo, TInlineAllocator<4>> ViewInfos;
	};

	const TWeakObjectPtrKeyMap<AWaterZone, FWaterZoneInfo>& GetWaterZoneInfos() const  { return WaterZoneInfos; };

private:
	int32 CurrentNumViews = 0;

	TWeakObjectPtrKeyMap<AWaterZone, FWaterZoneInfo> WaterZoneInfos;

	struct FWaterGPUResources
	{
		FBufferRHIRef WaterBodyDataBuffer;
		FShaderResourceViewRHIRef WaterBodyDataSRV;

		FBufferRHIRef AuxDataBuffer;
		FShaderResourceViewRHIRef AuxDataSRV;
	};

	TSharedRef<FWaterGPUResources, ESPMode::ThreadSafe> WaterGPUData;

	TArray<int32, TInlineAllocator<4>> ViewPlayerIndices;

	TMap<FSceneViewStateInterface*, int32> NonDataViewsQuadtreeKeys;

	bool bWaterInfoTextureRebuildPending = true;

	bool bRebuildGPUData = true;

	bool bRequestForcedBoundsUpdate = false;
	bool bForceBoundsUpdate = false;

	// this flag lets us know if we can skip looping through water zones for quadtrees updates, as an optimization
	bool bAnyQuadTreeUpdateRequired = false;

	// store the locations of every active water mesh scene proxy quad tree based on the key
	TMap<int32, FVector2D> QuadTreeKeyLocationMap;

	void UpdateGPUBuffers();

	void UpdateViewInfo(AWaterZone* WaterZone, const FSceneView& InView);
	
	void RenderWaterInfoTexture(FSceneViewFamily& InViewFamily, FSceneView& InView, const FWaterZoneInfo* WaterZoneInfo, FSceneInterface* Scene, const FVector& ZoneCenter);

	// Returns the index in the views array corresponding to InView's PlayerIndex. If the index is not found it adds a new entry.
	int32 GetOrAddViewindex(const FSceneView& InView);
	// Returns the index in the views array corresponding to InView's PlayerIndex. INDEX_NONE if it doesn't find it
	int32 GetViewIndex(int32 PlayerIndex) const;
	int32 GetViewIndex(const FSceneView& InView) const;

	bool ShouldHaveWaterZoneViewData(const FSceneView& InView) const;

	void DrawDebugInfo(const FSceneView& InView, AWaterZone* WaterZone);

	void OnWorldDestroyed(UWorld* InWorld);
};

struct FWaterMeshGPUWork
{
	struct FCallback
	{
		class FWaterMeshSceneProxy* Proxy = nullptr;
		TFunction<void(FRDGBuilder&, bool)> Function;
	};
	TArray<FCallback> Callbacks;
};

extern FWaterMeshGPUWork GWaterMeshGPUWork;
