// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaterWaves.generated.h"

#define UE_API WATER_API

class UWaterWaves;


UCLASS(MinimalAPI, EditInlineNew, BlueprintType, NotBlueprintable, Abstract)
class UWaterWavesBase : public UObject
{
	GENERATED_BODY()

public:
	/** Returns the maximum wave height that can be reached by those waves */
	virtual float GetMaxWaveHeight() const PURE_VIRTUAL(UWaterWavesBase::GetMaxWaveHeight, return 0.0f;)

	/** Return the underlying Water class type. This will jump through the potential UWaterWavesAssetReference to get to the actual wave data */
	virtual const UWaterWaves* GetWaterWaves() const PURE_VIRTUAL(UWaterWavesBase::GetWaterWaves, return nullptr;)
	virtual UWaterWaves* GetWaterWaves() PURE_VIRTUAL(UWaterWavesBase::GetWaterWaves, return nullptr;)

	/** Computes the raw wave perturbation of the water height/normal */
	virtual float GetWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime, FVector& OutNormal) const PURE_VIRTUAL(UWaterWavesBase::GetWaveHeightAtPosition, return 0.0f;)

	/** Computes the raw wave perturbation of the water height only (simple version : faster computation) */
	virtual float GetSimpleWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime) const PURE_VIRTUAL(UWaterWavesBase::GetSimpleWaveHeightAtPosition, return 0.0f;)

	/** Computes the attenuation factor to apply to the raw wave perturbation. Attenuates : normal/wave height/max wave height. */
	virtual float GetWaveAttenuationFactor(const FVector& InPosition, float InWaterDepth, float InTargetWaveMaskDepth) const PURE_VIRTUAL(UWaterWavesBase::GetWaveAttenuationFactor, return 0.0f;)

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	// Delegate called whenever the waves data is updated
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnUpdateWavesData, UWaterWavesBase* /*WaterWaves*/, EPropertyChangeType::Type /*ChangeType*/);
	FOnUpdateWavesData OnUpdateWavesData;
#endif // WITH_EDITORONLY_DATA
};

UCLASS(MinimalAPI, EditInlineNew, BlueprintType, NotBlueprintable, Abstract)
class UWaterWaves : public UWaterWavesBase
{
	GENERATED_BODY()

	virtual const UWaterWaves* GetWaterWaves() const override { return this; }
	virtual UWaterWaves* GetWaterWaves() override { return this; }
};

UCLASS(MinimalAPI, BlueprintType, NotBlueprintable, AutoExpandCategories = "Water Waves")
class UWaterWavesAsset : public UObject
{
	GENERATED_BODY()

public:
	UE_API UWaterWavesAsset();

	UE_API void SetWaterWaves(UWaterWaves* InWaterWaves);
	const UWaterWaves* GetWaterWaves() const { return WaterWaves; }
	UWaterWaves* GetWaterWaves() { return WaterWaves; }

	UE_API virtual void BeginDestroy() override;
#if WITH_EDITOR
	UE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

private:
#if WITH_EDITOR
	UE_API void OnWavesDataUpdated(UWaterWavesBase* InWaterWaves, EPropertyChangeType::Type InChangeType);
	UE_API void RegisterOnUpdateWavesData(UWaterWaves* InWaterWaves, bool bRegister);
#endif // WITH_EDITOR

public:
#if WITH_EDITORONLY_DATA
	// Delegate called whenever the waves data is updated
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnUpdateWavesAssetData, UWaterWavesAsset* /*WaterWavesAsset*/, EPropertyChangeType::Type /*ChangeType*/);
	FOnUpdateWavesAssetData OnUpdateWavesAssetData;
#endif // WITH_EDITORONLY_DATA

private:
	UPROPERTY(EditAnywhere, Category = "Water Waves", Instanced, DisplayName = "Waves Source")
	TObjectPtr<UWaterWaves> WaterWaves = nullptr;
};

UCLASS(MinimalAPI, BlueprintType, NotBlueprintable, AutoExpandCategories = "Water Waves Asset")
class UWaterWavesAssetReference : public UWaterWavesBase
{
	GENERATED_BODY()

public:
	UE_API void SetWaterWavesAsset(UWaterWavesAsset* InWaterWavesAsset);
	UE_API virtual void BeginDestroy() override;
	const UWaterWavesAsset* GetWaterWavesAsset(UWaterWaves* InWaterWavesAsset) const { return WaterWavesAsset; }

private:
	UPROPERTY(EditAnywhere, Category = "Water Waves Asset")
	TObjectPtr<UWaterWavesAsset> WaterWavesAsset;

public:
	virtual const UWaterWaves* GetWaterWaves() const override { return WaterWavesAsset ? WaterWavesAsset->GetWaterWaves() : nullptr; }
	virtual UWaterWaves* GetWaterWaves() override { return WaterWavesAsset ? WaterWavesAsset->GetWaterWaves() : nullptr; }

	UE_API virtual float GetMaxWaveHeight() const override;
	UE_API virtual float GetWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime, FVector& OutNormal) const override;
	UE_API virtual float GetSimpleWaveHeightAtPosition(const FVector& InPosition, float InWaterDepth, float InTime) const override;
	UE_API virtual float GetWaveAttenuationFactor(const FVector& InPosition, float InWaterDepth, float InTargetWaveMaskDepth) const override;

#if WITH_EDITOR
	UE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

private:
#if WITH_EDITOR
	UE_API void OnWavesAssetDataUpdated(UWaterWavesAsset* InWaterWavesAsset, EPropertyChangeType::Type InChangeType);
	UE_API void RegisterOnUpdateWavesAssetData(UWaterWavesAsset* InWaterWavesAsset, bool bRegister);
#endif // WITH_EDITOR
};

#undef UE_API
