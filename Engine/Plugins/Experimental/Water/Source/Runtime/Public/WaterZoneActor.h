// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "WaterZoneActor.generated.h"

#define UE_API WATER_API

class ALandscapeProxy;
class AWaterBody;
class FLandscapeProxyComponentDataChangedParams;
class UBillboardComponent;
class UBoxComponent;
class UTexture2D;
class UTextureRenderTarget2D;
class UTextureRenderTarget2DArray;
class UWaterBodyComponent;
class UWaterMeshComponent;
namespace EEndPlayReason { enum Type : int; }

enum class EWaterZoneRebuildFlags
{
	None = 0,
	UpdateWaterInfoTexture = (1 << 0),
	UpdateWaterMesh = (1 << 1),
	All = (~0),
};
ENUM_CLASS_FLAGS(EWaterZoneRebuildFlags);


DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnWaterInfoTextureArrayCreated, const UTextureRenderTarget2DArray*, WaterInfoTextureArray);

UCLASS(MinimalAPI, Blueprintable, HideCategories=(Physics, Replication, Input, Collision))
class AWaterZone : public AActor
{
	GENERATED_UCLASS_BODY()
public:
	UWaterMeshComponent* GetWaterMeshComponent() { return WaterMesh; }
	const UWaterMeshComponent* GetWaterMeshComponent() const { return WaterMesh; }

	/** Mark aspects of the water zone for rebuild based on the Flags parameter within a given region. Optionally the caller can pass in a UObject to identify who requested the update. */
	UE_API void MarkForRebuild(EWaterZoneRebuildFlags Flags, const FBox2D& RebuildRegion, const UObject* DebugRequestingObject = nullptr);
	/** Mark aspects of the water zone for rebuild based on the Flags parameter. Optionally the caller can pass in a UObject to identify who requested the update. */
	UE_API void MarkForRebuild(EWaterZoneRebuildFlags Flags, const UObject* DebugRequestingObject = nullptr);

	UE_API void Update();
		
	/** Execute a predicate function on each valid water body within the water zone. Predicate should return false for early exit. */
	UE_API void ForEachWaterBodyComponent(TFunctionRef<bool(UWaterBodyComponent*)> Predicate) const;

	UE_API void AddWaterBodyComponent(UWaterBodyComponent* WaterBodyComponent);
	UE_API void RemoveWaterBodyComponent(UWaterBodyComponent* WaterBodyComponent);

	UE_API FVector2D GetZoneExtent() const;
	UE_API void SetZoneExtent(FVector2D NewExtents);

	UE_API FBox2D GetZoneBounds2D() const;
	UE_API FBox GetZoneBounds() const;

	/** Retrieves all the per-view bounds for this water zone */
	UE_API void GetAllDynamicWaterInfoBounds(TArray<FBox>& OutBounds) const;
	/** Retrieves all the per-view centers for this water zone */
	UE_API void GetAllDynamicWaterInfoCenters(TArray<FVector>& OutCenters) const;

	UE_API void SetRenderTargetResolution(FIntPoint NewResolution);
	FIntPoint GetRenderTargetResolution() const;

	uint32 GetVelocityBlurRadius() const { return VelocityBlurRadius; }

	/** Retrieves the dynamic water info center for a specific player index in OutCenter. Returns true if a valid zone location has been found. */
	UE_API bool GetDynamicWaterInfoCenter(int32 PlayerIndex, FVector& OutCenter) const;
	/** Retrieves the dynamic water info bounds for a specific player index in OutBounds. Returns true if a valid zone location has been found.*/
	UE_API bool GetDynamicWaterInfoBounds(int32 PlayerIndex, FBox& OutBounds) const;

	UE_API FVector GetDynamicWaterInfoExtent() const;

	bool IsLocalOnlyTessellationEnabled() const { return bEnableLocalOnlyTessellation; }

	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual void PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostRegisterAllComponents() override;
	UE_API virtual void PostUnregisterAllComponents() override;
#if WITH_EDITORONLY_DATA
	static UE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	FVector2f GetWaterHeightExtents() const { return WaterHeightExtents; }
	float GetGroundZMin() const { return GroundZMin; }

	int32 GetOverlapPriority() const { return OverlapPriority; }

	UFUNCTION(BlueprintCallable, Category=Water)
	int32 GetWaterZoneIndex() const { return WaterZoneIndex; }

	UPROPERTY(Transient, DuplicateTransient, VisibleAnywhere, BlueprintReadOnly, Category = Water)
	TObjectPtr<UTextureRenderTarget2DArray> WaterInfoTextureArray;

	UPROPERTY(Transient, DuplicateTransient, VisibleAnywhere, BlueprintReadOnly, Category = Water)
	int32 WaterInfoTextureArrayNumSlices = 1;

	FOnWaterInfoTextureArrayCreated& GetOnWaterInfoTextureArrayCreated() { return OnWaterInfoTextureArrayCreated; }
	
#if WITH_EDITOR
	UE_API virtual TUniquePtr<class FWorldPartitionActorDesc> CreateClassActorDesc() const override;
	UE_API virtual void GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const override;
#endif //WITH_EDITOR

	UFUNCTION(BlueprintCallable, Category=Rendering)
	UE_API void SetFarMeshMaterial(UMaterialInterface* InFarMaterial);

private:

	/**
	 * Enqueues a command on the Water Scene View Extension to re-render the water info on the next frame.
	 * Returns false if the water info cannot be rendered this frame due to one of the dependencies not being ready yet (ie. a material under On-Demand-Shader-Compilation)
	 */
	UE_API bool UpdateWaterInfoTexture();

	/**
	 * Updates the list of owned water bodies causing any overlapping water bodies which do no have an owning water zone to register themselves to this water zone.
	 * Returns true if any any change to the list of owned water bodies occurred
	 */
	UE_API bool UpdateOverlappingWaterBodies();

	UE_API void OnExtentChanged();

	/** Delegates called when levels are added or removed from world. */
	UE_API void OnLevelAddedToWorld(ULevel* InLevel, UWorld* InWorld);
	UE_API void OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld);
	UE_API void OnLevelChanged(ULevel* InLevel, UWorld* InWorld);

	UE_API bool ContainsActorsAffectingWaterZone(ULevel* InLevel, const FBox& WaterZoneBounds) const;

	/** Returns true if the provided actor can affect waterzone resources.
	 *
	 * @param	InWaterZoneBounds	The bounds of this waterzone.
	 * @param	InActor		The Actor to check
	 * @return true if the provided actor can affect waterzone resources.
	 */
	UE_API bool IsAffectingWaterZone(const FBox& InWaterZoneBounds, const AActor* InActor) const;

#if WITH_EDITOR
	UE_API void OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh);
	
	UFUNCTION(CallInEditor, Category = Debug)
	UE_API void ForceUpdateWaterInfoTexture();

	UE_API virtual void PostEditMove(bool bFinished) override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostEditImport() override;

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Called when the Bounds component is modified. Updates the value of ZoneExtent to match the new bounds */
	UE_API void OnBoundsComponentModified();

	/** Called whenever a landscape component has its heightmap streamed */
	UE_API void OnLandscapeHeightmapStreamed(const struct FOnHeightmapStreamedContext& InContext);

	/** Called whenever a landscape component has its heightmap updated */
	UE_API void OnLandscapeComponentDataChanged(ALandscapeProxy* InLandscape, const FLandscapeProxyComponentDataChangedParams& InChangeParams);
#endif // WITH_EDITOR

private:

	UPROPERTY(Transient, Category = Water, VisibleAnywhere)
	TArray<TWeakObjectPtr<UWaterBodyComponent>> OwnedWaterBodies;

	/** Use Getter to access value since CVar r.Water.WaterInfo.RenderTargetResolutionMax may clamp the resolution */ 
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Water, meta = (AllowPrivateAccess = "true"))
	FIntPoint RenderTargetResolution;

	/** The water mesh component */
	UPROPERTY(VisibleAnywhere, Category = Water, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UWaterMeshComponent> WaterMesh;

	/** The maximum size in local space of the water zone. */
	UPROPERTY(Category = Water, EditAnywhere, BlueprintReadWrite, meta = (AllowPrivateAccess = "true"))
	FVector2D ZoneExtent;

	/** Offsets the height above the water zone at which the WaterInfoTexture is rendered. This is applied after computing the maximum Z of all the water bodies within the zone. */
	UPROPERTY(Category = Water, EditAnywhere, AdvancedDisplay, meta = (AllowPrivateAccess = "true"))
	float CaptureZOffset = 64.f;

	/** Determines if the WaterInfoTexture should be 16 or 32 bits per channel */
	UPROPERTY(Category = Water, EditAnywhere, AdvancedDisplay, meta = (AllowPrivateAccess = "true"))
	bool bHalfPrecisionTexture = true;

	/** Radius of the velocity blur in the finalize water info pass */
	UPROPERTY(Category = Water, EditAnywhere, AdvancedDisplay, meta = (ClampMin = 0, ClampMax = 16))
	int32 VelocityBlurRadius = 1;

	/** Higher number is higher priority. If Water Zones overlap and a water body does not have a manual water zone override, this priority will be used when automatically assigning the zone. */
	UPROPERTY(Category = Water, EditAnywhere, AdvancedDisplay)
	int32 OverlapPriority = 0;

	/**
	 * Enables the Local Tessellation mode for this water zone. In this mode, the WaterInfoTexture represents only a sliding window around the view location where the dynamically tessellated water mesh will be generated.
	 * The size of the sliding window is defined by the LocalTessellationExtent parameter which determines the diameters in world space units. In this mode, both the water info texture and water quad tree are regenerated
	 * at runtime.
	 */
	UPROPERTY(Category = LocalTessellation, EditAnywhere)
	bool bEnableLocalOnlyTessellation = false;

	/** The diameters in local space units for the region within which dynamic tessellation occurs. A smaller value increases the effective pixel density of the water info texture. */
	UPROPERTY(Category = LocalTessellation, EditAnywhere, meta = (EditCondition = "bEnableLocalOnlyTessellation"))
	FVector LocalTessellationExtent;

	/** When set to true, all landscape proxies that intersect with the bounds of this water zone will be included as ground actors regardless if they have WaterTerrain components. */
	UPROPERTY(Category = Terrain, EditAnywhere)
	bool bAutoIncludeLandscapesAsTerrain = true;

	bool bNeedsWaterInfoRebuild = true;

	FVector2f WaterHeightExtents;
	float GroundZMin;

	/** Unique Id for accessing zone data (Location, extent, ,...) in GPU buffers */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional, VisibleAnywhere, Category = Water)
	int32 WaterZoneIndex = INDEX_NONE;
	
	UPROPERTY(BlueprintAssignable, Category = Water)
	FOnWaterInfoTextureArrayCreated OnWaterInfoTextureArrayCreated;

#if WITH_EDITORONLY_DATA
	/** A manipulatable box for visualizing/editing the water zone bounds */
	UPROPERTY(Transient)
	TObjectPtr<UBoxComponent> BoundsComponent;

	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<AWaterBody>> SelectedWaterBodies;

	UPROPERTY(Transient)
	TObjectPtr<UBillboardComponent> ActorIcon;

	// Deprecated UPROPERTIES

	UPROPERTY(Transient)
	TObjectPtr<UTexture2D> WaterVelocityTexture_DEPRECATED;

	UPROPERTY()
	FVector TessellatedWaterMeshExtent_DEPRECATED;

	UPROPERTY()
	bool bEnableNonTesselatedLODMesh_DEPRECATED;

	FDelegateHandle OnHeightmapStreamedHandle;
	FDelegateHandle OnLandscapeComponentDataChangedHandle;
#endif // WITH_EDITORONLY_DATA
};
DEFINE_ACTORDESC_TYPE(AWaterZone, FWaterZoneActorDesc);

#undef UE_API
