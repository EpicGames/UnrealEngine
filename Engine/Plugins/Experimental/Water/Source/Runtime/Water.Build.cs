// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Water : ModuleRules
{
	public Water(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		StaticAnalyzerDisabledCheckers.Add("core.uninitialized.ArraySubscript");

		PrivateIncludePaths.AddRange(
			new string[] {
				System.IO.Path.Combine(GetModuleDirectory("Renderer"), "Private"),
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"Landmass",
				"RHI",
				"NavigationSystem",
				"AIModule",
				"GeometryCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"RenderCore",
				"Renderer",
				"NiagaraCore",
				"Niagara",
				"NiagaraShader",
				"Projects",
				"Landscape",
				"GeometryAlgorithms",
				"DynamicMesh",
				"MeshConversion",
				"ChaosCore",
				"Chaos",
				"PhysicsCore",
				"DeveloperSettings"
			}
		);

		bool bWithWaterSelectionSupport = false;
		if (Target.bBuildEditor)
        {
			bWithWaterSelectionSupport = true;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"MeshDescription"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"SourceControl",
					"UnrealEd",
					"StaticMeshDescription",
					"MeshMergeUtilities"
				}
			);
		}
		// Add a feature define instead of relying on the generic WITH_EDITOR define
		PublicDefinitions.Add("WITH_WATER_SELECTION_SUPPORT=" + (bWithWaterSelectionSupport ? 1 : 0));
	}
}
