{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Water",
	"Description": "Full suite of water tools and rendering techniques to easily add oceans, river, lakes or custom water bodies that carve landscape and interacts with gameplay",
	"Category": "Water",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "Water",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "WaterEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins": [
		{
			"Name": "Landmass",
			"Enabled": true
		},
		{
			"Name": "Niagara",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "BlueprintMaterialTextureNodes",
			"Enabled": true
		}
	]
}